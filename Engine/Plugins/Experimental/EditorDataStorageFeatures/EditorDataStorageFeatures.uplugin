{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "0.1",
	"FriendlyName": "TEDS: Editor Data Storage Features",
	"Description": "Experimental UI Features for the Editor, built on TEDS: Editor Data Storage.",
	"Category": "Editor",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "http://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": true,
	"IsBetaVersion": false,
	"IsExperimentalVersion": true,
	"Installed": false,
	"Modules": [
		{
			"Name": "TedsActorCompatibility",
			"Type": "Editor"
		},
		{
			"Name": "TedsAlerts",
			"Type": "Editor"
		},
		{
			"Name": "TedsAssetData",
			"Type": "Editor"
		},
		{
			"Name": "TedsContentBrowser",
			"Type": "Editor"
		},
		{
			"Name": "TedsDebugger",
			"Type": "Editor"
		},
		{
			"Name": "TedsOutliner",
			"Type": "Editor"
		},
		{
			"Name": "TedsPropertyEditor",
			"Type": "Editor"
		},
		{
			"Name": "TedsQueryStack",
			"Type": "Editor"
		},
		{
			"Name": "TedsRevisionControl",
			"Type": "Editor"
		},
		{
			"Name": "TedsSettings",
			"Type": "Editor"
		},
		{
			"Name": "TedsTableViewer",
			"Type": "Editor"
		},
		{
			"Name": "TedsEverythingPicker",
			"Type": "Editor"
		},
		{
			"Name": "TedsTypeInfo",
			"Type": "Editor"
		},
		{
			"Name": "TedsTypedElementBridge",
			"Type": "Editor"
		}
		,
		{
			"Name": "TedsEditorCompatibility",
			"Type": "Editor"
		},
		{
			"Name": "TedsOperations",
			"Type": "Editor"
		},
		{
			"Name": "UnifiedFavorites",
			"Type": "Editor"
		}
	],
	"Plugins": [
	],
	"SupportedPrograms": ["ChaosVisualDebugger"]
}