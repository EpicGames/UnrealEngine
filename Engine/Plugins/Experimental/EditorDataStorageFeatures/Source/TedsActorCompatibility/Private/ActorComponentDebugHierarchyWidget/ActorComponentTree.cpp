// Copyright Epic Games, Inc. All Rights Reserved.

#include "ActorComponentTree.h"

#include "SActorComponentTreeWidget.h"
#include "TedsHierarchyNode.h"
#include "TedsQueryNode.h"
#include "TedsQueryNode.h"
#include "TedsRowQueryResultsNode.h"
#include "WorkspaceMenuStructure.h"
#include "WorkspaceMenuStructureModule.h"
#include "WorkspaceMenuStructureModule.h"
#include "Editor/LevelEditor/Private/SLevelEditor.h"
#include "Elements/Columns/TypedElementCompatibilityColumns.h"
#include "Elements/Columns/TypedElementSelectionColumns.h"
#include "DataStorage/Features.h"
#include "Factories/TedsEditorHierarchyFactory.h"
#include "Framework/Docking/TabManager.h"
#include "QueryStackNodes/RowQueryCallbackResultsNode.h"
#include "Widgets/STedsTableViewer.h"
#include "Widgets/Docking/SDockTab.h"

#define LOCTEXT_NAMESPACE "ActorComponentTree"

namespace UE::Editor::DataStorage::ActorCompatibility
{
	const FLazyName ActorComponentTreeTab = TEXT("ActorComponent Tree Tab");
	
	void RegisterActionComponentDebugHierarchyWidget()
	{
		const FName TabId = ActorComponentTreeTab.Resolve();
		FTabSpawnerEntry& TabEntry = FGlobalTabmanager::Get()->RegisterNomadTabSpawner(
			TabId,
			FOnSpawnTab::CreateLambda([](const FSpawnTabArgs& Args)
			{
				using namespace  UE::Editor::DataStorage;
				using namespace UE::Editor::DataStorage::Queries;
				const TSharedRef<SDockTab> MajorTab = SNew(SDockTab)
					.TabRole(ETabRole::MajorTab);

				ICoreProvider* DataStorageInterface = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName);
				
				FHierarchyHandle HierarchyHandle = DataStorageInterface->FindHierarchyByName(UTedsEditorHierarchyFactory::EditorObjectHierarchyName);

				if (DataStorageInterface->IsValidHierarchyHandle(HierarchyHandle))
				{
					// Query node which will enumerate all selected actor rows and provide access to their hierarchy
					TSharedRef<QueryStack::FQueryNode> QueryNode = MakeShared<QueryStack::FQueryNode>(*DataStorageInterface);
					QueryNode->SetQuery(
						Select()
						.ReadOnly<FTypedElementSelectionColumn>()
						.Where()
							.All<FTypedElementActorTag>()
						.Compile());

					auto FilterRowsBySelectionSet = [](
						IDirectQueryContext& Context,
						TArrayView<const RowHandle> Rows,
						QueryStack::FRowQueryCallbackResultsNode::EmitRowFn EmitRows)
					{
						const FTypedElementSelectionColumn* SelectionColumnArrayPtr = Context.GetColumn<FTypedElementSelectionColumn>();
					
						TArrayView<const FTypedElementSelectionColumn> SelectionColumns(SelectionColumnArrayPtr, Context.GetRowCount());
					
						for (int32 RowIndex = 0; RowIndex < Rows.Num(); ++RowIndex)
						{
							if (SelectionColumns[RowIndex].SelectionSet == NAME_None)
							{
								EmitRows(MakeArrayView(&Rows[RowIndex], 1));
							}
						}
					};

					TSharedPtr<QueryStack::IRowNode> TopLevelRowNode = MakeShared<QueryStack::FRowQueryCallbackResultsNode>(
						*DataStorageInterface,
						QueryNode,
						QueryStack::FRowQueryResultsNode::ESyncActions::RefreshOnUpdate,
						FilterRowsBySelectionSet
						);

					// Use the actor-component hierarchy to enumerate the components of all the selected actors
					TSharedPtr<QueryStack::IRowNode> AllRowNode = MakeShared<QueryStack::FHierarchyRowNode>(
						*DataStorageInterface,
						HierarchyHandle,
						TopLevelRowNode,
						QueryStack::FHierarchyRowNode::ESyncFlags::Always);

					// Create a widget to display the view model, widget internally binds the results of the query and displays appropriately
					TSharedRef<SActorComponentTreeWidget> TreeWidget = SNew(SActorComponentTreeWidget, HierarchyHandle, AllRowNode);
			
					MajorTab->SetContent(TreeWidget);
				}
				else
				{
					MajorTab->SetContent(SNullWidget::NullWidget);
				}
				return MajorTab;
			}));

		TabEntry
		.SetGroup(WorkspaceMenu::GetMenuStructure().GetDeveloperToolsDebugCategory())
		.SetDisplayName(LOCTEXT("ActorComponentTree_DisplayName", "ActorComponent Tree"))
		.SetTooltipText(LOCTEXT("ActorComponentTree_OpenTree", "Opens the ActorComponent Tree"))
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "Debug"));
	}
}

#undef LOCTEXT_NAMESPACE