// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

namespace UE::Editor::DataStorage::ActorCompatibility
{
	void RegisterActionComponentDebugHierarchyWidget();
}