// Copyright Epic Games, Inc. All Rights Reserved.

#include "RowQueryCallbackResultsNode.h"

#include "Elements/Framework/TypedElementQueryBuilder.h"
#include "Elements/Interfaces/TypedElementQueryStorageInterfaces.h"

namespace UE::Editor::DataStorage::QueryStack
{
	FRowQueryCallbackResultsNode::FRowQueryCallbackResultsNode(
		ICoreProvider& Storage,
		TSharedPtr<IQueryNode> InQueryNode,
		CallbackFn InCallbackFn)
		: FRowQueryResultsNode(Storage, InQueryNode)
		, Callback(MoveTemp(InCallbackFn))
	{
	}

	FRowQueryCallbackResultsNode::FRowQueryCallbackResultsNode(
		ICoreProvider& Storage,
		TSharedPtr<IQueryNode> InQueryNode,
		ESyncActions SyncAction,
		CallbackFn InCallbackFn)
		: FRowQueryResultsNode(Storage, InQueryNode, SyncAction)
		, Callback(MoveTemp(InCallbackFn))
	{
	}

	void FRowQueryCallbackResultsNode::RefreshInternal(FRowHandleArray& TargetRows)
	{
		using namespace UE::Editor::DataStorage::Queries;
		Storage.RunQuery(QueryNode->GetQuery(), 
			EDirectQueryExecutionFlags::AllowBoundQueries,
			CreateDirectQueryCallbackBinding(
		[&TargetRows, this](IDirectQueryContext& Context, const RowHandle* QueryRowResults)
		{
			auto EmitRows = [&TargetRows](TArrayView<const RowHandle> InRows)
			{
				TargetRows.Append(InRows);
			};

			TArrayView<const RowHandle> QueryRowResultsView = MakeArrayView(QueryRowResults, Context.GetRowCount());

			Callback(Context, QueryRowResultsView, EmitRows);
		}));
	}
}
