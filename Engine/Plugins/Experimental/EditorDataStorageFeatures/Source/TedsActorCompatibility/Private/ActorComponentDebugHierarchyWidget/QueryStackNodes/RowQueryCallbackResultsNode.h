// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "TedsRowQueryResultsNode.h"

namespace UE::Editor::DataStorage
{
	struct IDirectQueryContext;
}

namespace UE::Editor::DataStorage::QueryStack
{
	/**
	* A node that is configured with a query and a callback.  The callback can be used to
	 * read and filter data from the rows that match the query and can emit 0 or more rows that
	 * will be added to the node.
	 * This node can be useful for collating rows referenced from other rows.
	 */
	class FRowQueryCallbackResultsNode : public FRowQueryResultsNode
	{
	public:
		using EmitRowFn = TFunctionRef<void(TArrayView<const RowHandle>)>;
		using CallbackFn = TFunction<void(IDirectQueryContext&, TArrayView<const RowHandle>, EmitRowFn EmitRows)>;
		
		FRowQueryCallbackResultsNode(
			ICoreProvider& Storage,
			TSharedPtr<IQueryNode> InQueryNode,
			CallbackFn InCallback);
		FRowQueryCallbackResultsNode(
			ICoreProvider& Storage,
			TSharedPtr<IQueryNode> InQueryNode,
			ESyncActions SyncAction,
			CallbackFn InCallback);
	
	protected:
		virtual void RefreshInternal(FRowHandleArray& TargetRows) override;

	private:
		CallbackFn Callback;
	};
}
