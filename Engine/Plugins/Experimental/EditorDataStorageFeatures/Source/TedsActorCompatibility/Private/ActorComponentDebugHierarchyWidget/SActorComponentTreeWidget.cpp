// Copyright Epic Games, Inc. All Rights Reserved.

#include "SActorComponentTreeWidget.h"

#include "TedsHierarchyNode.h"
#include "TedsQueryNode.h"
#include "Elements/Columns/TypedElementCompatibilityColumns.h"
#include "Elements/Columns/TypedElementLabelColumns.h"
#include "DataStorage/Features.h"
#include "Elements/Framework/TypedElementAttributeBinding.h"
#include "Widgets/STedsHierarchyViewer.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "ActorComponentTree"

SActorComponentTreeWidget::~SActorComponentTreeWidget() = default;

void SActorComponentTreeWidget::Construct(
	const FArguments& InArgs,
	UE::Editor::DataStorage::FHierarchyHandle InHierarchyHandle,
	TSharedPtr<UE::Editor::DataStorage::QueryStack::IRowNode>& InRowProvider)
{
	using namespace UE::Editor::DataStorage;
	using namespace UE::Editor::DataStorage::Queries;

	if (ICoreProvider* Storage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName))
	{
		TSharedPtr<FHierarchyViewerData> HierarchyData = MakeShared<FHierarchyViewerData>(*Storage,InHierarchyHandle);
	
		TSharedRef<SHierarchyViewer> HierarchyViewer =
			SNew(SHierarchyViewer,
				MoveTemp(HierarchyData))
				.AllNodeProvider(InRowProvider)
				.EmptyRowsMessage(LOCTEXT("EmptyRowsMessage", "The provided query has no results."))
				.CellWidgetPurpose(IUiProvider::FPurposeInfo("General", "RowLabel", NAME_None).GeneratePurposeID())
				.Columns({FTypedElementLabelColumn::StaticStruct()});

		ChildSlot
		[
			HierarchyViewer
		];
	}
}

#undef LOCTEXT_NAMESPACE

