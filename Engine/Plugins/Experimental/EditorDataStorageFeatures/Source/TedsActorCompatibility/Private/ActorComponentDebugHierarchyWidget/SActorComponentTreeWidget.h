// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Containers/Ticker.h"
#include "DataStorage/Handles.h"
#include "Elements/Interfaces/TypedElementDataStorageFactory.h"
#include "Elements/Interfaces/TypedElementDataStorageUiInterface.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SWidget.h"

namespace UE::Editor::DataStorage
{
	class STedsTableViewer;

	namespace QueryStack
	{
		class IRowNode;
		class FRowViewNode;
	}
}

class SActorComponentTreeWidget : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SActorComponentTreeWidget) {}
		
	SLATE_END_ARGS();

public:
	virtual ~SActorComponentTreeWidget() override;

	void Construct(
		const FArguments& InArgs,
		UE::Editor::DataStorage::FHierarchyHandle InHierarchyHandle,
		TSharedPtr<UE::Editor::DataStorage::QueryStack::IRowNode>& InRowProvider);

private:
};