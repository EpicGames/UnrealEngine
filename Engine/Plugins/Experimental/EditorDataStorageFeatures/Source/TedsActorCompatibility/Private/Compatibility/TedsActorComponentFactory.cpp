// Copyright Epic Games, Inc. All Rights Reserved.

#include "TedsActorComponentFactory.h"

#include "Columns/TedsActorComponentCompatibilityColumns.h"
#include "DataStorage/Features.h"
#include "Elements/Columns/TypedElementCompatibilityColumns.h"
#include "Elements/Columns/TypedElementLabelColumns.h"
#include "Elements/Columns/TypedElementMiscColumns.h"
#include "Elements/Columns/TypedElementTypeInfoColumns.h"
#include "Elements/Framework/TypedElementQueryBuilder.h"
#include "Elements/Interfaces/TypedElementDataStorageCompatibilityInterface.h"
#include "Engine/Level.h"
#include "Factories/TedsEditorHierarchyFactory.h"
#include "GameFramework/Actor.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TedsActorComponentFactory)

static bool bEnableActorComponentCompatibility = true;
	FAutoConsoleVariableRef CVarEnableActorComponentRegistrator(
		TEXT("TEDS.Feature.ActorCompatibility.ActorComponents.Enable"),
		bEnableActorComponentCompatibility,
		TEXT("Enables ActorComponent tracking in TEDS"),
		ECVF_ReadOnly);

void UTedsActorComponentFactory::RegisterTables(UE::Editor::DataStorage::ICoreProvider& DataStorage)
{
	using namespace UE::Editor::DataStorage;
	
	if (bEnableActorComponentCompatibility)
	{
		TableHandle ActorComponentTable = 
			DataStorage.RegisterTable<FActorComponentTypeTag, FLevelColumn, FTypedElementLabelColumn, FTypedElementLabelHashColumn>(
				FName("Editor_ActorComponentTable"),
				FTableRegistrationOptions
				{ 
					.SourceTable = DataStorage.FindTable(FName(TEXT("Editor_StandardUObjectTable"))) 
				});
		
		if (ICompatibilityProvider* CompatibilityProvider = GetMutableDataStorageFeature<ICompatibilityProvider>(CompatibilityFeatureName))
		{
			CompatibilityProvider->RegisterTypeTableAssociation(UActorComponent::StaticClass(), ActorComponentTable);
		}
	}
}

void UTedsActorComponentFactory::RegisterQueries(UE::Editor::DataStorage::ICoreProvider& DataStorage)
{
	using namespace UE::Editor::DataStorage;
	using namespace UE::Editor::DataStorage::Queries;
	
	if (bEnableActorComponentCompatibility)
	{
		// 1. For actors that need syncing, add any components they have to the compatibility layer
		// This solves a lot of issues around trying to intercept all the places components are added and removed
		struct FRegisterActorComponentsWithCompatibility
		{
			TWeakObjectPtr<AActor> WkActor;
			RowHandle ActorRow;
			TArray<TWeakObjectPtr<UActorComponent>> ComponentsToRegister;

			void operator()()
			{
				AActor* Actor = WkActor.Get();
				if (!Actor)
				{
					return;
				}
				if (ICompatibilityProvider* CompatibilityProvider = GetMutableDataStorageFeature<ICompatibilityProvider>(CompatibilityFeatureName))
				{
					for (TWeakObjectPtr<UActorComponent> Component : ComponentsToRegister)
					{
						UActorComponent* NewComponent = Component.Get();
						if (NewComponent)
						{
							CompatibilityProvider->AddCompatibleObject(NewComponent);
						}
					}
				}
			};
		};
		DataStorage.RegisterQuery(
			Select(
				TEXT("Register ActorComponent Ownership"),
				FProcessor(EQueryTickPhase::DuringPhysics, DataStorage.GetQueryTickGroupName(EQueryTickGroups::Default))
				.SetExecutionMode(EExecutionMode::GameThread),
				[](IQueryContext& Context, RowHandle Row, const FTypedElementUObjectColumn& ObjectColumn)
				{
					if (AActor* Actor = static_cast<AActor*>(ObjectColumn.Object.Get()))
					{
						FRegisterActorComponentsWithCompatibility Command;
						Command.WkActor = Actor;
						Command.ActorRow = Row;
						
						constexpr bool bIncludeFromChildActors = false;
						Actor->ForEachComponent(bIncludeFromChildActors, [&Context, &Command](UActorComponent* Component)
						{
							RowHandle ComponentRow = Context.LookupMappedRow(ICompatibilityProvider::ObjectMappingDomain, FMapKeyView(static_cast<const UObject*>(Component)));
							if (ComponentRow == InvalidRowHandle)
							{
								Command.ComponentsToRegister.Add(Component);
							}
						});
						
						if (!Command.ComponentsToRegister.IsEmpty())
						{
							Context.PushCommand<FRegisterActorComponentsWithCompatibility>(MoveTemp(Command));
						}
					}
				})
			.Where()
				.All<FTypedElementActorTag, FTypedElementSyncFromWorldTag>()
			.Compile());

		// 2. For components that aren't a child, which means they don't have an associated owner, set up that owner
		//    If a component is not associated with an owner, then remove that association in TEDS - in practice this is usally only a
		//    transitory state
		//    Note: ActorComponent ownership can change due to renaming.  
		DataStorage.RegisterQuery(
			Select(
				TEXT("Set ActorComponent Hierarchy"),
				FProcessor(EQueryTickPhase::DuringPhysics, DataStorage.GetQueryTickGroupName(EQueryTickGroups::Default))
				.SetExecutionMode(EExecutionMode::GameThread),
				[](IQueryContext& Context, RowHandle ComponentRow, const FTypedElementUObjectColumn& ObjectColumn)
				{
					if (UActorComponent* ActorComponent = Cast<UActorComponent>(ObjectColumn.Object.Get()))
					{
						const AActor* OwnerActor = ActorComponent->GetOwner();
						RowHandle ActorRow;
						if (OwnerActor)
						{
							ActorRow = Context.LookupMappedRow(ICompatibilityProvider::ObjectMappingDomain, FMapKeyView(static_cast<const UObject*>(OwnerActor)));
						}
						else
						{
							ActorRow = InvalidRowHandle;
						}
						
						const bool bNotAssociatedWithActor = OwnerActor == nullptr && ActorRow == InvalidRowHandle;
						const bool bActorRowAssigned = Context.IsRowAssigned(ActorRow);

						RowHandle ParentRowOfComponent = Context.GetParentRow(ComponentRow);
						
						if (bNotAssociatedWithActor || (bActorRowAssigned && ParentRowOfComponent != ActorRow))
						{
							Context.SetParentRow(ComponentRow, ActorRow);
						}
					}
				})
				.AccessesHierarchy(UTedsEditorHierarchyFactory::EditorObjectHierarchyName)
				.Where()
					.All<FActorComponentTypeTag, FTypedElementSyncFromWorldTag>()
				.Compile());

		// Sync Actor Component Label
		DataStorage.RegisterQuery(
			Select(
				TEXT("Sync actorcomponent label to column"),
				FProcessor(EQueryTickPhase::PrePhysics, DataStorage.GetQueryTickGroupName(EQueryTickGroups::SyncExternalToDataStorage))
				.SetExecutionMode(EExecutionMode::GameThread),
				[](const FTypedElementUObjectColumn& Actor, FTypedElementLabelColumn& Label, FTypedElementLabelHashColumn& LabelHash)
				{
					if (const UActorComponent* ComponentInstance = Cast<UActorComponent>(Actor.Object); ComponentInstance != nullptr)
					{
						const FString Name = ComponentInstance->GetName();
						uint64 NameHash = CityHash64(reinterpret_cast<const char*>(*Name), Name.Len() * sizeof(**Name));
						if (LabelHash.LabelHash != NameHash)
						{
							Label.Label = Name;
							LabelHash.LabelHash = NameHash;
						}
					}
				}
			)
			.Where()
				.All<FActorComponentTypeTag, FTypedElementSyncFromWorldTag>()
			.Compile());

		// Sync Actor Component ULevel
		DataStorage.RegisterQuery(
			Select(
				TEXT("Sync ActorComponent level to column"),
				FProcessor(EQueryTickPhase::PrePhysics, DataStorage.GetQueryTickGroupName(EQueryTickGroups::SyncExternalToDataStorage))
				.SetExecutionMode(EExecutionMode::GameThread),
				[](const FTypedElementUObjectColumn& Actor, FLevelColumn& LevelColumn)
				{
					if (const UActorComponent* ComponentInstance = Cast<UActorComponent>(Actor.Object); ComponentInstance != nullptr)
					{
						if (ULevel* Level = ComponentInstance->GetComponentLevel())
						{
							LevelColumn.Level = Level;
						}
					}
				}
			)
			.Where()
				.All<FActorComponentTypeTag, FTypedElementSyncFromWorldTag>()
			.Compile());
	}
}
