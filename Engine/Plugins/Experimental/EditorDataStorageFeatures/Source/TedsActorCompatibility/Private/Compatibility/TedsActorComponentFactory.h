// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/Interfaces/TypedElementDataStorageInterface.h"
#include "Elements/Interfaces/TypedElementDataStorageFactory.h"
#include "Engine/World.h"

#include "TedsActorComponentFactory.generated.h"

UCLASS()
class UTedsActorComponentFactory : public UEditorDataStorageFactory
{
	GENERATED_BODY()

public:
	virtual void RegisterTables(UE::Editor::DataStorage::ICoreProvider& DataStorage) override;
	virtual void RegisterQueries(UE::Editor::DataStorage::ICoreProvider& DataStorage) override;
};