// Copyright Epic Games, Inc. All Rights Reserved.

#include "Compatibility/TedsActorGuidQueries.h"

#include "Columns/TedsActorWorldPartitionColumns.h"
#include "DataStorage/Queries/Types.h"
#include "Elements/Columns/TypedElementCompatibilityColumns.h"
#include "Elements/Columns/TypedElementMiscColumns.h"
#include "Elements/Framework/TypedElementQueryBuilder.h"
#include "GameFramework/Actor.h"
#include "WorldPartition/WorldPartition.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TedsActorGuidQueries)

void UActorGuidDataStorageFactory::RegisterQueries(UE::Editor::DataStorage::ICoreProvider& DataStorage)
{
	using namespace UE::Editor::DataStorage::Queries;

	DataStorage.RegisterQuery(
		Select(
			TEXT("Actor Guid to Column"),
			FProcessor(EQueryTickPhase::PrePhysics, DataStorage.GetQueryTickGroupName(EQueryTickGroups::SyncExternalToDataStorage))
				.SetExecutionMode(EExecutionMode::GameThread),
			[](IQueryContext&, RowHandle, const FTypedElementUObjectColumn& Object, FGuidColumn& Guid)
			{
				if (const AActor* Actor = Cast<AActor>(Object.Object))
				{
					Guid.Guid = Actor->GetActorGuid();
				}
			})
		.Where()
			.All<FTypedElementActorTag, FTypedElementSyncFromWorldTag>()
		.Compile()
	);
}