// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/Interfaces/TypedElementDataStorageFactory.h"
#include "UObject/ObjectMacros.h"

#include "TedsActorGuidQueries.generated.h"

/**
 * Registers queries that keep FGuidColumn in sync with the actor's
 * Guid.
 */
UCLASS()
class UActorGuidDataStorageFactory : public UEditorDataStorageFactory
{
	GENERATED_BODY()

public:
	~UActorGuidDataStorageFactory() override = default;

	void RegisterQueries(UE::Editor::DataStorage::ICoreProvider& DataStorage) override;
};
