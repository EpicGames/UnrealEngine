// Copyright Epic Games, Inc. All Rights Reserved.

#include "Compatibility/TedsActorIconOverrideQueries.h"

#include "Elements/Columns/TypedElementCompatibilityColumns.h"
#include "Elements/Columns/TypedElementIconOverrideColumns.h"
#include "Elements/Columns/TypedElementMiscColumns.h"
#include "Elements/Framework/TypedElementQueryBuilder.h"
#include "GameFramework/Actor.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TedsActorIconOverrideQueries)

void UActorIconOverrideDataStorageFactory::RegisterQueries(UE::Editor::DataStorage::ICoreProvider& DataStorage)
{
	using namespace UE::Editor::DataStorage::Queries;

	DataStorage.RegisterQuery(
		Select(
			TEXT("Add icon override column to actor"),
			FProcessor(EQueryTickPhase::PrePhysics, DataStorage.GetQueryTickGroupName(EQueryTickGroups::SyncExternalToDataStorage))
				.SetExecutionMode(EExecutionMode::GameThread),
			[](IQueryContext& Context, RowHandle Row, const FTypedElementUObjectColumn& ActorColumn)
			{
				if (const AActor* Actor = Cast<AActor>(ActorColumn.Object))
				{
					if (FName IconName = Actor->GetCustomIconName(); !IconName.IsNone())
					{
						Context.AddColumn(Row, FTypedElementIconOverrideColumn{ .IconName = IconName });
					}
				}
			})
		.Where()
			.All<FTypedElementSyncFromWorldTag, FTypedElementActorTag>()
			.None<FTypedElementIconOverrideColumn>()
		.Compile());

	DataStorage.RegisterQuery(
		Select(
			TEXT("Update/remove icon override column to actor"),
			FProcessor(EQueryTickPhase::PrePhysics, DataStorage.GetQueryTickGroupName(EQueryTickGroups::SyncExternalToDataStorage))
				.SetExecutionMode(EExecutionMode::GameThread),
			[]( IQueryContext& Context, 
				RowHandle Row, 
				const FTypedElementUObjectColumn& ActorColumn, 
				FTypedElementIconOverrideColumn& IconColumn)
			{
				if (const AActor* Actor = Cast<AActor>(ActorColumn.Object))
				{
					if (FName IconName = Actor->GetCustomIconName(); !IconName.IsNone())
					{
						IconColumn.IconName = IconName;
					}
					else
					{
						Context.RemoveColumns<FTypedElementIconOverrideColumn>(Row);
					}
				}
			})
		.Where()
			.All<FTypedElementSyncFromWorldTag, FTypedElementActorTag>()
		.Compile());
}
