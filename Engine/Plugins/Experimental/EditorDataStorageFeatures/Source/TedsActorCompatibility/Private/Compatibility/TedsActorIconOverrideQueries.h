// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/Interfaces/TypedElementDataStorageFactory.h"
#include "UObject/ObjectMacros.h"

#include "TedsActorIconOverrideQueries.generated.h"

UCLASS()
class UActorIconOverrideDataStorageFactory : public UEditorDataStorageFactory
{
	GENERATED_BODY()

public:
	virtual ~UActorIconOverrideDataStorageFactory() override = default;

	virtual void RegisterQueries(UE::Editor::DataStorage::ICoreProvider& DataStorage) override;
};
