// Copyright Epic Games, Inc. All Rights Reserved.

#include "Compatibility/TedsActorLabelQueries.h"

#include "Editor/EditorEngine.h"
#include "Elements/Columns/TypedElementCompatibilityColumns.h"
#include "Elements/Columns/TypedElementMiscColumns.h"
#include "Elements/Columns/TypedElementLabelColumns.h"
#include "Elements/Framework/TypedElementQueryBuilder.h"
#include "Elements/Framework/TypedElementRegistry.h"
#include "Hash/CityHash.h"
#include "ScopedTransaction.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TedsActorLabelQueries)

#define LOCTEXT_NAMESPACE "TedsCore"

void UActorLabelDataStorageFactory::RegisterQueries(UE::Editor::DataStorage::ICoreProvider& DataStorage)
{
	RegisterActorLabelToColumnQuery(DataStorage);
	RegisterLabelColumnToActorQuery(DataStorage);
}

void UActorLabelDataStorageFactory::RegisterActorLabelToColumnQuery(UE::Editor::DataStorage::ICoreProvider& DataStorage) const
{
	using namespace UE::Editor::DataStorage::Queries;
	
	DataStorage.RegisterQuery(
		Select(
			TEXT("Sync actor label to column"),
			FProcessor(EQueryTickPhase::PrePhysics, DataStorage.GetQueryTickGroupName(EQueryTickGroups::SyncExternalToDataStorage))
				.SetExecutionMode(EExecutionMode::GameThread),
			[](const FTypedElementUObjectColumn& Actor, FTypedElementLabelColumn& Label, FTypedElementLabelHashColumn& LabelHash)
			{
				if (const AActor* ActorInstance = Cast<AActor>(Actor.Object); ActorInstance != nullptr)
				{
					const FString& ActorLabel = ActorInstance->GetActorLabel();
					uint64 ActorLabelHash = CityHash64(reinterpret_cast<const char*>(*ActorLabel), ActorLabel.Len() * sizeof(**ActorLabel));
					if (LabelHash.LabelHash != ActorLabelHash)
					{
						Label.Label = ActorLabel;
						LabelHash.LabelHash = ActorLabelHash;
					}
				}
			}
		)
		.Where()
			.All<FTypedElementActorTag, FTypedElementSyncFromWorldTag>()
		.Compile());
}

void UActorLabelDataStorageFactory::RegisterLabelColumnToActorQuery(UE::Editor::DataStorage::ICoreProvider& DataStorage) const
{
	using namespace UE::Editor::DataStorage::Queries;

	struct FRenameActorCommand
	{
		void operator()()
		{
			TStrongObjectPtr<AActor> PinnedActor = Actor.Pin();
			if (PinnedActor)
			{
				const FScopedTransaction Transaction(LOCTEXT("RenameActorTransaction", "Rename Actor"));
				FActorLabelUtilities::RenameExistingActor(PinnedActor.Get(), NewLabel);
			}
			
		}
		TWeakObjectPtr<AActor> Actor;
		FString NewLabel;
	};
	
	DataStorage.RegisterQuery(
		Select(
			TEXT("Sync label column to actor"),
			FProcessor(EQueryTickPhase::FrameEnd, DataStorage.GetQueryTickGroupName(EQueryTickGroups::SyncDataStorageToExternal))
				.SetExecutionMode(EExecutionMode::GameThread),
			[](IQueryContext& Context, FTypedElementUObjectColumn& Actor, const FTypedElementLabelColumn& Label, const FTypedElementLabelHashColumn& LabelHash)
			{
				if (AActor* ActorInstance = Cast<AActor>(Actor.Object); ActorInstance != nullptr)
				{
					const FString& ActorLabel = ActorInstance->GetActorLabel(false);
					uint64 ActorLabelHash = CityHash64(reinterpret_cast<const char*>(*ActorLabel), ActorLabel.Len() * sizeof(**ActorLabel));
					if (LabelHash.LabelHash != ActorLabelHash)
					{
						Context.PushCommand(FRenameActorCommand
							{
								.Actor = ActorInstance,
								.NewLabel = Label.Label
							});
					}
				}
			}
		)
		.Where()
			.All<FTypedElementActorTag, FTypedElementSyncBackToWorldTag>()
		.Compile());
}

#undef LOCTEXT_NAMESPACE
