// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/Interfaces/TypedElementDataStorageFactory.h"
#include "UObject/ObjectMacros.h"

#include "TedsActorLabelQueries.generated.h"


UCLASS()
class UActorLabelDataStorageFactory : public UEditorDataStorageFactory
{
	GENERATED_BODY()

public:
	~UActorLabelDataStorageFactory() override = default;

	void RegisterQueries(UE::Editor::DataStorage::ICoreProvider& DataStorage) override;

private:
	/**
	 * Takes the label set on an actor and copies it to the Data Storage if they differ.
	 */
	void RegisterActorLabelToColumnQuery(UE::Editor::DataStorage::ICoreProvider& DataStorage) const;
	/**
	 * Takes the label stored in the Data Storage and copies it to the actor's label if the FTypedElementSyncBackToWorldTag
	 * has been set and the labels differ.
	 */
	void RegisterLabelColumnToActorQuery(UE::Editor::DataStorage::ICoreProvider& DataStorage) const;
};