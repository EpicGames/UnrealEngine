// Copyright Epic Games, Inc. All Rights Reserved.

#include "Compatibility/TedsActorLevelQueries.h"

#include "Elements/Columns/TypedElementCompatibilityColumns.h"
#include "Elements/Columns/TypedElementMiscColumns.h"
#include "Elements/Framework/TypedElementQueryBuilder.h"
#include "Engine/Level.h"
#include "GameFramework/Actor.h"
#include "LevelInstance/LevelInstanceInterface.h"

void UActorLevelDataStorageFactory::RegisterQueries(UE::Editor::DataStorage::ICoreProvider& DataStorage)
{
	RegisterActorAddLevelColumn(DataStorage);
	RegisterActorLevelToColumnQuery(DataStorage);
}

void UActorLevelDataStorageFactory::RegisterActorAddLevelColumn(UE::Editor::DataStorage::ICoreProvider& DataStorage) const
{
	using namespace UE::Editor::DataStorage::Queries;

	DataStorage.RegisterQuery(
		Select(
			TEXT("Actor Level to New Column"),
			FProcessor(EQueryTickPhase::PrePhysics, DataStorage.GetQueryTickGroupName(EQueryTickGroups::SyncExternalToDataStorage))
			.SetExecutionMode(EExecutionMode::GameThread),
			[](IQueryContext& Context, RowHandle Row, const FTypedElementUObjectColumn& Object)
			{
				if (const AActor* ActorInstance = Cast<AActor>(Object.Object); ActorInstance != nullptr)
				{
					if (ULevel* ActorLevel = ActorInstance->GetLevel(); ActorLevel != nullptr)
					{
						Context.AddColumn(Row, FLevelColumn{ .Level = ActorLevel });
					}
				}
			})
		.Where()
			.All<FTypedElementActorTag, FTypedElementSyncFromWorldTag>()
			.None<FLevelColumn>()
		.Compile()
	);
}

void UActorLevelDataStorageFactory::RegisterActorLevelToColumnQuery(UE::Editor::DataStorage::ICoreProvider& DataStorage) const
{
	using namespace UE::Editor::DataStorage::Queries;

	DataStorage.RegisterQuery(
		Select(
			TEXT("Actor Level to Column"),
			FProcessor(EQueryTickPhase::PrePhysics, DataStorage.GetQueryTickGroupName(EQueryTickGroups::SyncExternalToDataStorage))
			.SetExecutionMode(EExecutionMode::GameThread),
			[](IQueryContext& Context, RowHandle Row, const FTypedElementUObjectColumn& Object, FLevelColumn& LevelColumn)
			{
				if (const AActor* ActorInstance = Cast<AActor>(Object.Object); ActorInstance != nullptr)
				{
					if (ULevel* ActorLevel = ActorInstance->GetLevel(); ActorLevel != nullptr)
					{
						LevelColumn.Level = ActorLevel;
						return;
					}
				}
				Context.RemoveColumns<FLevelColumn>(Row);
			})
		.Where()
			.All<FTypedElementActorTag, FTypedElementSyncFromWorldTag>()
		.Compile()
	);
}