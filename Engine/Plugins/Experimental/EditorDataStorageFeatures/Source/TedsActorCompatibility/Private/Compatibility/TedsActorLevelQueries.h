// Copyright Epic Games, Inc.All Rights Reserved.

#pragma once

#include "Elements/Interfaces/TypedElementDataStorageFactory.h"
#include "UObject/ObjectMacros.h"

#include "TedsActorLevelQueries.generated.h"

UCLASS()
class UActorLevelDataStorageFactory : public UEditorDataStorageFactory
{
	GENERATED_BODY()

public:
	~UActorLevelDataStorageFactory() override = default;

	void RegisterQueries(UE::Editor::DataStorage::ICoreProvider& DataStorage) override;
private:

	/**
	 * Adds the level column to new actors that do not have one already.
	 */
	void RegisterActorAddLevelColumn(UE::Editor::DataStorage::ICoreProvider& DataStorage) const;
	/**
	 * Takes the level of the actor and copies it to the Data Storage if they differ.
	 */
	void RegisterActorLevelToColumnQuery(UE::Editor::DataStorage::ICoreProvider& DataStorage) const;
};
