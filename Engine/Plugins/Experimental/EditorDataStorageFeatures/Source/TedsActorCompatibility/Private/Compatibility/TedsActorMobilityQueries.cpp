// Copyright Epic Games, Inc. All Rights Reserved.

#include "Compatibility/TedsActorMobilityQueries.h"

#include "Elements/Columns/TypedElementCompatibilityColumns.h"
#include "Elements/Columns/TypedElementMiscColumns.h"
#include "Elements/Columns/TypedElementTransformColumns.h" // FTypedElementLockTransformTag
#include "Columns/TedsActorMobilityColumns.h"
#include "Columns/TedsActorUncachedLightsColumns.h"
#include "Elements/Framework/TypedElementQueryBuilder.h"
#include "GameFramework/Actor.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TedsActorMobilityQueries)

void UActorMobilityDataStorageFactory::RegisterQueries(UE::Editor::DataStorage::ICoreProvider& DataStorage)
{
	RegisterActorAddMobilityColumn(DataStorage);
	RegisterActorMobilityToColumnQuery(DataStorage);
	RegisterMobilityColumnToActorQuery(DataStorage);

	RegisterLockTransformActorToTagQuery(DataStorage);
	RegisterLockTransformTagToActorQuery(DataStorage);
}

void UActorMobilityDataStorageFactory::RegisterActorAddMobilityColumn(UE::Editor::DataStorage::ICoreProvider& DataStorage) const
{
	using namespace UE::Editor::DataStorage::Queries;

	DataStorage.RegisterQuery(
		Select(
			TEXT("Add Actor Mobility to a New Column"),
			FProcessor(EQueryTickPhase::PrePhysics, DataStorage.GetQueryTickGroupName(EQueryTickGroups::SyncExternalToDataStorage))
			.SetExecutionMode(EExecutionMode::GameThread),
			[](IQueryContext& Context, RowHandle Row, const FTypedElementUObjectColumn& Object)
			{
				if (const AActor* ActorInstance = Cast<AActor>(Object.Object); ActorInstance != nullptr)
				{
					if (const USceneComponent* ActorInstanceRootComponent = ActorInstance->GetRootComponent())
					{
						const EComponentMobility::Type Mobility = ActorInstanceRootComponent->GetMobility();
						Context.AddColumn(Row, FTedsActorMobilityColumn{ .Mobility = Mobility });
						if (Mobility != EComponentMobility::Movable)
						{
							Context.AddColumns<FTedsActorUncachedLightsTag>(Row);
						}
					}
				}
			})
		.Where()
			.All<FTypedElementActorTag, FTypedElementSyncFromWorldTag>()
			.None<FTedsActorMobilityColumn>()
		.Compile()
	);
}

void UActorMobilityDataStorageFactory::RegisterActorMobilityToColumnQuery(UE::Editor::DataStorage::ICoreProvider& DataStorage) const
{
	using namespace UE::Editor::DataStorage::Queries;

	DataStorage.RegisterQuery(
		Select(
			TEXT("Sync Actor's Mobility to Column"),
			FProcessor(EQueryTickPhase::PrePhysics, DataStorage.GetQueryTickGroupName(EQueryTickGroups::SyncExternalToDataStorage))
			.SetExecutionMode(EExecutionMode::GameThread),
			[](IQueryContext& Context, RowHandle Row, const FTypedElementUObjectColumn& Object, FTedsActorMobilityColumn& MobilityColumn)
			{
				if (const AActor* ActorInstance = Cast<AActor>(Object.Object); ActorInstance != nullptr)
				{
					if (const USceneComponent* ActorInstanceRootComponent = ActorInstance->GetRootComponent())
					{
						MobilityColumn.Mobility = ActorInstanceRootComponent->GetMobility();
						if (MobilityColumn.Mobility != EComponentMobility::Movable)
						{
							Context.AddColumns<FTedsActorUncachedLightsTag>(Row);
						}
						else
						{
							Context.RemoveColumns<FTedsActorUncachedLightsTag>(Row);
						}
					}
				}
			})
		.Where()
			.All<FTypedElementActorTag, FTypedElementSyncFromWorldTag>()
		.Compile()
	);
}

void UActorMobilityDataStorageFactory::RegisterMobilityColumnToActorQuery(UE::Editor::DataStorage::ICoreProvider& DataStorage) const
{
	using namespace UE::Editor::DataStorage::Queries;

	DataStorage.RegisterQuery(
		Select(
			TEXT("Sync Column to Actor's Mobility"),
			FProcessor(EQueryTickPhase::PrePhysics, DataStorage.GetQueryTickGroupName(EQueryTickGroups::SyncDataStorageToExternal)).SetExecutionMode(EExecutionMode::GameThread),
			[&](IQueryContext& Context, RowHandle Row, FTypedElementUObjectColumn& Object, const FTedsActorMobilityColumn& MobilityColumn)
			{
				if (AActor* ActorInstance = Cast<AActor>(Object.Object); ActorInstance != nullptr)
				{
						if (USceneComponent* ActorInstanceRootComponent = ActorInstance->GetRootComponent())
						{
							ActorInstanceRootComponent->SetMobility(MobilityColumn.Mobility);
						}
				}
			})
		.Where()
			.All<FTypedElementActorTag, FTypedElementSyncBackToWorldTag>()
		.Compile()
	);
}

void UActorMobilityDataStorageFactory::RegisterLockTransformActorToTagQuery(UE::Editor::DataStorage::ICoreProvider& DataStorage) const
{
	using namespace UE::Editor::DataStorage::Queries;

	// We do this in two queries- one across actor rows without the tag, and across rows with the tag

	DataStorage.RegisterQuery(
		Select(
			TEXT("Add Lock Transform tag if needed"),
			FProcessor(EQueryTickPhase::PrePhysics, DataStorage.GetQueryTickGroupName(EQueryTickGroups::SyncExternalToDataStorage))
			.SetExecutionMode(EExecutionMode::GameThread),
			[](IQueryContext& Context, RowHandle Row, const FTypedElementUObjectColumn& Object)
			{
				if (const AActor* ActorInstance = Cast<AActor>(Object.Object))
				{
					if (ActorInstance->IsLockLocation())
					{
						Context.AddColumns<FTypedElementLockTransformTag>(Row);
					}
				}
			})
		.Where()
			.All<FTypedElementActorTag, FTypedElementSyncFromWorldTag>()
			.None<FTypedElementLockTransformTag>()
		.Compile()
	);
	DataStorage.RegisterQuery(
		Select(
			TEXT("Remove Lock Transform tag if needed"),
			FProcessor(EQueryTickPhase::PrePhysics, DataStorage.GetQueryTickGroupName(EQueryTickGroups::SyncExternalToDataStorage))
			.SetExecutionMode(EExecutionMode::GameThread),
			[](IQueryContext& Context, RowHandle Row, const FTypedElementUObjectColumn& Object)
			{
				if (const AActor* ActorInstance = Cast<AActor>(Object.Object))
				{
					if (!ActorInstance->IsLockLocation())
					{
						Context.RemoveColumns<FTypedElementLockTransformTag>(Row);
					}
				}
			})
		.Where()
			.All<FTypedElementActorTag, FTypedElementSyncFromWorldTag, FTypedElementLockTransformTag>()
		.Compile()
	);
}

void UActorMobilityDataStorageFactory::RegisterLockTransformTagToActorQuery(UE::Editor::DataStorage::ICoreProvider& DataStorage) const
{
	using namespace UE::Editor::DataStorage::Queries;

	// We do this in two queries- one across actor rows without the tag, and across rows with the tag

	DataStorage.RegisterQuery(
		Select(
			TEXT("Unlock actor location if needed"),
			FProcessor(EQueryTickPhase::PrePhysics, DataStorage.GetQueryTickGroupName(EQueryTickGroups::SyncDataStorageToExternal)).SetExecutionMode(EExecutionMode::GameThread),
			[](IQueryContext& Context, RowHandle Row, FTypedElementUObjectColumn& Object)
			{
				AActor* ActorInstance = Cast<AActor>(Object.Object);
				if (ActorInstance && ActorInstance->IsLockLocation())
				{
					ActorInstance->SetLockLocation(false);
				}
			})
		.Where()
		.All<FTypedElementActorTag, FTypedElementSyncBackToWorldTag>()
		.None<FTypedElementLockTransformTag>()
		.Compile()
	);

	DataStorage.RegisterQuery(
		Select(
			TEXT("Lock actor location if needed"),
			FProcessor(EQueryTickPhase::PrePhysics, DataStorage.GetQueryTickGroupName(EQueryTickGroups::SyncDataStorageToExternal)).SetExecutionMode(EExecutionMode::GameThread),
			[](IQueryContext& Context, RowHandle Row, FTypedElementUObjectColumn& Object)
			{
				AActor* ActorInstance = Cast<AActor>(Object.Object);
				if (ActorInstance && !ActorInstance->IsLockLocation())
				{
					ActorInstance->SetLockLocation(true);
				}
			})
		.Where()
			.All<FTypedElementActorTag, FTypedElementSyncBackToWorldTag, FTypedElementLockTransformTag>()
		.Compile()
	);
}
