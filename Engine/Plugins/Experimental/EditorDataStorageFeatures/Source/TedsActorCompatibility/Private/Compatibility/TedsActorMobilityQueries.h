// Copyright Epic Games, Inc.All Rights Reserved.

#pragma once

#include "Elements/Interfaces/TypedElementDataStorageFactory.h"
#include "UObject/ObjectMacros.h"

#include "TedsActorMobilityQueries.generated.h"

UCLASS()
class UActorMobilityDataStorageFactory : public UEditorDataStorageFactory
{
	GENERATED_BODY()

public:
	~UActorMobilityDataStorageFactory() override = default;

	void RegisterQueries(UE::Editor::DataStorage::ICoreProvider& DataStorage) override;
private:

	/**
	 * Adds the mobility column to new actors that do not have one already.
	 */
	void RegisterActorAddMobilityColumn(UE::Editor::DataStorage::ICoreProvider& DataStorage) const;
	/**
	 * Takes the mobility set on an actor and copies it to the Data Storage if they differ.
	 */
	void RegisterActorMobilityToColumnQuery(UE::Editor::DataStorage::ICoreProvider& DataStorage) const;
	/**
	 * Takes the mobility stored in the Data Storage and copies it to the actor's mobility if the FTypedElementSyncBackToWorldTag
	 * has been set and the visibility differ.
	 */
	void RegisterMobilityColumnToActorQuery(UE::Editor::DataStorage::ICoreProvider& DataStorage) const;

	/**
	 * Takes each actor's IsLockLocation status, and adds/removes a FTypedElementLockTransformTag
	 *  in the Data Storage if FTypedElementSyncFromWorldTag is set.
	 */
	void RegisterLockTransformActorToTagQuery(UE::Editor::DataStorage::ICoreProvider& DataStorage) const;

	/**
	 * Takes the FTypedElementLockTransformTag presence/absence in the Data Storage and calls
	 *  SetLockLocation on the actor if the FTypedElementSyncBackToWorldTag has been set and
	 *  the lock status differs.
	 */
	void RegisterLockTransformTagToActorQuery(UE::Editor::DataStorage::ICoreProvider& DataStorage) const;
};
