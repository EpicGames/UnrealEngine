// Copyright Epic Games, Inc. All Rights Reserved.

#include "Compatibility/TedsActorParentQueries.h"

#include "ActorFolders/TedsActorFolderUtils.h"
#include "DataStorage/MapKey.h"
#include "Elements/Columns/TypedElementCompatibilityColumns.h"
#include "Elements/Columns/TypedElementHiearchyColumns.h"
#include "Elements/Columns/TypedElementMiscColumns.h"
#include "Elements/Framework/TypedElementQueryBuilder.h"
#include "Engine/Engine.h"
#include "Engine/Level.h"
#include "Factories/TedsEditorHierarchyFactory.h"
#include "GameFramework/Actor.h"
#include "LevelInstance/LevelInstanceInterface.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TedsActorParentQueries)

namespace UE::Editor::DataStorage::ActorParentQueries::Private
{
	static bool bAddParentColumnToActors = true;

	// Cvar to add parenting info for actors to TEDS
	static FAutoConsoleVariableRef CvarAddParentColumnToActors(
		TEXT("TEDS.AddParentColumnToActors"),
		bAddParentColumnToActors,
		TEXT("Mirror parent information for actors to TEDS (only works when set on startup)"));
};

void UActorParentDataStorageFactory::RegisterQueries(UE::Editor::DataStorage::ICoreProvider& InDataStorage)
{
	if (UE::Editor::DataStorage::ActorParentQueries::Private::bAddParentColumnToActors)
	{
		RegisterActorParentQueries(InDataStorage);
	}
}

void UActorParentDataStorageFactory::PreRegister(UE::Editor::DataStorage::ICoreProvider& InDataStorage)
{
	GEngine->OnLevelActorDetached().AddUObject(this, &UActorParentDataStorageFactory::OnLevelActorDetached);

	// Store a pointer to data storage so we don't have to grab it from the global instance of the registry every time
	DataStorage = &InDataStorage;
}

void UActorParentDataStorageFactory::PreShutdown(UE::Editor::DataStorage::ICoreProvider& InDataStorage)
{
	GEngine->OnLevelActorDetached().RemoveAll(this);
}

void UActorParentDataStorageFactory::RegisterActorParentQueries(UE::Editor::DataStorage::ICoreProvider& InDataStorage) const
{
	using namespace UE::Editor::DataStorage;
	using namespace UE::Editor::DataStorage::Queries;

	InDataStorage.RegisterQuery(
		Select(
			TEXT("Add parent column to actor"),
			FProcessor(EQueryTickPhase::PrePhysics, InDataStorage.GetQueryTickGroupName(EQueryTickGroups::SyncExternalToDataStorage))
			.SetExecutionMode(EExecutionMode::GameThread),
			[](IQueryContext& Context, RowHandle Row, const FTypedElementUObjectColumn& Actor)
			{
				if (const AActor* ActorInstance = Cast<AActor>(Actor.Object))
				{
					FMapKey IdKey;
					FName MappingDomain;
					
					if (ActorFolders::GetEditorParentForActor(ActorInstance, MappingDomain, IdKey))
					{
						RowHandle ParentRow = Context.LookupMappedRow(MappingDomain, IdKey);
					
						if (Context.IsRowAvailable(ParentRow))
						{
							Context.SetParentRow(Row, ParentRow);
						}
						else
						{
							Context.SetUnresolvedParent(Row, IdKey, MappingDomain);
						}
					}
					
				}
			})
		.AccessesHierarchy(UTedsEditorHierarchyFactory::EditorObjectHierarchyName)
		.Where()
			.All<FTypedElementSyncFromWorldTag, FTypedElementActorTag>()
			.None(InDataStorage.GetChildTagType(EditorActorHierarchyHandle))
		.Compile());

	InDataStorage.RegisterQuery(
		Select(
			TEXT("Sync actor's parent to column"),
			FProcessor(EQueryTickPhase::PrePhysics, InDataStorage.GetQueryTickGroupName(EQueryTickGroups::SyncExternalToDataStorage))
			.SetExecutionMode(EExecutionMode::GameThread),
			[](IQueryContext& Context, RowHandle TargetRow, const FTypedElementUObjectColumn& Actor)
			{
				if (const AActor* ActorInstance = Cast<AActor>(Actor.Object))
				{
					FMapKey IdKey;
					FName MappingDomain;
					
					if (ActorFolders::GetEditorParentForActor(ActorInstance, MappingDomain, IdKey))
					{
						RowHandle NewParentRow = Context.LookupMappedRow(MappingDomain, IdKey);
						RowHandle CurrentParentRow = Context.GetParentRow(TargetRow);

						if (Context.IsRowAvailable(NewParentRow))
						{
							if (CurrentParentRow != NewParentRow)
							{
								Context.SetParentRow(TargetRow, NewParentRow);
							}
						}
						else
						{
							Context.SetUnresolvedParent(TargetRow, IdKey, MappingDomain);
						}
						return;
					}
				}

				// If we have reached here, the actor does not have a parent
				Context.SetParentRow(TargetRow, InvalidRowHandle);
			})
		.AccessesHierarchy(UTedsEditorHierarchyFactory::EditorObjectHierarchyName)
		.Where()
			.All<FTypedElementActorTag, FTypedElementSyncFromWorldTag>()
		.Compile());
}

void UActorParentDataStorageFactory::OnLevelActorDetached(AActor* Actor, const AActor* OldParent) const
{
	// Detaching an actor from its parent only calls Modify() on the parent and not on the child, so we explicitly add the sync tag to the child so the
	// hierarchy in TEDS is updated
	using namespace UE::Editor::DataStorage;
	
	RowHandle Row = DataStorage->LookupMappedRow(ICompatibilityProvider::ObjectMappingDomain, FMapKey(Actor));
	DataStorage->AddColumn<FTypedElementSyncFromWorldTag>(Row);
}
