// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataStorage/CommonTypes.h"
#include "DataStorage/Handles.h"
#include "Elements/Interfaces/TypedElementDataStorageFactory.h"
#include "UObject/ObjectMacros.h"

#include "TedsActorParentQueries.generated.h"

UCLASS()
class UActorParentDataStorageFactory : public UEditorDataStorageFactory
{
	GENERATED_BODY()

public:
	~UActorParentDataStorageFactory() override = default;

	virtual void RegisterQueries(UE::Editor::DataStorage::ICoreProvider& DataStorage) override;
	virtual void PreRegister(UE::Editor::DataStorage::ICoreProvider& DataStorage) override;
	virtual void PreShutdown(UE::Editor::DataStorage::ICoreProvider& DataStorage) override;

private:
	/**
	 * Sets the hierarchy for an actor in the editor
	 */
	void RegisterActorParentQueries(UE::Editor::DataStorage::ICoreProvider& DataStorage) const;
	
	/**
	 * Function called when an actor is detached from its parent in the world
	 */
	void OnLevelActorDetached(AActor* Actor, const AActor* OldParent) const;

private:
	UE::Editor::DataStorage::FHierarchyHandle EditorActorHierarchyHandle;
	UE::Editor::DataStorage::ICoreProvider* DataStorage;
};
