// Copyright Epic Games, Inc. All Rights Reserved.

#include "Compatibility/TedsActorRootObjectQueries.h"

#include "ActorFolders/ActorFolderColumns.h"
#include "Elements/Columns/TypedElementCompatibilityColumns.h"
#include "Elements/Columns/TypedElementMiscColumns.h"
#include "Elements/Framework/TypedElementQueryBuilder.h"
#include "GameFramework/Actor.h"

void UActorRootObjectDataStorageFactory::RegisterQueries(UE::Editor::DataStorage::ICoreProvider& DataStorage)
{
	RegisterActorAddRootObjectColumn(DataStorage);
	RegisterActorRootObjectToColumnQuery(DataStorage);
}

void UActorRootObjectDataStorageFactory::RegisterActorAddRootObjectColumn(UE::Editor::DataStorage::ICoreProvider& DataStorage) const
{
	using namespace UE::Editor::DataStorage::Queries;

	DataStorage.RegisterQuery(
		Select(
			TEXT("Actor Root Object to New Column"),
			FProcessor(EQueryTickPhase::PrePhysics, DataStorage.GetQueryTickGroupName(EQueryTickGroups::SyncExternalToDataStorage))
			.SetExecutionMode(EExecutionMode::GameThread),
			[](IQueryContext& Context, RowHandle Row, const FTypedElementUObjectColumn& Object)
			{
				if (const AActor* Actor = Cast<AActor>(Object.Object); Actor != nullptr)
				{
					Context.AddColumn(Row, FRootObjectColumn{ .RootObject = Actor->GetFolderRootObject() });
				}
			})
		.Where()
			.All<FTypedElementActorTag, FTypedElementSyncFromWorldTag>()
			.None<FRootObjectColumn>()
		.Compile()
	);
}

void UActorRootObjectDataStorageFactory::RegisterActorRootObjectToColumnQuery(UE::Editor::DataStorage::ICoreProvider& DataStorage) const
{
	using namespace UE::Editor::DataStorage::Queries;

	DataStorage.RegisterQuery(
		Select(
			TEXT("Actor Root Object to Column"),
			FProcessor(EQueryTickPhase::PrePhysics, DataStorage.GetQueryTickGroupName(EQueryTickGroups::SyncExternalToDataStorage))
			.SetExecutionMode(EExecutionMode::GameThread),
			[](IQueryContext&, RowHandle, const FTypedElementUObjectColumn& Object, FRootObjectColumn& RootObjectColumn)
			{
				if (const AActor* Actor = Cast<AActor>(Object.Object); Actor != nullptr)
				{
					RootObjectColumn.RootObject = Actor->GetFolderRootObject();
				}
			})
		.Where()
			.All<FTypedElementActorTag, FTypedElementSyncFromWorldTag>()
		.Compile()
	);
}
