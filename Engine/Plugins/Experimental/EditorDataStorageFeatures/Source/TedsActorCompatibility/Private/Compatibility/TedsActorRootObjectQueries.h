// Copyright Epic Games, Inc.All Rights Reserved.

#pragma once

#include "Elements/Interfaces/TypedElementDataStorageFactory.h"
#include "UObject/ObjectMacros.h"

#include "TedsActorRootObjectQueries.generated.h"

UCLASS()
class UActorRootObjectDataStorageFactory : public UEditorDataStorageFactory
{
	GENERATED_BODY()

public:
	~UActorRootObjectDataStorageFactory() override = default;

	void RegisterQueries(UE::Editor::DataStorage::ICoreProvider& DataStorage) override;
private:

	/**
	 * Adds the root object column to new actors that do not have one already.
	 */
	void RegisterActorAddRootObjectColumn(UE::Editor::DataStorage::ICoreProvider& DataStorage) const;
	/**
	 * Takes the folder root object of the actor and copies it to the Data Storage.
	 */
	void RegisterActorRootObjectToColumnQuery(UE::Editor::DataStorage::ICoreProvider& DataStorage) const;
};
