// Copyright Epic Games, Inc. All Rights Reserved.

#include "Compatibility/TedsActorSocketQueries.h"

#include "Elements/Columns/TypedElementCompatibilityColumns.h"
#include "Elements/Columns/TypedElementMiscColumns.h"
#include "Columns/TedsActorSocketColumns.h"
#include "Elements/Framework/TypedElementQueryBuilder.h"
#include "GameFramework/Actor.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TedsActorSocketQueries)

void UActorSocketDataStorageFactory::RegisterQueries(UE::Editor::DataStorage::ICoreProvider& DataStorage)
{
	RegisterActorAddSocketColumn(DataStorage);
	RegisterActorSocketToColumnQuery(DataStorage);
}

void UActorSocketDataStorageFactory::RegisterActorAddSocketColumn(UE::Editor::DataStorage::ICoreProvider& DataStorage) const
{
	using namespace UE::Editor::DataStorage::Queries;

	DataStorage.RegisterQuery(
		Select(
			TEXT("Add the attached Socket of an Actor to a New Column"),
			FProcessor(EQueryTickPhase::PrePhysics, DataStorage.GetQueryTickGroupName(EQueryTickGroups::SyncExternalToDataStorage))
			.SetExecutionMode(EExecutionMode::GameThread),
			[](IQueryContext& Context, RowHandle Row, const FTypedElementUObjectColumn& Object)
			{
				if (const AActor* ActorInstance = Cast<AActor>(Object.Object); ActorInstance != nullptr)
				{
					if (const FName SocketName = ActorInstance->GetAttachParentSocketName(); !SocketName.IsNone())
					{
						Context.AddColumn(Row, FTedsActorSocketColumn
							{ .AttachedSocket = SocketName });
					}
				}
			})
		.Where()
			.All<FTypedElementActorTag, FTypedElementSyncFromWorldTag>()
			.None<FTedsActorSocketColumn>()
		.Compile()
	);
}

void UActorSocketDataStorageFactory::RegisterActorSocketToColumnQuery(UE::Editor::DataStorage::ICoreProvider& DataStorage) const
{
	using namespace UE::Editor::DataStorage::Queries;

	DataStorage.RegisterQuery(
		Select(
			TEXT("Sync Actor's Attached Socket to Column"),
			FProcessor(EQueryTickPhase::PrePhysics, DataStorage.GetQueryTickGroupName(EQueryTickGroups::SyncExternalToDataStorage))
			.SetExecutionMode(EExecutionMode::GameThread),
			[](IQueryContext& Context, RowHandle Row, const FTypedElementUObjectColumn& Object, FTedsActorSocketColumn& SocketColumn)
			{
				if (const AActor* ActorInstance = Cast<AActor>(Object.Object); ActorInstance != nullptr)
				{
					if (const FName SocketName = ActorInstance->GetAttachParentSocketName(); !SocketName.IsNone())
					{
						SocketColumn.AttachedSocket = SocketName;
					}
					else
					{
						Context.RemoveColumns<FTedsActorSocketColumn>(Row);
					}
				}
			})
		.Where()
			.All<FTypedElementActorTag, FTypedElementSyncFromWorldTag>()
		.Compile()
	);
}