// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/Interfaces/TypedElementDataStorageFactory.h"

#include "TedsActorSocketQueries.generated.h"

UCLASS()
class UActorSocketDataStorageFactory : public UEditorDataStorageFactory
{
	GENERATED_BODY()

public:
	~UActorSocketDataStorageFactory() override = default;

	void RegisterQueries(UE::Editor::DataStorage::ICoreProvider& DataStorage) override;
private:

	/**
	 * Adds the socket column to new actors that do not have one already.
	 */
	void RegisterActorAddSocketColumn(UE::Editor::DataStorage::ICoreProvider& DataStorage) const;
	/**
	 * Takes the socket name set on an actor and copies it to the Data Storage.
	 */
	void RegisterActorSocketToColumnQuery(UE::Editor::DataStorage::ICoreProvider& DataStorage) const;
};
