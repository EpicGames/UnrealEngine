// Copyright Epic Games, Inc. All Rights Reserved.

#include "Compatibility/TedsActorUIDisplayQueries.h"

#include "ActorEditorUtils.h"
#include "Columns/TedsActorComponentCompatibilityColumns.h"
#include "Elements/Columns/TypedElementCompatibilityColumns.h"
#include "Elements/Columns/TypedElementMiscColumns.h"
#include "Elements/Columns/TypedElementUIColumns.h"
#include "Elements/Framework/TypedElementQueryBuilder.h"
#include "GameFramework/Actor.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TedsActorUIDisplayQueries)

void UActorUIDisplayDataStorageFactory::RegisterQueries(UE::Editor::DataStorage::ICoreProvider& DataStorage)
{
	RegisterSyncActorHideRowFromUITagQuery(DataStorage);
}

void UActorUIDisplayDataStorageFactory::RegisterSyncActorHideRowFromUITagQuery(UE::Editor::DataStorage::ICoreProvider& DataStorage) const
{
	using namespace UE::Editor::DataStorage::Queries;
	
	DataStorage.RegisterQuery(
		Select(
			TEXT("Sync Actor's Hide From UI State"),
			FProcessor(EQueryTickPhase::PrePhysics, DataStorage.GetQueryTickGroupName(EQueryTickGroups::SyncExternalToDataStorage))
			.SetExecutionMode(EExecutionMode::GameThread),
			[](IQueryContext& Context, RowHandle Row, const FTypedElementUObjectColumn& Object)
			{
				if (FActorEditorUtils::IsActorDisplayable(Cast<AActor>(Object.Object)))
				{
					Context.RemoveColumns<FHideRowFromUITag>(Row);
				}
				else
				{
					Context.AddColumns<FHideRowFromUITag>(Row);
				}
			})
		.Where()
			.All<FTypedElementSyncFromWorldTag, FTypedElementActorTag>()
		.Compile()
	);
	DataStorage.RegisterQuery(
		Select(
			TEXT("Sync Actor Component's Hide From UI State"),
			FProcessor(EQueryTickPhase::PrePhysics, DataStorage.GetQueryTickGroupName(EQueryTickGroups::SyncExternalToDataStorage))
			.SetExecutionMode(EExecutionMode::GameThread),
			[](IQueryContext& Context, RowHandle Row, const FTypedElementUObjectColumn& Object)
			{
				if (const UActorComponent* ActorComponentInstance = Cast<UActorComponent>(Object.Object); ActorComponentInstance != nullptr)
				{
					if (FActorEditorUtils::IsActorDisplayable(Cast<AActor>(ActorComponentInstance->GetOuter())))
					{
						Context.RemoveColumns<FHideRowFromUITag>(Row);
					}
					else
					{
						Context.AddColumns<FHideRowFromUITag>(Row);
					}
				}
			})
		.Where()
			.All<FTypedElementSyncFromWorldTag, FActorComponentTypeTag>()
		.Compile()
	);
}
