// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/Interfaces/TypedElementDataStorageFactory.h"
#include "UObject/ObjectMacros.h"

#include "TedsActorUIDisplayQueries.generated.h"

UCLASS()
class UActorUIDisplayDataStorageFactory : public UEditorDataStorageFactory
{
	GENERATED_BODY()

public:
	~UActorUIDisplayDataStorageFactory() override = default;

	void RegisterQueries(UE::Editor::DataStorage::ICoreProvider& DataStorage) override;
private:
	/**
	 * Registers queries to check if the updated Actor row or the Component row of an Actor should be given the HideRowFromUITag
	 */
	void RegisterSyncActorHideRowFromUITagQuery(UE::Editor::DataStorage::ICoreProvider& DataStorage) const;
};
