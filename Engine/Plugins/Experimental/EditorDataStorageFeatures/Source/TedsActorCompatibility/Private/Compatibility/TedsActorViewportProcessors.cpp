// Copyright Epic Games, Inc. All Rights Reserved.

#include "Compatibility/TedsActorViewportProcessors.h"

#include "Components/PrimitiveComponent.h"
#include "Editor.h"
#include "DataStorage/Features.h"
#include "Elements/Columns/TypedElementCompatibilityColumns.h"
#include "Elements/Columns/TypedElementViewportColumns.h"
#include "Elements/Columns/TypedElementMiscColumns.h"
#include "Elements/Framework/TypedElementQueryBuilder.h"
#include "Elements/Interfaces/TypedElementDataStorageInterface.h"
#include "HAL/IConsoleManager.h"
#include "Elements/Columns/TypedElementSelectionColumns.h"
#include "ProfilingDebugging/CpuProfilerTrace.h"


#include UE_INLINE_GENERATED_CPP_BY_NAME(TedsActorViewportProcessors)

namespace UE::Editor::DataStorage::Private
{
	FAutoConsoleCommandWithArgsAndOutputDevice SetOutlineColorConsoleCommand(
		TEXT("TEDS.Debug.SetOutlineColor"),
		TEXT("Adds an outline color to selected objects."),
		FConsoleCommandWithArgsAndOutputDeviceDelegate::CreateLambda([](const TArray<FString>& Args, FOutputDevice& Output)
			{
				using namespace UE::Editor::DataStorage::Queries;

				TRACE_CPUPROFILER_EVENT_SCOPE(TEDS.Debug.AddOverlayColorToSelectionCommand);

				if (ICoreProvider* DataStorage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName))
				{
					static QueryHandle OverlayQuery = InvalidQueryHandle;
					if (OverlayQuery == InvalidQueryHandle)
					{
						OverlayQuery = DataStorage->RegisterQuery(
							Select()
							.Where()
								.All<FTypedElementSelectionColumn>()
							.Compile());
					}
				
					if (OverlayQuery == InvalidQueryHandle)
					{
						return;
					}

					if (Args.IsEmpty())
					{
						Output.Log(TEXT("Provide a color index (0-7) to use as outline"));
						return;
					}
				
					// Parse the color
					int32 ColorIndex;
					LexFromString(ColorIndex, *Args[0]);

					if (!(ColorIndex >= 0 && ColorIndex <= 7))
					{
						Output.Log(TEXT("Color index must be in range [0,7]"));
						return;
					}

					TArray<RowHandle> RowHandles;
				
					DataStorage->RunQuery(OverlayQuery, CreateDirectQueryCallbackBinding(
						[ColorIndex, &RowHandles](IDirectQueryContext& Context, const RowHandle*)
					{
						RowHandles.Append(Context.GetRowHandles());
					}));

					for (RowHandle Row : RowHandles)
					{
						DataStorage->AddColumn(Row, 
							FTypedElementViewportOutlineColorColumn{ .SelectionOutlineColorIndex = static_cast<uint8>(ColorIndex) });
						DataStorage->AddColumn<FTypedElementSyncBackToWorldTag>(Row);
					}
				}
			}));

	FAutoConsoleCommandWithArgsAndOutputDevice SetSelectionOverlayColorConsoleCommand(
		TEXT("TEDS.Debug.SetOverlayColor"),
		TEXT("Adds an overlay color to selected objects."),
		FConsoleCommandWithArgsAndOutputDeviceDelegate::CreateLambda([](const TArray<FString>& Args, FOutputDevice& Output)
			{
				using namespace UE::Editor::DataStorage::Queries;

				TRACE_CPUPROFILER_EVENT_SCOPE(TEDS.Debug.AddOverlayColorToSelectionCommand);

				if (ICoreProvider* DataStorage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName))
				{
					static QueryHandle OverlayQuery = InvalidQueryHandle;
					if (OverlayQuery == InvalidQueryHandle)
					{
						OverlayQuery = DataStorage->RegisterQuery(
							Select()
							.Where()
								.All<FTypedElementSelectionColumn>()
							.Compile());
					}
				
					if (OverlayQuery == InvalidQueryHandle)
					{
						return;
					}

					if (Args.IsEmpty())
					{
						Output.Log(TEXT("Provide a color in hexadecimal format (#RRGGBBAA) to overlay."));
						return;
					}
				
					// Parse the color
					FColor Color = FColor::FromHex(Args[0]);
					Color.A = FMath::Clamp<uint8>(Color.A, 0, 128);

					TArray<RowHandle> RowHandles;
				
					DataStorage->RunQuery(OverlayQuery, CreateDirectQueryCallbackBinding(
						[Color, &RowHandles](IDirectQueryContext& Context, const RowHandle*)
					{
						RowHandles.Append(Context.GetRowHandles());
					}));

					for (RowHandle Row : RowHandles)
					{
						DataStorage->AddColumn(Row, FTypedElementViewportOverlayColorColumn{ .OverlayColor = Color });
						DataStorage->AddColumns<FTypedElementViewportOverlayColorDirtyFlag>(Row);
					}
				}
			}));

	FAutoConsoleCommandWithArgsAndOutputDevice RemoveSelectionOverlayColorConsoleCommand(
		TEXT("TEDS.Debug.RemoveOverlayColor"),
		TEXT("Removes an overlay color to selected objects."),
		FConsoleCommandWithArgsAndOutputDeviceDelegate::CreateLambda([](const TArray<FString>& Args, FOutputDevice& Output)
			{
				using namespace UE::Editor::DataStorage::Queries;

				TRACE_CPUPROFILER_EVENT_SCOPE(TEDS.Debug.AddOverlayColorToSelectionCommand);

				if (ICoreProvider* DataStorage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName))
				{
					static QueryHandle OverlayQuery = InvalidQueryHandle;
					if (OverlayQuery == InvalidQueryHandle)
					{
						OverlayQuery = DataStorage->RegisterQuery(
							Select()
							.Where()
								.All<FTypedElementSelectionColumn>()
							.Compile());
					}
				
					if (OverlayQuery == InvalidQueryHandle)
					{
						return;
					}

					TArray<RowHandle> RowHandles;
				
					DataStorage->RunQuery(OverlayQuery, CreateDirectQueryCallbackBinding(
							[&RowHandles](IDirectQueryContext& Context, const RowHandle*)
					{
						RowHandles.Append(Context.GetRowHandles());
					}));

					for (RowHandle Row : RowHandles)
					{
						DataStorage->RemoveColumn<FTypedElementViewportOverlayColorColumn>(Row);
					}
				}
			}));
} // namespace UE::Editor::DataStorage::Private

void UActorViewportDataStorageFactory::RegisterQueries(UE::Editor::DataStorage::ICoreProvider& DataStorage)
{
	RegisterOutlineColorColumnToActor(DataStorage);
	RegisterOverlayColorColumnToActor(DataStorage);
}

void UActorViewportDataStorageFactory::RegisterOutlineColorColumnToActor(UE::Editor::DataStorage::ICoreProvider& DataStorage)
{
	using namespace UE::Editor::DataStorage::Queries;

	DataStorage.RegisterQuery(
		Select(
			TEXT("Sync viewport outline color column to actor"),
			FProcessor(EQueryTickPhase::DuringPhysics, DataStorage.GetQueryTickGroupName(EQueryTickGroups::SyncDataStorageToExternal))
				.SetExecutionMode(EExecutionMode::GameThread),
			[](FTypedElementUObjectColumn& Actor, const FTypedElementViewportOutlineColorColumn& ViewportColor)
			{
				if (AActor* ActorInstance = Cast<AActor>(Actor.Object); ActorInstance != nullptr)
				{
					bool bIncludeFromChildActors = false;
					ActorInstance->ForEachComponent<UPrimitiveComponent>(bIncludeFromChildActors, [&ViewportColor](UPrimitiveComponent* PrimitiveComponent)
					{
						PrimitiveComponent->SetSelectionOutlineColorIndex(ViewportColor.SelectionOutlineColorIndex);
					});
				}
			})
		.Where()
			.All<FTypedElementActorTag, FTypedElementSyncBackToWorldTag>()
		.Compile());
}

void UActorViewportDataStorageFactory::RegisterOverlayColorColumnToActor(UE::Editor::DataStorage::ICoreProvider& DataStorage)
{
	using namespace UE::Editor::DataStorage::Queries;
	
	auto RefreshActorOverlayColor = [](FTypedElementUObjectColumn& Actor, const FTypedElementViewportOverlayColorColumn& ViewportColor)
		{
			if (AActor* ActorInstance = Cast<AActor>(Actor.Object); ActorInstance != nullptr)
			{
				bool bIncludeFromChildActors = true;
				ActorInstance->ForEachComponent<UPrimitiveComponent>(bIncludeFromChildActors, [&ViewportColor](UPrimitiveComponent* PrimitiveComponent)
					{
						PrimitiveComponent->SetOverlayColor(ViewportColor.OverlayColor);
						PrimitiveComponent->MarkRenderStateDirty();
					});
				GEditor->RedrawLevelEditingViewports();
			}
		};

	DataStorage.RegisterQuery(
		Select(
			TEXT("Sync viewport overlay color column to actor"),
			FObserver::OnAdd<FTypedElementViewportOverlayColorColumn>()
				.SetExecutionMode(EExecutionMode::GameThread),
				RefreshActorOverlayColor
			)
		.Where()
			.All<FTypedElementActorTag>()
		.Compile());

	DataStorage.RegisterQuery(
		Select(
			TEXT("Refresh viewport overlay color column to actor"),
			FProcessor(EQueryTickPhase::PostPhysics, DataStorage.GetQueryTickGroupName(EQueryTickGroups::Update))
				.SetExecutionMode(EExecutionMode::GameThread),
				RefreshActorOverlayColor
			)
		.Where()
			.All<FTypedElementActorTag, FTypedElementViewportOverlayColorDirtyFlag>()
		.Compile());

	DataStorage.RegisterQuery(
		Select(
			TEXT("Remove viewport overlay dirty flag to actor"),
			FProcessor(EQueryTickPhase::FrameEnd, DataStorage.GetQueryTickGroupName(EQueryTickGroups::Update))
				.SetExecutionMode(EExecutionMode::GameThread),
				[](IQueryContext& ActorQueryContext, RowHandle ObjectRow)
				{
					ActorQueryContext.RemoveColumns<FTypedElementViewportOverlayColorDirtyFlag>(ObjectRow);
				}
			)
		.Where()
			.All<FTypedElementActorTag, FTypedElementViewportOverlayColorDirtyFlag>()
		.Compile());

	DataStorage.RegisterQuery(
		Select(
			TEXT("Remove viewport overlay color column from actor"),
			FObserver::OnRemove<FTypedElementViewportOverlayColorColumn>(),
			[](FTypedElementUObjectColumn& Actor)
			{
				if (AActor* ActorInstance = Cast<AActor>(Actor.Object); ActorInstance != nullptr)
				{
					bool bIncludeFromChildActors = true;
					ActorInstance->ForEachComponent<UPrimitiveComponent>(bIncludeFromChildActors, [](UPrimitiveComponent* PrimitiveComponent)
					{
						PrimitiveComponent->RemoveOverlayColor();
						PrimitiveComponent->MarkRenderStateDirty();
					});
					GEditor->RedrawLevelEditingViewports();
				}
			})
		.Where()
			.All<FTypedElementActorTag>()
		.Compile());
}
