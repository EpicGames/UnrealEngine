// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/Interfaces/TypedElementDataStorageFactory.h"
#include "UObject/ObjectMacros.h"

#include "TedsActorViewportProcessors.generated.h"

UCLASS()
class UActorViewportDataStorageFactory : public UEditorDataStorageFactory
{
	GENERATED_BODY()

public:
	~UActorViewportDataStorageFactory() override = default;

	void RegisterQueries(UE::Editor::DataStorage::ICoreProvider& DataStorage) override;

private:
	void RegisterOutlineColorColumnToActor(UE::Editor::DataStorage::ICoreProvider& DataStorage);
	void RegisterOverlayColorColumnToActor(UE::Editor::DataStorage::ICoreProvider& DataStorage);
};
