// Copyright Epic Games, Inc. All Rights Reserved.
#include "Compatibility/TedsActorVisibilityQueries.h"

#include "DataStorage/Queries/Types.h"
#include "Elements/Columns/TypedElementCompatibilityColumns.h"
#include "Elements/Columns/TypedElementMiscColumns.h"
#include "Elements/Columns/TypedElementVisibilityColumns.h"
#include "Elements/Framework/TypedElementQueryBuilder.h"
#include "GameFramework/Actor.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TedsActorVisibilityQueries)

void UActorVisibilityDataStorageFactory::RegisterQueries(UE::Editor::DataStorage::ICoreProvider& DataStorage)
{
	RegisterActorAddVisibilityColumn(DataStorage);
	RegisterActorVisibilityToColumnQuery(DataStorage);
	RegisterVisibilityColumnToActorQuery(DataStorage);
}

void UActorVisibilityDataStorageFactory::RegisterActorAddVisibilityColumn(UE::Editor::DataStorage::ICoreProvider& DataStorage) const
{
	using namespace UE::Editor::DataStorage::Queries;

	DataStorage.RegisterQuery(
		Select(
			TEXT("Actor Visibility Object to New Column"),
			FProcessor(EQueryTickPhase::PrePhysics, DataStorage.GetQueryTickGroupName(EQueryTickGroups::SyncExternalToDataStorage)).SetExecutionMode(EExecutionMode::GameThread),
			[](IQueryContext& Context, RowHandle Row, const FTypedElementUObjectColumn& Object)
			{
				if (const AActor* ActorInstance = Cast<AActor>(Object.Object); ActorInstance != nullptr)
				{
					Context.AddColumn(Row, FVisibleInEditorColumn{ .bIsVisibleInEditor = !ActorInstance->IsTemporarilyHiddenInEditor() });
				}
			})
		.Where()
			.All<FTypedElementActorTag, FTypedElementSyncFromWorldTag>()
			.None<FVisibleInEditorColumn>()
		.Compile()
	);
}

void UActorVisibilityDataStorageFactory::RegisterActorVisibilityToColumnQuery(UE::Editor::DataStorage::ICoreProvider& DataStorage) const
{
	using namespace UE::Editor::DataStorage::Queries;

	DataStorage.RegisterQuery(
		Select(
			TEXT("Actor Visibility Object to Column"),
			FProcessor(EQueryTickPhase::PrePhysics, DataStorage.GetQueryTickGroupName(EQueryTickGroups::SyncExternalToDataStorage)).SetExecutionMode(EExecutionMode::GameThread),
			[](IQueryContext& Context, RowHandle Row, const FTypedElementUObjectColumn& Object, FVisibleInEditorColumn& VisibilityColumn)
			{
				if (const AActor* ActorInstance = Cast<AActor>(Object.Object); ActorInstance != nullptr)
				{
					VisibilityColumn.bIsVisibleInEditor = !ActorInstance->IsTemporarilyHiddenInEditor();
				}
			})
		.Where()
			.All<FTypedElementActorTag, FTypedElementSyncFromWorldTag>()
		.Compile()
	);
}

void UActorVisibilityDataStorageFactory::RegisterVisibilityColumnToActorQuery(UE::Editor::DataStorage::ICoreProvider& DataStorage) const
{
	using namespace UE::Editor::DataStorage::Queries;

	struct FSetActorVisibilityCommand
	{
		void operator()()
		{
			TStrongObjectPtr<AActor> PinnedActor = Actor.Pin();
			if (PinnedActor)
			{
				PinnedActor->SetIsTemporarilyHiddenInEditor(!bIsVisible);
			}
			
		}
		TWeakObjectPtr<AActor> Actor;
		bool bIsVisible;
	};
	
	DataStorage.RegisterQuery(
		Select(
			TEXT("Actor Visibility Column to Object"),
			FProcessor(EQueryTickPhase::PrePhysics, DataStorage.GetQueryTickGroupName(EQueryTickGroups::SyncDataStorageToExternal)).SetExecutionMode(EExecutionMode::GameThread),
			[&](IQueryContext& Context, RowHandle Row, FTypedElementUObjectColumn& Object, const FVisibleInEditorColumn& VisibilityColumn)
			{
				if (AActor* ActorInstance = Cast<AActor>(Object.Object); ActorInstance != nullptr)
				{
					Context.PushCommand(FSetActorVisibilityCommand
							{
								.Actor = ActorInstance,
								.bIsVisible = VisibilityColumn.bIsVisibleInEditor
							});
				}
			})
		.Where()
			.All<FTypedElementActorTag, FVisibilityChangedTag>()
		.Compile()
	);
}
