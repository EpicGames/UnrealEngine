// Copyright Epic Games, Inc. All Rights Reserved.

#include "Compatibility/TedsActorWorldPartitionContentBundleQueries.h"

#include "Columns/TedsActorWorldPartitionColumns.h"
#include "DataStorage/Queries/Types.h"
#include "Elements/Columns/TypedElementCompatibilityColumns.h"
#include "Elements/Columns/TypedElementMiscColumns.h"
#include "Elements/Framework/TypedElementQueryBuilder.h"
#include "GameFramework/Actor.h"
#include "WorldPartition/ContentBundle/ContentBundleDescriptor.h"
#include "WorldPartition/ContentBundle/ContentBundleEngineSubsystem.h"
#include "WorldPartition/WorldPartition.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TedsActorWorldPartitionContentBundleQueries)

namespace UE::Editor::DataStorage::Private
{
	/** Looks up the content bundle descriptor display name for a loaded actor, or returns empty string. */
	static FString BuildContentBundleString(const AActor* Actor)
	{
		if (const UContentBundleEngineSubsystem* Subsystem = UContentBundleEngineSubsystem::Get())
		{
			if (const UContentBundleDescriptor* Descriptor = Subsystem->GetContentBundleDescriptor(Actor->GetContentBundleGuid()))
			{
				return Descriptor->GetDisplayName();
			}
		}
		
		return TEXT("");
	}
} // namespace UE::Editor::DataStorage::Private

void UActorWorldPartitionContentBundleDataStorageFactory::RegisterQueries(UE::Editor::DataStorage::ICoreProvider& DataStorage)
{
	RegisterActorAddWorldPartitionContentBundleColumn(DataStorage);
	RegisterActorWorldPartitionContentBundleToColumnQuery(DataStorage);
}

void UActorWorldPartitionContentBundleDataStorageFactory::RegisterActorAddWorldPartitionContentBundleColumn(
	UE::Editor::DataStorage::ICoreProvider& DataStorage) const
{
	using namespace UE::Editor::DataStorage::Queries;

	DataStorage.RegisterQuery(
		Select(
			TEXT("Actor World Partition Content Bundle Object to New Column"),
			FProcessor(EQueryTickPhase::DuringPhysics, DataStorage.GetQueryTickGroupName(EQueryTickGroups::SyncExternalToDataStorage))
				.SetExecutionMode(EExecutionMode::GameThread),
			[](IQueryContext& Context, RowHandle Row, const FTypedElementUObjectColumn& Object)
			{
				if (const AActor* Actor = Cast<AActor>(Object.Object); Actor && Actor->GetContentBundleGuid().IsValid())
				{
					Context.AddColumn(Row, FWorldPartitionContentBundleColumn
						{
							.ContentBundle = UE::Editor::DataStorage::Private::BuildContentBundleString(Actor)
						});
				}
			})
		.Where()
			.All<FTypedElementActorTag, FTypedElementSyncFromWorldTag, FLevelColumn>()
			.None<FWorldPartitionContentBundleColumn>()
		.Compile()
	);
}

void UActorWorldPartitionContentBundleDataStorageFactory::RegisterActorWorldPartitionContentBundleToColumnQuery(
	UE::Editor::DataStorage::ICoreProvider& DataStorage) const
{
	using namespace UE::Editor::DataStorage::Queries;

	DataStorage.RegisterQuery(
		Select(
			TEXT("Actor World Partition Content Bundle Object to Column"),
			FProcessor(EQueryTickPhase::PrePhysics, DataStorage.GetQueryTickGroupName(EQueryTickGroups::SyncExternalToDataStorage))
				.SetExecutionMode(EExecutionMode::GameThread),
			[](IQueryContext&, RowHandle, const FTypedElementUObjectColumn& Object,
			   FWorldPartitionContentBundleColumn& ContentBundleColumn)
			{
				if (const AActor* Actor = Cast<AActor>(Object.Object))
				{
					ContentBundleColumn.ContentBundle = UE::Editor::DataStorage::Private::BuildContentBundleString(Actor);
				}
			})
		.Where()
			.All<FTypedElementActorTag, FTypedElementSyncFromWorldTag>()
		.Compile()
	);
}
