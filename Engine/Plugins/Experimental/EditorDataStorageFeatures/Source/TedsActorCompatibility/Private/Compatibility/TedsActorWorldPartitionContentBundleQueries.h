// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/Interfaces/TypedElementDataStorageFactory.h"
#include "UObject/ObjectMacros.h"

#include "TedsActorWorldPartitionContentBundleQueries.generated.h"

/**
 * Registers queries that keep FWorldPartitionContentBundleColumn in sync with the actor's
 * content bundle assignment. Read-only — no write-back to world.
 */
UCLASS()
class UActorWorldPartitionContentBundleDataStorageFactory : public UEditorDataStorageFactory
{
	GENERATED_BODY()

public:
	~UActorWorldPartitionContentBundleDataStorageFactory() override = default;

	void RegisterQueries(UE::Editor::DataStorage::ICoreProvider& DataStorage) override;

private:
	/**
	 * Adds FWorldPartitionContentBundleColumn to actor rows in WP worlds with content bundles
	 * enabled that do not already have the column.
	 */
	void RegisterActorAddWorldPartitionContentBundleColumn(UE::Editor::DataStorage::ICoreProvider& DataStorage) const;

	/**
	 * Syncs the actor's content bundle display name into FWorldPartitionContentBundleColumn
	 * when it may have changed.
	 */
	void RegisterActorWorldPartitionContentBundleToColumnQuery(UE::Editor::DataStorage::ICoreProvider& DataStorage) const;
};
