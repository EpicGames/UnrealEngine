// Copyright Epic Games, Inc. All Rights Reserved.

#include "Compatibility/TedsActorWorldPartitionDataLayerQueries.h"

#include "Algo/Find.h"
#include "Columns/TedsActorWorldPartitionColumns.h"
#include "DataStorage/Queries/Types.h"
#include "Elements/Columns/TypedElementCompatibilityColumns.h"
#include "Elements/Columns/TypedElementMiscColumns.h"
#include "Elements/Framework/TypedElementQueryBuilder.h"
#include "GameFramework/Actor.h"
#include "WorldPartition/DataLayer/DataLayerInstance.h"
#include "WorldPartition/DataLayer/ExternalDataLayerInstance.h"
#include "WorldPartition/WorldPartition.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TedsActorWorldPartitionDataLayerQueries)

namespace UE::Editor::DataStorage::Private
{
	// Builds the comma-separated data layer display string for a loaded actor, replicating the legacy DataLayerInfoText lambda in SceneOutlinerModule.cpp for the FActorTreeItem case.
	static FString BuildDataLayerString(const AActor* Actor)
	{
		TStringBuilder<128> Builder;
		TSet<FString> DataLayerShortNames;

		auto BuildDataLayers = [&Builder, &DataLayerShortNames](const TArray<const UDataLayerInstance*>& DataLayerInstances, bool bPartOfOtherLevel)
		{
			if (!bPartOfOtherLevel)
			{
				if (const UDataLayerInstance* const* ExternalDataLayerInstance = Algo::FindByPredicate(DataLayerInstances,
					[](const UDataLayerInstance* DataLayerInstance) { return DataLayerInstance->IsA<UExternalDataLayerInstance>(); }))
				{
					Builder += (*ExternalDataLayerInstance)->GetDataLayerShortName();
				}
			}

			for (const UDataLayerInstance* DataLayerInstance : DataLayerInstances)
			{
				if (!DataLayerInstance->IsA<UExternalDataLayerInstance>())
				{
					bool bIsAlreadyInSet = false;
					DataLayerShortNames.Add(DataLayerInstance->GetDataLayerShortName(), &bIsAlreadyInSet);
					if (!bIsAlreadyInSet)
					{
						if (Builder.Len())
						{
							Builder += TEXT(", ");
						}
						// Put a '*' in front of DataLayers that are not part of the main world
						if (bPartOfOtherLevel)
						{
							Builder += TEXT("*");
						}
						Builder += DataLayerInstance->GetDataLayerShortName();
					}
				}
			}
		};

		// List Actor's DataLayers part of the owning world, then those only part of the actor level
		BuildDataLayers(Actor->GetDataLayerInstances(), false);
		BuildDataLayers(Actor->GetDataLayerInstancesForLevel(), true);

		return Builder.ToString();
	}
} // namespace UE::Editor::DataStorage::Private

void UActorWorldPartitionDataLayerDataStorageFactory::RegisterQueries(UE::Editor::DataStorage::ICoreProvider& DataStorage)
{
	RegisterActorAddWorldPartitionDataLayerColumn(DataStorage);
	RegisterActorWorldPartitionDataLayerToColumnQuery(DataStorage);
}

void UActorWorldPartitionDataLayerDataStorageFactory::RegisterActorAddWorldPartitionDataLayerColumn(UE::Editor::DataStorage::ICoreProvider& DataStorage) const
{
	using namespace UE::Editor::DataStorage::Queries;

	DataStorage.RegisterQuery(
		Select(
			TEXT("Actor World Partition Data Layer Object to New Column"),
			FProcessor(EQueryTickPhase::DuringPhysics, DataStorage.GetQueryTickGroupName(EQueryTickGroups::SyncExternalToDataStorage)).SetExecutionMode(EExecutionMode::GameThread),
			[](IQueryContext& Context, RowHandle Row, const FTypedElementUObjectColumn& Object, const FTypedElementWorldColumn& WorldColumn)
			{
				// Return if not a WP World
				if (const UWorld* World = WorldColumn.World.Get(); !World || !World->GetWorldPartition())
				{
					return;
				}

				if (const AActor* Actor = Cast<AActor>(Object.Object))
				{
					Context.AddColumn(Row, FWorldPartitionDataLayerColumn{ .DataLayers = UE::Editor::DataStorage::Private::BuildDataLayerString(Actor) });
				}
			})
		.Where()
			.All<FTypedElementActorTag, FTypedElementSyncFromWorldTag, FLevelColumn>()
			.None<FWorldPartitionDataLayerColumn>()
		.Compile()
	);
}

void UActorWorldPartitionDataLayerDataStorageFactory::RegisterActorWorldPartitionDataLayerToColumnQuery(UE::Editor::DataStorage::ICoreProvider& DataStorage) const
{
	using namespace UE::Editor::DataStorage::Queries;

	DataStorage.RegisterQuery(
		Select(
			TEXT("Actor World Partition Data Layer Object to Column"),
			FProcessor(EQueryTickPhase::PrePhysics, DataStorage.GetQueryTickGroupName(EQueryTickGroups::SyncExternalToDataStorage)).SetExecutionMode(EExecutionMode::GameThread),
			[](IQueryContext&, RowHandle, const FTypedElementUObjectColumn& Object, FWorldPartitionDataLayerColumn& DataLayerColumn)
			{
				if (const AActor* Actor = Cast<AActor>(Object.Object))
				{
					DataLayerColumn.DataLayers = UE::Editor::DataStorage::Private::BuildDataLayerString(Actor);
				}
			})
		.Where()
			.All<FTypedElementActorTag, FTypedElementSyncFromWorldTag>()
		.Compile()
	);
}
