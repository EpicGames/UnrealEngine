// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/Interfaces/TypedElementDataStorageFactory.h"
#include "UObject/ObjectMacros.h"

#include "TedsActorWorldPartitionDataLayerQueries.generated.h"

/**
 * Registers queries that keep FWorldPartitionDataLayerColumn in sync with the actor's
 * data layer assignments. Read-only — no write-back to world.
 */
UCLASS()
class UActorWorldPartitionDataLayerDataStorageFactory : public UEditorDataStorageFactory
{
	GENERATED_BODY()

public:
	~UActorWorldPartitionDataLayerDataStorageFactory() override = default;

	void RegisterQueries(UE::Editor::DataStorage::ICoreProvider& DataStorage) override;

private:
	/**
	 * Adds FWorldPartitionDataLayerColumn to actor rows in WP worlds that do not already have the column.
	 */
	void RegisterActorAddWorldPartitionDataLayerColumn(UE::Editor::DataStorage::ICoreProvider& DataStorage) const;

	/**
	 * Syncs the actor's data layer names into FWorldPartitionDataLayerColumn when they may have changed.
	 */
	void RegisterActorWorldPartitionDataLayerToColumnQuery(UE::Editor::DataStorage::ICoreProvider& DataStorage) const;
};
