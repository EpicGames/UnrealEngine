// Copyright Epic Games, Inc. All Rights Reserved.

#include "Compatibility/TedsActorWorldPartitionSubPackageQueries.h"

#include "Columns/TedsActorWorldPartitionColumns.h"
#include "DataStorage/Queries/Types.h"
#include "Elements/Columns/TypedElementCompatibilityColumns.h"
#include "Elements/Columns/TypedElementMiscColumns.h"
#include "Elements/Framework/TypedElementQueryBuilder.h"
#include "GameFramework/Actor.h"
#include "LevelInstance/LevelInstanceInterface.h"
#include "Engine/World.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TedsActorWorldPartitionSubPackageQueries)

void UActorWorldPartitionSubPackageDataStorageFactory::RegisterQueries(UE::Editor::DataStorage::ICoreProvider& DataStorage)
{
	RegisterActorAddWorldPartitionSubPackageColumn(DataStorage);
	RegisterActorWorldPartitionSubPackageToColumnQuery(DataStorage);
}

void UActorWorldPartitionSubPackageDataStorageFactory::RegisterActorAddWorldPartitionSubPackageColumn(UE::Editor::DataStorage::ICoreProvider& DataStorage) const
{
	using namespace UE::Editor::DataStorage::Queries;

	DataStorage.RegisterQuery(
		Select(
			TEXT("Actor World Partition Sub Package Object to New Column"),
			FProcessor(EQueryTickPhase::PrePhysics, DataStorage.GetQueryTickGroupName(EQueryTickGroups::SyncExternalToDataStorage)).SetExecutionMode(EExecutionMode::GameThread),
			[](IQueryContext& Context, RowHandle Row, const FTypedElementUObjectColumn& Object, const FTypedElementWorldColumn& WorldColumn)
			{
				// Return if not a WP World
				if (const UWorld* World = WorldColumn.World.Get(); !World || !World->GetWorldPartition())
				{
					return;
				}

				if (const ILevelInstanceInterface* LevelInstance = Cast<ILevelInstanceInterface>(Object.Object))
				{
					Context.AddColumn(Row, FWorldPartitionSubPackageColumn{ .SubPackage = LevelInstance->GetWorldAssetPackage() });
				}
			})
		.Where()
			.All<FTypedElementActorTag, FTypedElementSyncFromWorldTag, FTypedElementLevelColumn>()
			.None<FWorldPartitionSubPackageColumn>()
		.Compile()
	);
}

void UActorWorldPartitionSubPackageDataStorageFactory::RegisterActorWorldPartitionSubPackageToColumnQuery(UE::Editor::DataStorage::ICoreProvider& DataStorage) const
{
	using namespace UE::Editor::DataStorage::Queries;

	DataStorage.RegisterQuery(
		Select(
			TEXT("Actor World Partition Sub Package Object to Column"),
			FProcessor(EQueryTickPhase::PrePhysics, DataStorage.GetQueryTickGroupName(EQueryTickGroups::SyncExternalToDataStorage)).SetExecutionMode(EExecutionMode::GameThread),
			[](IQueryContext&, RowHandle, const FTypedElementUObjectColumn& Object, FWorldPartitionSubPackageColumn& SubPackageColumn)
			{
				if (const ILevelInstanceInterface* LevelInstance = Cast<ILevelInstanceInterface>(Object.Object))
				{
					SubPackageColumn.SubPackage = LevelInstance->GetWorldAssetPackage();
				}
			})
		.Where()
			.All<FTypedElementActorTag, FTypedElementSyncFromWorldTag>()
		.Compile()
	);
}
