// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/Interfaces/TypedElementDataStorageFactory.h"
#include "UObject/ObjectMacros.h"

#include "TedsActorWorldPartitionSubPackageQueries.generated.h"

/**
 * Registers queries that keep FWorldPartitionSubPackageColumn in sync with the actor's
 * world asset package name. Read-only — no write-back to world.
 * Only added to Level Instance actors (those implementing ILevelInstanceInterface).
 */
UCLASS()
class UActorWorldPartitionSubPackageDataStorageFactory : public UEditorDataStorageFactory
{
	GENERATED_BODY()

public:
	~UActorWorldPartitionSubPackageDataStorageFactory() override = default;

	void RegisterQueries(UE::Editor::DataStorage::ICoreProvider& DataStorage) override;

private:
	/**
	 * Adds FWorldPartitionSubPackageColumn to actor rows for Level Instance actors
	 * that do not already have the column.
	 */
	void RegisterActorAddWorldPartitionSubPackageColumn(UE::Editor::DataStorage::ICoreProvider& DataStorage) const;

	/**
	 * Syncs the actor's world asset package name into FWorldPartitionSubPackageColumn when it may have changed.
	 */
	void RegisterActorWorldPartitionSubPackageToColumnQuery(UE::Editor::DataStorage::ICoreProvider& DataStorage) const;
};
