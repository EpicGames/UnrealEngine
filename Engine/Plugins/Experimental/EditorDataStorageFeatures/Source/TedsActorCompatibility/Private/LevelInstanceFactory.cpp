// Copyright Epic Games, Inc. All Rights Reserved.


#include "LevelInstanceFactory.h"

#include "ActorFolders/ActorFolderColumns.h"
#include "ActorDesc/TedsActorDescColumns.h"
#include "Columns/TedsLevelInstanceColumns.h"
#include "DataStorage/Features.h"
#include "DataStorage/Handles.h"
#include "Elements/Columns/TypedElementCompatibilityColumns.h"
#include "Elements/Columns/TypedElementFolderColumns.h"
#include "Elements/Columns/TypedElementMiscColumns.h"
#include "Elements/Columns/TypedElementUIColumns.h"
#include "Elements/Framework/TypedElementQueryBuilder.h"
#include "Engine/World.h"
#include "GameFramework/Actor.h"
#include "LevelInstance/LevelInstanceInterface.h"
#include "LevelInstance/LevelInstanceSubsystem.h"
#include "Modules/ModuleManager.h"
#include "WorldPartition/ActorDescContainerInstance.h"
#include "WorldPartition/WorldPartitionActorDescInstance.h"

void UTedsLevelInstanceFactory::PreRegister(UE::Editor::DataStorage::ICoreProvider& InDataStorage)
{
	if (ILevelInstanceEditorModule* EditorModule = FModuleManager::GetModulePtr<ILevelInstanceEditorModule>("LevelInstanceEditor"))
	{
		EditorModule->OnLevelInstanceEditModeChanged().AddUObject(this, &UTedsLevelInstanceFactory::LevelInstanceEditModeChanged);
	}
	DataStorage = &InDataStorage;

	using namespace UE::Editor::DataStorage;
	CompatibilityProvider = GetMutableDataStorageFeature<ICompatibilityProvider>(CompatibilityFeatureName);
}

void UTedsLevelInstanceFactory::PreShutdown(UE::Editor::DataStorage::ICoreProvider& InDataStorage)
{
	if (ILevelInstanceEditorModule* EditorModule = FModuleManager::GetModulePtr<ILevelInstanceEditorModule>("LevelInstanceEditor"))
	{
		EditorModule->OnLevelInstanceEditModeChanged().RemoveAll(this);
	}
}

void UTedsLevelInstanceFactory::LevelInstanceEditModeChanged(ILevelInstanceInterface* LevelInstanceInterface,
	ILevelInstanceEditorModule::ELevelInstanceEditMode LevelInstanceEditMode) const
{
	if (AActor* LevelInstanceActor = Cast<AActor>(LevelInstanceInterface))
	{
		using namespace UE::Editor::DataStorage;

		if (CompatibilityProvider)
		{
			if (const RowHandle Row = CompatibilityProvider->FindRowWithCompatibleObject(LevelInstanceActor); Row != InvalidRowHandle)
			{
				if (LevelInstanceEditMode == ILevelInstanceEditorModule::ELevelInstanceEditMode::None)
				{
					DataStorage->RemoveColumn<FLevelInstanceEditingColumn>(Row);
				}
				else
				{
					DataStorage->AddColumn<FLevelInstanceEditingColumn>(Row, { .EditMode = LevelInstanceEditMode });
				}
			}
		}
	}
}

void UTedsLevelInstanceFactory::RegisterQueries(UE::Editor::DataStorage::ICoreProvider& InDataStorage)
{
	using namespace UE::Editor::DataStorage;
	using namespace UE::Editor::DataStorage::Queries;

	InDataStorage.RegisterQuery(
		Select(
			TEXT("Sync level instance tags on actor rows"),
			FProcessor(EQueryTickPhase::PrePhysics, InDataStorage.GetQueryTickGroupName(EQueryTickGroups::SyncExternalToDataStorage))
				.SetExecutionMode(EExecutionMode::GameThread),
			[](IQueryContext& Context, RowHandle Row, const FTypedElementUObjectColumn& ObjectColumn)
			{
				const AActor* Actor = Cast<AActor>(ObjectColumn.Object.Get());
				if (!Actor)
				{
					return;
				}

				if (Cast<ILevelInstanceInterface>(Actor))
				{
					Context.AddColumns<FLevelInstanceTag>(Row);
				}
				else
				{
					Context.RemoveColumns<FLevelInstanceTag>(Row);
				}

				if (Actor->IsInLevelInstance())
				{
					Context.AddColumns<FInLevelInstanceTag>(Row);
				}
				else
				{
					Context.RemoveColumns<FInLevelInstanceTag>(Row);
				}
			})
		.Where()
			.All<FTypedElementActorTag, FTypedElementSyncFromWorldTag>()
		.Compile());

	InDataStorage.RegisterQuery(
		Select(
			TEXT("Sync level instance tags on actor desc rows"),
			FProcessor(EQueryTickPhase::PrePhysics, InDataStorage.GetQueryTickGroupName(EQueryTickGroups::SyncExternalToDataStorage))
				.SetExecutionMode(EExecutionMode::GameThread),
			[](IQueryContext& Context, RowHandle Row,
				const FWorldPartitionHandleColumn& HandleColumn,
				const FTypedElementWorldColumn& WorldColumn)
			{
				const FWorldPartitionActorDescInstance* ActorDescInstance = *HandleColumn.Handle;
				if (!ActorDescInstance)
				{
					Context.RemoveColumns<FLevelInstanceTag>(Row);
					Context.RemoveColumns<FInLevelInstanceTag>(Row);
					return;
				}

				if (const UClass* NativeClass = ActorDescInstance->GetActorNativeClass();
					NativeClass && NativeClass->ImplementsInterface(ULevelInstanceInterface::StaticClass()))
				{
					Context.AddColumns<FLevelInstanceTag>(Row);
				}
				else
				{
					Context.RemoveColumns<FLevelInstanceTag>(Row);
				}

				bool bInsideLevelInstance = false;
				if (UWorld* World = WorldColumn.World.Get())
				{
					if (const ULevelInstanceSubsystem* LevelInstanceSubsystem = UWorld::GetSubsystem<ULevelInstanceSubsystem>(World))
					{
						if (const UActorDescContainerInstance* ContainerInstance = ActorDescInstance->GetContainerInstance())
						{
							if (const UWorld* OuterWorld = ContainerInstance->GetTypedOuter<UWorld>())
							{
								bInsideLevelInstance = LevelInstanceSubsystem->GetOwningLevelInstance(OuterWorld->PersistentLevel) != nullptr;
							}
						}
					}
				}

				if (bInsideLevelInstance)
				{
					Context.AddColumns<FInLevelInstanceTag>(Row);
				}
				else
				{
					Context.RemoveColumns<FInLevelInstanceTag>(Row);
				}
			})
		.Where()
			.All<FActorDescTag, FTypedElementSyncFromWorldTag>()
		.Compile());

	InDataStorage.RegisterQuery(
		Select(
			TEXT("Sync level instance tag on folder rows"),
			FProcessor(EQueryTickPhase::DuringPhysics, InDataStorage.GetQueryTickGroupName(EQueryTickGroups::SyncExternalToDataStorage))
				.SetExecutionMode(EExecutionMode::GameThread),
			[](IQueryContext& Context, RowHandle Row, const FFolderCompatibilityColumn& FolderColumn)
			{
				if (Cast<ILevelInstanceInterface>(FolderColumn.Folder.GetRootObjectPtr()))
				{
					Context.AddColumns<FInLevelInstanceTag>(Row);
				}
				else
				{
					Context.RemoveColumns<FInLevelInstanceTag>(Row);
				}
			})
		.Where()
			.All<FFolderTag, FTypedElementSyncFromWorldTag>()
		.Compile());
		
	InDataStorage.RegisterQuery(
        Select(
        	TEXT("Hide actor desc rows inside non-editing level instances"),
        	FProcessor(EQueryTickPhase::DuringPhysics, InDataStorage.GetQueryTickGroupName(EQueryTickGroups::SyncExternalToDataStorage))
        		.SetExecutionMode(EExecutionMode::GameThread),
        	[](IQueryContext& Context, RowHandle Row, const FWorldPartitionHandleColumn& HandleColumn,
        		const FTypedElementWorldColumn& WorldColumn)
        	{
        		if (const UWorld* World = WorldColumn.World.Get(); World && HandleColumn.Handle.IsValid())
        		{
        			if (const FWorldPartitionActorDescInstance* ActorDescInstance = *HandleColumn.Handle)
        			{
        				const ULevelInstanceSubsystem* LevelInstanceSubsystem = UWorld::GetSubsystem<ULevelInstanceSubsystem>(World);
        				if (!LevelInstanceSubsystem)
        				{
        					return;
        				}

        				if (const UActorDescContainerInstance* ContainerInstance = ActorDescInstance->GetContainerInstance())
        				{
							if (const UWorld* OuterWorld = ContainerInstance->GetTypedOuter<UWorld>(); OuterWorld && OuterWorld->PersistentLevel)
							{
								if (Context.HasColumn<FInLevelInstanceTag>() || ActorDescInstance->IsLoaded()
								|| Cast<UActorDescContainerInstance>(ContainerInstance->GetOuter()) != nullptr)
								{
									Context.AddColumns<FHideRowFromUITag>(Row);
								}
								else
								{
									Context.RemoveColumns<FHideRowFromUITag>(Row);
								}
							}
        				}
        			}
        		}
        	})
        .Where()
        	.All<FActorDescTag, FTypedElementSyncFromWorldTag>()
        .Compile()
        );
}
