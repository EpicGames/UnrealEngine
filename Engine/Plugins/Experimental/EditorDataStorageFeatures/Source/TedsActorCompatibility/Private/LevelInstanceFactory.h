// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Elements/Interfaces/TypedElementDataStorageFactory.h"
#include "LevelInstance/ILevelInstanceEditorModule.h"
#include "Elements/Interfaces/TypedElementDataStorageCompatibilityInterface.h"
#include "LevelInstanceFactory.generated.h"

class IEditorDataStorageProvider;


UCLASS()
class UTedsLevelInstanceFactory : public UEditorDataStorageFactory
{
	GENERATED_BODY()
public:
	virtual ~UTedsLevelInstanceFactory() override = default;
	virtual void PreRegister(UE::Editor::DataStorage::ICoreProvider& InDataStorage) override;
	virtual void PreShutdown(UE::Editor::DataStorage::ICoreProvider& DataStorage) override;
	virtual void RegisterQueries(UE::Editor::DataStorage::ICoreProvider& DataStorage) override;

private:
	void LevelInstanceEditModeChanged(ILevelInstanceInterface* LevelInstanceInterface, ILevelInstanceEditorModule::ELevelInstanceEditMode LevelInstanceEditMode) const;

	UE::Editor::DataStorage::ICoreProvider* DataStorage;
	UE::Editor::DataStorage::ICompatibilityProvider* CompatibilityProvider;
};
