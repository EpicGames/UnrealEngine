// Copyright Epic Games, Inc. All Rights Reserved.

#include "TedsActorCompatibilityFactory.h"

#include "Engine/Engine.h"
#include "Engine/World.h"
#include "Elements/Columns/TypedElementCompatibilityColumns.h"
#include "Elements/Columns/TypedElementLabelColumns.h"
#include "Columns/TedsActorMobilityColumns.h"
#include "DataStorage/Features.h"
#include "GameFramework/Actor.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TedsActorCompatibilityFactory)

#define LOCTEXT_NAMESPACE "TedsActorFactory"

void UTedsActorCompatibilityFactory::PreRegister(UE::Editor::DataStorage::ICoreProvider& DataStorage)
{
	using namespace UE::Editor::DataStorage;

	DataStorageCompatibility = GetMutableDataStorageFeature<ICompatibilityProvider>(CompatibilityFeatureName);
	check(DataStorageCompatibility);

	// Register the Actor tables in this stage since it is a standard archetype and may be depended upon 
	// before this factory has had the chance to register its' table.
	RegisterTable(DataStorage);

	PostWorldInitializationDelegateHandle = FWorldDelegates::OnPostWorldInitialization.AddUObject(this, &UTedsActorCompatibilityFactory::OnPostWorldInitialization);
	PreWorldFinishDestroyDelegateHandle = FWorldDelegates::OnPreWorldFinishDestroy.AddUObject(this, &UTedsActorCompatibilityFactory::OnPreWorldFinishDestroy);

	ActorOuterChangedDelegateHandle = GEngine->OnLevelActorOuterChanged().AddUObject(this, &UTedsActorCompatibilityFactory::OnActorOuterChanged);
}

void UTedsActorCompatibilityFactory::RegisterTable(UE::Editor::DataStorage::ICoreProvider& DataStorage)
{
	using namespace UE::Editor::DataStorage;

	TableHandle UObjectTable = DataStorage.FindTable("Editor_StandardUObjectTable");
	checkf(DataStorage.IsTableValid(UObjectTable), TEXT("Actor factor is unable to retrieve the UObject table, "
		"which is a required template for the actor table."))

	StandardActorTable = DataStorage.RegisterTable<
		FTypedElementLabelColumn, FTypedElementLabelHashColumn, FTedsActorMobilityColumn, FTypedElementActorTag, FLevelColumn, FGuidColumn>(
		FName("Editor_StandardActorTable"), FTableRegistrationOptions{ .SourceTable = UObjectTable });

	DataStorageCompatibility->RegisterTypeTableAssociation(AActor::StaticClass(), StandardActorTable);
}

void UTedsActorCompatibilityFactory::PreShutdown(UE::Editor::DataStorage::ICoreProvider& DataStorage)
{
	using namespace UE::Editor::DataStorage;

	for (TPair<UWorld*, FDelegateHandle>& It : ActorDestroyedDelegateHandles)
	{
		It.Key->RemoveOnActorDestroyedHandler(It.Value);
	}

	GEngine->OnLevelActorOuterChanged().RemoveAll(this);

	FWorldDelegates::OnPreWorldFinishDestroy.Remove(PreWorldFinishDestroyDelegateHandle);
	FWorldDelegates::OnPostWorldInitialization.Remove(PostWorldInitializationDelegateHandle);
}

void UTedsActorCompatibilityFactory::OnPostWorldInitialization(UWorld* World, const UWorld::InitializationValues InitializationValues)
{
	using namespace UE::Editor::DataStorage;

	FDelegateHandle Handle = World->AddOnActorDestroyedHandler(
		FOnActorDestroyed::FDelegate::CreateUObject(this, &UTedsActorCompatibilityFactory::OnActorDestroyed));
	ActorDestroyedDelegateHandles.Add(World, Handle);
}

void UTedsActorCompatibilityFactory::OnPreWorldFinishDestroy(UWorld* World)
{
	using namespace UE::Editor::DataStorage;

	FDelegateHandle Handle;
	if (ActorDestroyedDelegateHandles.RemoveAndCopyValue(World, Handle))
	{
		World->RemoveOnActorDestroyedHandler(Handle);
	}
}

void UTedsActorCompatibilityFactory::OnActorDestroyed(AActor* Actor)
{
	// The only function called is already thread safe.
	if (DataStorageCompatibility)
	{
		DataStorageCompatibility->RemoveCompatibleObjectExplicit(Actor);
	}
}

void UTedsActorCompatibilityFactory::OnActorOuterChanged(AActor* Actor, UObject* Outer)
{
	// We only want to register actors outer'd to a level in TEDS - so if the outer changes we add/remove the object based on that status
	if (DataStorageCompatibility)
	{
		if (Actor->GetLevel())
		{
			DataStorageCompatibility->AddCompatibleObjectExplicit(Actor);
		}
		else
		{
			DataStorageCompatibility->RemoveCompatibleObjectExplicit(Actor);
		}
	}
}

#undef LOCTEXT_NAMESPACE
