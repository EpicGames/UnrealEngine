// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/Interfaces/TypedElementDataStorageCompatibilityInterface.h"
#include "Elements/Interfaces/TypedElementDataStorageInterface.h"
#include "Elements/Interfaces/TypedElementDataStorageFactory.h"
#include "Engine/World.h"

#include "TedsActorCompatibilityFactory.generated.h"

UCLASS()
class UTedsActorCompatibilityFactory : public UEditorDataStorageFactory
{
	GENERATED_BODY()

public:
	virtual void PreRegister(UE::Editor::DataStorage::ICoreProvider& DataStorage) override;
	virtual void PreShutdown(UE::Editor::DataStorage::ICoreProvider& DataStorage) override;

private:
	void RegisterTable(UE::Editor::DataStorage::ICoreProvider& DataStorage);

	void OnPostWorldInitialization(UWorld* World, const UWorld::InitializationValues InitializationValues);
	void OnPreWorldFinishDestroy(UWorld* World);
	void OnActorDestroyed(AActor* Actor);
	void OnActorOuterChanged(AActor* Actor, UObject* Outer);

	UE::Editor::DataStorage::ICompatibilityProvider* DataStorageCompatibility{nullptr};

	UE::Editor::DataStorage::TableHandle StandardActorTable{ UE::Editor::DataStorage::InvalidTableHandle };

	TMap<UWorld*, FDelegateHandle> ActorDestroyedDelegateHandles;
	FDelegateHandle PostWorldInitializationDelegateHandle;
	FDelegateHandle PreWorldFinishDestroyDelegateHandle;
	FDelegateHandle ActorOuterChangedDelegateHandle;
};