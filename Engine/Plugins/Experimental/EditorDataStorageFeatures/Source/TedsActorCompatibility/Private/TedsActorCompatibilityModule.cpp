// Copyright Epic Games, Inc. All Rights Reserved.

#include "TedsActorCompatibilityModule.h"

#include "ActorComponentDebugHierarchyWidget/ActorComponentTree.h"
#include "Columns/TedsActorComponentCompatibilityColumns.h"
#include "GameFramework/Actor.h"
#include "Components/ActorComponent.h"
#include "Elements/Columns/TypedElementLabelColumns.h"
#include "DataStorage/Features.h"
#include "Elements/Columns/TypedElementCompatibilityColumns.h"
#include "Elements/Columns/TypedElementMiscColumns.h"
#include "Elements/Columns/TypedElementTypeInfoColumns.h"
#include "Elements/Framework/TypedElementQueryBuilder.h"
#include "Elements/Interfaces/TypedElementDataStorageCompatibilityInterface.h"
#include "Elements/Interfaces/TypedElementDataStorageInterface.h"
#include "HAL/IConsoleManager.h"
#include "Modules/ModuleManager.h"

namespace UE::Editor::DataStorage
{
	static bool bEnableDemoTab = false;
	FAutoConsoleVariableRef CVarAddDemoTab(
		TEXT("TEDS.Debug.ActorCompatibility.ActorComponents.EnableHierarchyViewer"),
		bEnableDemoTab,
		TEXT("Enables a debug tab to demonstrate usage of hierarchy views with ActorComponents"),
		ECVF_ReadOnly);
	
	void FTedsActorCompatibilityModule::StartupModule()
	{
		IModuleInterface::StartupModule();

		FModuleManager::Get().LoadModule(TEXT("TypedElementFramework"));
		FModuleManager::Get().LoadModule(TEXT("EditorDataStorageHierarchy"));

		if (bEnableDemoTab)
		{
			UE::Editor::DataStorage::ActorCompatibility::RegisterActionComponentDebugHierarchyWidget();
		}
	}

	void FTedsActorCompatibilityModule::ShutdownModule()
	{
		IModuleInterface::ShutdownModule();
	}
} // namespace UE::Editor::DataStorage

IMPLEMENT_MODULE(UE::Editor::DataStorage::FTedsActorCompatibilityModule, TedsActorCompatibility);