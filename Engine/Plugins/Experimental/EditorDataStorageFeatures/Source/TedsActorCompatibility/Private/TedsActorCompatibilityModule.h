// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"

namespace UE::Editor::DataStorage
{
	class FTedsActorCompatibilityModule
		: public IModuleInterface
	{
	public:

		FTedsActorCompatibilityModule() = default;

		// IModuleInterface interface
		virtual void StartupModule() override;
		virtual void ShutdownModule() override;
	};
} // namespace UE::Editor::DataStorage
