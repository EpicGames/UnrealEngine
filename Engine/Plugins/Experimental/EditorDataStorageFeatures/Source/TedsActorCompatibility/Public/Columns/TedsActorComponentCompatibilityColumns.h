// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "DataStorage/CommonTypes.h"

#include "TedsActorComponentCompatibilityColumns.generated.h"

USTRUCT()
struct FActorComponentTypeTag : public FEditorDataStorageTag
{
	GENERATED_BODY()
};