// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/Interfaces/TypedElementDataStorageInterface.h"
#include "Engine/EngineTypes.h"

#include "TedsActorMobilityColumns.generated.h"

/**
 * Column that stores the mobility of an actor's scene component.
 */
USTRUCT(meta = (DisplayName = "Mobility"))
struct FTedsActorMobilityColumn final : public FEditorDataStorageColumn
{
	GENERATED_BODY()

	UPROPERTY(meta = (Sortable))
	TEnumAsByte<EComponentMobility::Type> Mobility = EComponentMobility::Movable;
};