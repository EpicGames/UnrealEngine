// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/Interfaces/TypedElementDataStorageInterface.h"

#include "TedsActorSocketColumns.generated.h"

/**
 * Column that stores the name of the socket the actor is attached to.
 */
USTRUCT(meta = (DisplayName = "Socket"))
struct FTedsActorSocketColumn final : public FEditorDataStorageColumn
{
	GENERATED_BODY()

	UPROPERTY(meta = (Sortable, Searchable))
	FName AttachedSocket = NAME_None;
};