// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/Interfaces/TypedElementDataStorageInterface.h"

#include "TedsActorUncachedLightsColumns.generated.h"

/**
 * Tag identifying actor rows that should be evaluated for uncached static lighting interactions in a widget.
 * Added when an actor has Static or Stationary mobility, removed when Movable.
 */
USTRUCT(meta = (DisplayName = "Uncached Lights"))
struct FTedsActorUncachedLightsTag final : public FEditorDataStorageTag
{
	GENERATED_BODY()
};
