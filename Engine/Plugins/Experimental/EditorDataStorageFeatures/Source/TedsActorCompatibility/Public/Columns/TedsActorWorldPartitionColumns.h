// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/Interfaces/TypedElementDataStorageInterface.h"

#include "TedsActorWorldPartitionColumns.generated.h"

/**
 * Column that stores the World Partition data layer names for an actor.
 * Only added to rows for actors in worlds with World Partition enabled.
 */
USTRUCT(meta = (DisplayName = "World Partition Data Layer"))
struct FWorldPartitionDataLayerColumn final : public FEditorDataStorageColumn
{
	GENERATED_BODY()

	UPROPERTY(meta = (Sortable, Searchable))
	FString DataLayers;
};

/**
 * Column that stores the World Partition content bundle display name for an actor.
 * Only added to rows for actors in worlds with content bundles enabled.
 */
USTRUCT(meta = (DisplayName = "World Partition Content Bundle"))
struct FWorldPartitionContentBundleColumn final : public FEditorDataStorageColumn
{
	GENERATED_BODY()

	UPROPERTY(meta = (Sortable, Searchable))
	FString ContentBundle;
};

/**
 * Column that stores the World Asset Package name for Level Instance actors.
 * Only added to rows for actors that implement ILevelInstanceInterface.
 */
USTRUCT(meta = (DisplayName = "World Partition Sub Package"))
struct FWorldPartitionSubPackageColumn final : public FEditorDataStorageColumn
{
	GENERATED_BODY()

	UPROPERTY(meta = (Sortable, Searchable))
	FString SubPackage;
};

