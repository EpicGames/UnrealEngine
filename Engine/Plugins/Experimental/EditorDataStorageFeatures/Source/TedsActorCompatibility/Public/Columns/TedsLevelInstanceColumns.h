// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "DataStorage/CommonTypes.h"
#include "LevelInstance/ILevelInstanceEditorModule.h"

#include "TedsLevelInstanceColumns.generated.h"

/**
 * Tag to identify a row with an LevelInstance actor that is being edited.
 */
USTRUCT(meta = (DisplayName = "Editing Level Instance"))
struct FLevelInstanceEditingColumn final : public FEditorDataStorageColumn
{
	GENERATED_BODY()

	ILevelInstanceEditorModule::ELevelInstanceEditMode EditMode;
};

/**
 * Tag set on a row whose underlying object is an ILevelInstanceInterface
 */
USTRUCT(meta = (DisplayName = "Level Instance"))
struct FLevelInstanceTag final : public FEditorDataStorageTag
{
	GENERATED_BODY()
};

/**
 * Tag set on a row that lives inside a level instance. 
 */
USTRUCT(meta = (DisplayName = "In Level Instance"))
struct FInLevelInstanceTag final : public FEditorDataStorageTag
{
	GENERATED_BODY()
};
