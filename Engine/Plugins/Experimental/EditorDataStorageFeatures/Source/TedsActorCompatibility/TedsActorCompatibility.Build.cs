// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class TedsActorCompatibility : ModuleRules
{
	public TedsActorCompatibility(ReadOnlyTargetRules Target) : base(Target)
	{
		// Enable truncation warnings in this module
		CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Warning;

		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		if (Target.bBuildEditor)
		{
			PublicIncludePaths.AddRange(new string[] { });
			PrivateIncludePaths.AddRange(new string[] { });

			PublicDependencyModuleNames.AddRange(new string[] { });
			PrivateDependencyModuleNames.AddRange(
				new string[] 
				{
					"Core",
					"CoreUObject",
					"Engine",
					"TypedElementFramework",
					"UnrealEd",
					"TedsQueryStack",
					"WorkspaceMenuStructure",
					"TedsTableViewer",
					"SlateCore",
					"Slate",
					"LevelInstanceEditor",
					"TedsEditorCompatibility"
				});

			DynamicallyLoadedModuleNames.AddRange(new string[] { });
		}
	}
}