// Copyright Epic Games, Inc. All Rights Reserved.

#include "TedsAlertWidget.h"

#include "Columns/SlateHeaderColumns.h"
#include "DataStorage/Features.h"
#include "Elements/Columns/TypedElementMiscColumns.h"
#include "Elements/Columns/TypedElementSlateWidgetColumns.h"
#include "TedsAlertColumns.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/SOverlay.h"
#include "Widgets/Text/STextBlock.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TedsAlertWidget)

#define LOCTEXT_NAMESPACE "TedsAlertWidget"

namespace UE::Editor::DataStorage
{
	using namespace UE::Editor::DataStorage::Columns;

	FText FAlertWidgetSorter::GetShortName() const
	{
		return LOCTEXT("AlertWidgetSorter", "Alert Sorter");
	}

	FPrefixInfo FAlertWidgetSorter::CalculatePrefix(const FAlertColumn& Column, uint32 ByteIndex) const
	{
		return CreateSortPrefix(ByteIndex, -static_cast<int32>(Column.AlertType), Column.Priority);
	}
}

namespace UE::Editor::DataStorage::Widgets::Private
{
	void UpdateWidget(const TSharedPtr<SWidget>& Widget, const FText& Alert, bool bIsWarning, uint16 ErrorCount, uint16 WarningCount,
		RowHandle RowWithAlertAction)
	{
		using namespace UE::Editor::DataStorage::Columns;

		if (Widget)
		{
			uint32 ChildCount = ErrorCount + WarningCount;
			if (FChildren* Children = Widget->GetChildren())
			{
				SImage& Background =
					static_cast<SImage&>(*Children->GetSlotAt(FAlertWidgetConstructor::IconBackgroundSlot).GetWidget());
				SImage& Badge =
					static_cast<SImage&>(*Children->GetSlotAt(FAlertWidgetConstructor::IconBadgeSlot).GetWidget());
				STextBlock& CounterText =
					static_cast<STextBlock&>(*Children->GetSlotAt(FAlertWidgetConstructor::CounterTextSlot).GetWidget());
				SButton& ActionButton =
					static_cast<SButton&>(*Children->GetSlotAt(FAlertWidgetConstructor::ActionButtonSlot).GetWidget());

				// Setup the background image
				if (ChildCount > 0)
				{
					if (!Alert.IsEmpty())
					{
						Background.SetImage(bIsWarning
							? FAppStyle::GetBrush("Icons.Warning.Background")
							: FAppStyle::GetBrush("Icons.Error.Background"));
					}
					else
					{
						Background.SetImage(FAppStyle::GetBrush("Icons.Alert.Background"));
					}
				}
				else
				{
					if (!Alert.IsEmpty())
					{
						Background.SetImage(bIsWarning
							? FAppStyle::GetBrush("Icons.Warning.Solid")
							: FAppStyle::GetBrush("Icons.Error.Solid"));
					}
					else
					{
						Background.SetImage(FAppStyle::GetBrush("Icons.Alert.Solid"));
					}
				}

				// Set counter if needed, otherwise turn it off.
				if (ChildCount == 0)
				{
					// If there are no children, don't show the badge and don't show a counter.
					Badge.SetVisibility(EVisibility::Hidden);
					CounterText.SetVisibility(EVisibility::Hidden);
				}
				else
				{
					// If there are children, also take into account if there's an alert as well.
					uint32 TotalChildCount = ChildCount + (Alert.IsEmpty() ? 0 : 1);
					if (TotalChildCount <= 9)
					{
						Badge.SetVisibility(EVisibility::HitTestInvisible);
						CounterText.SetVisibility(EVisibility::HitTestInvisible);
						CounterText.SetText(FText::AsNumber(TotalChildCount));
						CounterText.SetFont(FCoreStyle::GetDefaultFontStyle("Regular", FAlertWidgetConstructor::BadgeFontSize));
						CounterText.SetMargin(FMargin(
							FAlertWidgetConstructor::BadgeHorizontalOffset,
							FAlertWidgetConstructor::BadgeVerticalOffset));
					}
					else
					{
						Badge.SetVisibility(EVisibility::HitTestInvisible);
						CounterText.SetVisibility(EVisibility::HitTestInvisible);
						CounterText.SetText(FText::FromString(TEXT("*")));
						CounterText.SetFont(FCoreStyle::GetDefaultFontStyle("Regular", 14));
						CounterText.SetMargin(FMargin(
							FAlertWidgetConstructor::BadgeHorizontalOffset - 2.0f,
							FAlertWidgetConstructor::BadgeVerticalOffset - 6.5f));
					}
				}

				// Setup the tool tip text
				if (!Alert.IsEmpty() && ChildCount > 0)
				{
					FText ToolTipText = FText::Format(LOCTEXT("ChildAlertCountWithMessage", "Errors: {0}\nWarnings: {1}\n\n{2}"),
						FText::AsNumber(ErrorCount + (bIsWarning ? 0 : 1)),
						FText::AsNumber(WarningCount + (bIsWarning ? 1 : 0)), Alert);
					Background.SetToolTipText(ToolTipText);
					ActionButton.SetToolTipText(MoveTemp(ToolTipText));
				}
				else if (!Alert.IsEmpty())
				{
					Background.SetToolTipText(Alert);
					ActionButton.SetToolTipText(Alert);
				}
				else if (ChildCount > 0)
				{
					FText ToolTipText = FText::Format(LOCTEXT("ChildAlertCount", "Errors: {0}\nWarnings: {1}"),
						FText::AsNumber(ErrorCount), FText::AsNumber(WarningCount));
					Background.SetToolTipText(ToolTipText);
					ActionButton.SetToolTipText(MoveTemp(ToolTipText));
				}

				// If there's an action to call, enable the invisible button, otherwise turn it off.
				if (RowWithAlertAction != InvalidRowHandle)
				{
					Background.SetVisibility(EVisibility::HitTestInvisible);
					ActionButton.SetVisibility(EVisibility::Visible);
					ActionButton.SetOnClicked(FOnClicked::CreateLambda([RowWithAlertAction]()
						{
							const ICoreProvider* DataStorage = GetDataStorageFeature<ICoreProvider>(StorageFeatureName);
							if (const FAlertActionColumn* Action =
								DataStorage->GetColumn<FAlertActionColumn>(RowWithAlertAction))
							{
								Action->Action(RowWithAlertAction);
							}
							return FReply::Handled();
						}));
				}
				else
				{
					Background.SetVisibility(EVisibility::Visible);
					ActionButton.SetVisibility(EVisibility::Hidden);
				}
			}
		}
	}
}

//
// UAlertWidgetFactory
//

void UAlertWidgetFactory::RegisterWidgetConstructors(
	UE::Editor::DataStorage::ICoreProvider& DataStorage,
	UE::Editor::DataStorage::IUiProvider& DataStorageUi) const
{
	using namespace UE::Editor::DataStorage::Columns;
	using namespace UE::Editor::DataStorage::Queries;

	DataStorageUi.RegisterWidgetFactory<FAlertWidgetConstructor>(DataStorageUi.FindPurpose(DataStorageUi.GetGeneralWidgetPurposeID()),
		TColumn<FAlertColumn>() || TColumn<FChildAlertColumn>());

	DataStorageUi.RegisterWidgetFactory<FAlertHeaderWidgetConstructor>(
		DataStorageUi.FindPurpose(IUiProvider::FPurposeInfo("General", "Header", NAME_None).GeneratePurposeID()),
		TColumn<FAlertColumn>() || TColumn<FChildAlertColumn>());
}

void UAlertWidgetFactory::RegisterQueries(UE::Editor::DataStorage::ICoreProvider& DataStorage)
{
	RegisterAlertQueries(DataStorage);
	RegisterAlertHeaderQueries(DataStorage);
}

void UAlertWidgetFactory::RegisterAlertQueries(UE::Editor::DataStorage::ICoreProvider& DataStorage)
{
	using namespace UE::Editor::DataStorage::Columns;
	using namespace UE::Editor::DataStorage::Queries;

	QueryHandle UpdateWidget_OnlyAlert = DataStorage.RegisterQuery(
		Select()
		.ReadOnly<FAlertColumn>()
		.Where()
			.Any<FTypedElementSyncFromWorldTag, FTypedElementSyncBackToWorldTag>()
			.None<FChildAlertColumn>()
		.Compile());

	QueryHandle UpdateWidget_OnlyChildAlert = DataStorage.RegisterQuery(
		Select()
		.ReadOnly<FChildAlertColumn>()
		.Where()
			.Any<FTypedElementSyncFromWorldTag, FTypedElementSyncBackToWorldTag>()
			.None<FAlertColumn>()
		.Compile());

	QueryHandle UpdateWidget_Both = DataStorage.RegisterQuery(
		Select()
		.ReadOnly<FAlertColumn, FChildAlertColumn>()
		.Where()
			.Any<FTypedElementSyncFromWorldTag, FTypedElementSyncBackToWorldTag>()
		.Compile());

	DataStorage.RegisterQuery(
		Select(TEXT("Update alert widget state"),
			FProcessor(EQueryTickPhase::FrameEnd, DataStorage.GetQueryTickGroupName(EQueryTickGroups::SyncWidgets))
				.SetExecutionMode(EExecutionMode::GameThread),
			[](IQueryContext& Context,
				FTypedElementSlateWidgetReferenceColumn& Widget,
				const FTypedElementRowReferenceColumn& ReferenceColumn)
			{
				Context.RunSubquery(0, ReferenceColumn.Row, CreateSubqueryCallbackBinding(
					[&Widget](ISubqueryContext& Context, RowHandle Row, const FAlertColumn& Alert)
					{
						checkf(
							Alert.AlertType == FAlertColumnType::Warning ||
							Alert.AlertType == FAlertColumnType::Error,
							TEXT("Alert column has unsupported type %i"), static_cast<int>(Alert.AlertType));
						Widgets::Private::UpdateWidget(Widget.Widget.Pin(), Alert.Message,
							Alert.AlertType == FAlertColumnType::Warning, 0, 0,
							Context.HasColumn<FAlertActionColumn>() ? Row : InvalidRowHandle);
					}));
				Context.RunSubquery(1, ReferenceColumn.Row, CreateSubqueryCallbackBinding(
					[&Widget](ISubqueryContext& Context, RowHandle Row, const FChildAlertColumn& ChildAlert)
					{
						Widgets::Private::UpdateWidget(Widget.Widget.Pin(), FText::GetEmpty(), false,
							ChildAlert.Counts[static_cast<size_t>(FAlertColumnType::Error)],
							ChildAlert.Counts[static_cast<size_t>(FAlertColumnType::Warning)],
							Context.HasColumn<FAlertActionColumn>() ? Row : InvalidRowHandle);
					}));
				Context.RunSubquery(2, ReferenceColumn.Row, CreateSubqueryCallbackBinding(
					[&Widget](
						ISubqueryContext& Context, RowHandle Row,
						const FAlertColumn& Alert,
						const FChildAlertColumn& ChildAlert)
					{
						checkf(
							Alert.AlertType == FAlertColumnType::Warning ||
							Alert.AlertType == FAlertColumnType::Error,
							TEXT("Alert column has unsupported type %i"), static_cast<int>(Alert.AlertType));
						Widgets::Private::UpdateWidget(Widget.Widget.Pin(), Alert.Message,
							Alert.AlertType == FAlertColumnType::Warning,
							ChildAlert.Counts[static_cast<size_t>(FAlertColumnType::Error)],
							ChildAlert.Counts[static_cast<size_t>(FAlertColumnType::Warning)],
							Context.HasColumn<FAlertActionColumn>() ? Row : InvalidRowHandle);
					}));
			}
		)
		.Where()
			.All<FAlertWidgetTag>()
		.DependsOn()
			.SubQuery(UpdateWidget_OnlyAlert)
			.SubQuery(UpdateWidget_OnlyChildAlert)
			.SubQuery(UpdateWidget_Both)
		.Compile());
}

void UAlertWidgetFactory::RegisterAlertHeaderQueries(UE::Editor::DataStorage::ICoreProvider& DataStorage)
{
	using namespace UE::Editor::DataStorage::Columns;
	using namespace UE::Editor::DataStorage::Queries;

	QueryHandle AlertCount = DataStorage.RegisterQuery(
		Count()
		.Where()
			.Any<FAlertColumn>()
		.Compile());

	DataStorage.RegisterQuery(
		Select(TEXT("Update alert header"),
			FProcessor(EQueryTickPhase::FrameEnd, DataStorage.GetQueryTickGroupName(EQueryTickGroups::SyncWidgets))
				.SetExecutionMode(EExecutionMode::GameThread),
			[](IQueryContext& Context, RowHandle Row, FTypedElementSlateWidgetReferenceColumn& Widget)
			{
				FQueryResult Result = Context.RunSubquery(0);
				if (Result.Count > 0)
				{
					if (TSharedPtr<SWidget> WidgetPtr = Widget.Widget.Pin())
					{
						static_cast<SImage*>(WidgetPtr.Get())->SetImage(FAppStyle::GetBrush("Icons.Warning.Solid"));
						Context.AddColumns<FAlertHeaderActiveWidgetTag>(Row);
					}
				}
			})
		.Where()
			.All<FAlertHeaderWidgetTag>()
			.None<FAlertHeaderActiveWidgetTag>()
		.DependsOn()
			.SubQuery(AlertCount)
		.Compile());

	DataStorage.RegisterQuery(
		Select(TEXT("Update active alert header"),
			FProcessor(EQueryTickPhase::FrameEnd, DataStorage.GetQueryTickGroupName(EQueryTickGroups::SyncWidgets))
			.SetExecutionMode(EExecutionMode::GameThread),
			[](IQueryContext& Context, RowHandle Row, FTypedElementSlateWidgetReferenceColumn& Widget)
			{
				FQueryResult Result = Context.RunSubquery(0);
				if (Result.Count == 0)
				{
					if (TSharedPtr<SWidget> WidgetPtr = Widget.Widget.Pin())
					{
						static_cast<SImage*>(WidgetPtr.Get())->SetImage(FAppStyle::GetBrush("Icons.Alert"));
						Context.RemoveColumns<FAlertHeaderActiveWidgetTag>(Row);
					}
				}
			}
		)
		.Where()
			.All<FAlertHeaderWidgetTag, FAlertHeaderActiveWidgetTag>()
		.DependsOn()
			.SubQuery(AlertCount)
		.Compile());
}



//
// FAlertWidgetConstructor
//

FAlertWidgetConstructor::FAlertWidgetConstructor()
	: Super(StaticStruct())
{
}

TSharedPtr<SWidget> FAlertWidgetConstructor::CreateWidget(
	const UE::Editor::DataStorage::FMetaDataView& Arguments)
{
	return SNew(SOverlay)
		+SOverlay::Slot()
			.VAlign(VAlign_Fill)
			.HAlign(HAlign_Fill)
		[
			SNew(SButton)
				.ButtonStyle(FAppStyle::Get(), "FlatButton")
				.Text(FText::FromString(TEXT("X"))) // There needs to be at least some content otherwise nothing will show.
				.ForegroundColor(FLinearColor::Transparent) // Then the color needs to be cleared so the X doesn't show.
				.ContentPadding(FMargin(0.0f))
		]
		+SOverlay::Slot()
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Center)
		[
			SNew(SImage)
				.Image(FAppStyle::GetBrush("Icons.Alert.Solid"))
				.DesiredSizeOverride(FVector2D(16.0f, 16.0f))
		]
		+SOverlay::Slot()
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Center)
		[
			SNew(SImage)
				.Image(FAppStyle::GetBrush("Icons.Alert.Badge"))
				.DesiredSizeOverride(FVector2D(16.0f, 16.0f))
		]
		+SOverlay::Slot()
			.VAlign(VAlign_Bottom)
			.HAlign(HAlign_Center)
		[
			SNew(STextBlock)
				.Font(FCoreStyle::GetDefaultFontStyle("Regular", BadgeFontSize))
				.ColorAndOpacity(FLinearColor::Black)
				.Margin(FMargin(BadgeHorizontalOffset, BadgeVerticalOffset))
		];
}

TConstArrayView<const UScriptStruct*> FAlertWidgetConstructor::GetAdditionalColumnsList() const
{
	using namespace UE::Editor::DataStorage;
	static TTypedElementColumnTypeList<FAlertWidgetTag> Columns;
	return Columns;
}

bool FAlertWidgetConstructor::FinalizeWidget(UE::Editor::DataStorage::ICoreProvider* DataStorage,
	UE::Editor::DataStorage::IUiProvider* DataStorageUi,
	UE::Editor::DataStorage::RowHandle Row, const TSharedPtr<SWidget>& Widget)
{
	using namespace UE::Editor::DataStorage;
	using namespace UE::Editor::DataStorage::Columns;

	const FTypedElementRowReferenceColumn* RowReferenceColumn = DataStorage->GetColumn<FTypedElementRowReferenceColumn>(Row);

	if (!ensureMsgf(RowReferenceColumn, TEXT("The alert widget requires your widget to have a target row")))
	{
		return false;
	}

	RowHandle TargetRow = RowReferenceColumn->Row;
	const FAlertColumn* Alert = DataStorage->GetColumn<FAlertColumn>(TargetRow);
	const FChildAlertColumn* ChildAlert = DataStorage->GetColumn<FChildAlertColumn>(TargetRow);

	uint16 ErrorCount = ChildAlert ? ChildAlert->Counts[static_cast<size_t>(FAlertColumnType::Error)] : 0;
	uint16 WarningCount = ChildAlert ? ChildAlert->Counts[static_cast<size_t>(FAlertColumnType::Warning)] : 0;

	Widgets::Private::UpdateWidget(
		Widget,
		Alert ? Alert->Message : FText::GetEmpty(),
		Alert ? (Alert->AlertType == FAlertColumnType::Warning) : false,
		ErrorCount, WarningCount,
		DataStorage->HasColumns<FAlertActionColumn>(TargetRow) ? TargetRow : InvalidRowHandle);

	return true;
}

TArray<TSharedPtr<const UE::Editor::DataStorage::FColumnSorterInterface>> FAlertWidgetConstructor::ConstructColumnSorters(
	UE::Editor::DataStorage::ICoreProvider* DataStorage,
	UE::Editor::DataStorage::IUiProvider* DataStorageUi,
	const UE::Editor::DataStorage::FMetaDataView& Arguments)
{
	using namespace UE::Editor::DataStorage;
	return TArray<TSharedPtr<const FColumnSorterInterface>>(
		{
			MakeShared<FAlertWidgetSorter>()
		});
}



//
// FAlertHeaderWidgetConstructor
//

FAlertHeaderWidgetConstructor::FAlertHeaderWidgetConstructor()
	: Super(StaticStruct())
{
}

TSharedPtr<SWidget> FAlertHeaderWidgetConstructor::CreateWidget(
	const UE::Editor::DataStorage::FMetaDataView& Arguments)
{
	return SNew(SImage)
		.DesiredSizeOverride(FVector2D(16.f, 16.f))
		.ColorAndOpacity(FSlateColor::UseForeground())
		.Image(FAppStyle::GetBrush("Icons.Alert"))
		.ToolTipText(FText(LOCTEXT("AlertColumnHeader", "Alerts")));
}

TConstArrayView<const UScriptStruct*> FAlertHeaderWidgetConstructor::GetAdditionalColumnsList() const
{
	using namespace UE::Editor::DataStorage;
	static TTypedElementColumnTypeList<FAlertHeaderWidgetTag> Columns;
	return Columns;
}

bool FAlertHeaderWidgetConstructor::FinalizeWidget(UE::Editor::DataStorage::ICoreProvider* DataStorage,
	UE::Editor::DataStorage::IUiProvider* DataStorageUi,
	UE::Editor::DataStorage::RowHandle Row, const TSharedPtr<SWidget>& Widget)
{
	DataStorage->AddColumn(Row, FHeaderWidgetSizeColumn
		{
			.ColumnSizeMode = EColumnSizeMode::Fixed,
			.Width = 24.0f
		});
	return true;
}

#undef LOCTEXT_NAMESPACE
