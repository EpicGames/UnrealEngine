// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataStorage/CommonTypes.h"
#include "Elements/Interfaces/TypedElementDataStorageFactory.h"
#include "Elements/Interfaces/TypedElementDataStorageUiInterface.h"
#include "TedsAlertColumns.h"

#include "TedsAlertWidget.generated.h"

namespace UE::Editor::DataStorage
{
	using namespace UE::Editor::DataStorage::Columns;

	class FAlertWidgetSorter final : public TColumnSorterInterface<FColumnSorterInterface::ESortType::FixedSize64, FAlertColumn>
	{
	public:
		virtual FText GetShortName() const override;
	protected:
		virtual FPrefixInfo CalculatePrefix(const FAlertColumn& Column, uint32 ByteIndex) const override;
	};
}

UCLASS()
class UAlertWidgetFactory final : public UEditorDataStorageFactory
{
	GENERATED_BODY()

public:
	~UAlertWidgetFactory() override = default;

	void RegisterWidgetConstructors(UE::Editor::DataStorage::ICoreProvider& DataStorage,
		UE::Editor::DataStorage::IUiProvider& DataStorageUi) const override;
	void RegisterQueries(UE::Editor::DataStorage::ICoreProvider& DataStorage) override;

private:
	void RegisterAlertQueries(UE::Editor::DataStorage::ICoreProvider& DataStorage);
	void RegisterAlertHeaderQueries(UE::Editor::DataStorage::ICoreProvider& DataStorage);
};

USTRUCT()
struct FAlertWidgetConstructor final : public FTypedElementWidgetConstructor
{
	GENERATED_BODY()

public:
	static constexpr int32 IconBackgroundSlot = 1;
	static constexpr int32 IconBadgeSlot = 2;
	static constexpr int32 CounterTextSlot = 3;
	static constexpr int32 ActionButtonSlot = 0;

	static constexpr float BadgeFontSize = 7.0f;
	static constexpr float BadgeHorizontalOffset = 13.0f;
	static constexpr float BadgeVerticalOffset = 1.0f;

	FAlertWidgetConstructor();
	~FAlertWidgetConstructor() override = default;

private:
	TSharedPtr<SWidget> CreateWidget(const UE::Editor::DataStorage::FMetaDataView& Arguments) override;
	TConstArrayView<const UScriptStruct*> GetAdditionalColumnsList() const override;
	bool FinalizeWidget(UE::Editor::DataStorage::ICoreProvider* DataStorage,
		UE::Editor::DataStorage::IUiProvider* DataStorageUi,
		UE::Editor::DataStorage::RowHandle Row, const TSharedPtr<SWidget>& Widget) override;
	TArray<TSharedPtr<const UE::Editor::DataStorage::FColumnSorterInterface>> ConstructColumnSorters(
		UE::Editor::DataStorage::ICoreProvider* DataStorage,
		UE::Editor::DataStorage::IUiProvider* DataStorageUi,
		const UE::Editor::DataStorage::FMetaDataView& Arguments) override;
};

USTRUCT(meta = (DisplayName = "General purpose alert"))
struct FAlertWidgetTag : public FEditorDataStorageTag
{
	GENERATED_BODY()
};

USTRUCT()
struct FAlertHeaderWidgetConstructor final : public FTypedElementWidgetConstructor
{
	GENERATED_BODY()

public:
	FAlertHeaderWidgetConstructor();
	~FAlertHeaderWidgetConstructor() override = default;

protected:
	TSharedPtr<SWidget> CreateWidget(const UE::Editor::DataStorage::FMetaDataView& Arguments) override;
	TConstArrayView<const UScriptStruct*> GetAdditionalColumnsList() const override;
	bool FinalizeWidget(UE::Editor::DataStorage::ICoreProvider* DataStorage,
		UE::Editor::DataStorage::IUiProvider* DataStorageUi,
		UE::Editor::DataStorage::RowHandle Row, const TSharedPtr<SWidget>& Widget) override;
};

USTRUCT(meta = (DisplayName = "General purpose alert header"))
struct FAlertHeaderWidgetTag : public FEditorDataStorageTag
{
	GENERATED_BODY()
};

USTRUCT(meta = (DisplayName = "Alert header active"))
struct FAlertHeaderActiveWidgetTag : public FEditorDataStorageTag
{
	GENERATED_BODY()
};
