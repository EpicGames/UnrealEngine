// Copyright Epic Games, Inc. All Rights Reserved.

#include "TedsAlerts.h"

#include "DataStorage/Features.h"
#include "Elements/Columns/TypedElementMiscColumns.h"
#include "Elements/Framework/TypedElementQueryBuilder.h"
#include "Elements/Interfaces/TypedElementDataStorageInterface.h"
#include "HAL/IConsoleManager.h"
#include "TedsAlertsFactory.h"

namespace UE::Editor::DataStorage::Alerts
{
	static TAutoConsoleVariable<bool> CVarEnableTedsAlerts(
		TEXT("Teds.Feature.Alerts"),
		true,
		TEXT("Enable adding alerts to rows in TEDS"),
		ECVF_Default
	);

	void AddAlert(ICoreProvider& DataStorage, RowHandle TargetRow, const FName& Name, FText Message, 
		Columns::FAlertColumnType Type, uint8 Priority, Columns::FAlertActionCallback Action, const FName& HierarchyName)
	{
		using namespace UE::Editor::DataStorage::Columns;

		if (!CVarEnableTedsAlerts.GetValueOnGameThread())
		{
			return;
		}

		// Add the new alert to the unsorted alert chain table and let TEDS sort is out at an appropriate time.
		if (UTedsAlertsFactory* AlertsFactory = DataStorage.FindFactory<UTedsAlertsFactory>())
		{
			// To prevent duplicates, if there is already an alert with this name on the row remove and re-add it, which acts like an overwrite.
			RemoveAlert(DataStorage, TargetRow, Name);
			
			// Add a placeholder alert to be filled in later.
			if (!DataStorage.HasColumns<FAlertColumn>(TargetRow))
			{
				DataStorage.AddColumn<FAlertColumn>(TargetRow);
			}

			FHierarchyHandle HierarchyHandle = DataStorage.FindHierarchyByName(HierarchyName);

			if (DataStorage.IsValidHierarchyHandle(HierarchyHandle))
			{
				checkf(DataStorage.GetParentChangedColumnType(HierarchyHandle),
						TEXT("Alerts with hierarchies require a parent changed column to properly detect changes. This can be specified on hierarchy registration."));
				
				AlertsFactory->RegisterAlertHierarchy(DataStorage, HierarchyName);
			}

			DataStorage.AddRow(AlertsFactory->GetUnsortedAlertChainTable(),
				[&DataStorage, TargetRow, Name, Message = MoveTemp(Message), Type, Priority, Action = MoveTemp(Action), HierarchyHandle]
				(RowHandle Row) mutable
				{
					DataStorage.AddColumn(Row, FAlertColumn
						{
							.Message = MoveTemp(Message),
							.NextAlert = TargetRow,
							.Name = Name,
							.AlertType = Type,
							.Priority = Priority
						});
					
					DataStorage.AddColumn(Row, FAlertHierarchyColumn{.HierarchyHandle =  HierarchyHandle});

					if (Action)
					{
						DataStorage.AddColumn(Row, FAlertActionColumn
							{
								.Action = MoveTemp(Action)
							});
					}
				});
		}
	}

	void AddAlert(IQueryContext& Context, RowHandle TargetRow, const FName& Name, FText Message,
		Columns::FAlertColumnType Type, uint8 Priority, Columns::FAlertActionCallback Action, const FName& HierarchyName)
	{
		using namespace UE::Editor::DataStorage;
		
		Context.PushCommand([TargetRow, Name, Message = MoveTemp(Message), Type, Priority, Action = MoveTemp(Action), HierarchyName]() mutable
			{
				if (ICoreProvider* DataStorage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName))
				{
					AddAlert(*DataStorage, TargetRow, Name, MoveTemp(Message), Type, Priority, MoveTemp(Action), HierarchyName);
				}
			});
	}

	void UpdateAlertText(ICoreProvider& DataStorage, RowHandle TargetRow, const FName& Name, FText Message)
	{
		using namespace UE::Editor::DataStorage::Columns;

		FAlertColumn* Alert = DataStorage.GetColumn<FAlertColumn>(TargetRow);
		while (Alert)
		{
			if (Alert->Name == Name)
			{
				Alert->Message = MoveTemp(Message);
				DataStorage.AddColumns<FTypedElementSyncBackToWorldTag>(TargetRow);
				return;
			}
			TargetRow = Alert->NextAlert;
			Alert = DataStorage.GetColumn<FAlertColumn>(TargetRow);
		}
	}

	void UpdateAlertText(IQueryContext& Context, RowHandle TargetRow, const FName& Name, FText Message)
	{
		Context.PushCommand([TargetRow, Name, Message = MoveTemp(Message)]() mutable
			{
				if (ICoreProvider* DataStorage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName))
				{
					UpdateAlertText(*DataStorage, TargetRow, Name, MoveTemp(Message));
				}
			});
	}

	void UpdateAlertAction(ICoreProvider& DataStorage, RowHandle TargetRow, const FName& Name, 
		Columns::FAlertActionCallback Action)
	{
		using namespace UE::Editor::DataStorage::Columns;

		FAlertColumn* Alert = DataStorage.GetColumn<FAlertColumn>(TargetRow);
		while (Alert)
		{
			if (Alert->Name == Name)
			{
				if (Action)
				{
					if (FAlertActionColumn* StoredAction = DataStorage.GetColumn<FAlertActionColumn>(TargetRow))
					{
						StoredAction->Action = MoveTemp(Action);
					}
					else
					{
						DataStorage.AddColumn(TargetRow, FAlertActionColumn{ .Action = MoveTemp(Action) });
					}
				}
				else
				{
					DataStorage.RemoveColumn<FAlertActionColumn>(TargetRow);
				}
				DataStorage.AddColumns<FTypedElementSyncBackToWorldTag>(TargetRow);
				return;
			}
			TargetRow = Alert->NextAlert;
			Alert = DataStorage.GetColumn<FAlertColumn>(TargetRow);
		}
	}

	void UpdateAlertAction(IQueryContext& Context, RowHandle TargetRow, const FName& Name, Columns::FAlertActionCallback Action)
	{
		Context.PushCommand([TargetRow, Name, Action = MoveTemp(Action)]() mutable
			{
				if (ICoreProvider* DataStorage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName))
				{
					UpdateAlertAction(*DataStorage, TargetRow, Name, MoveTemp(Action));
				}
			});
	}

	void RemoveAlert(ICoreProvider& DataStorage, RowHandle TargetRow, const FName& Name)
	{
		using namespace UE::Editor::DataStorage::Columns;
		using namespace UE::Editor::DataStorage::Queries;
		
		FAlertColumn* PreviousAlert = DataStorage.GetColumn<FAlertColumn>(TargetRow);
		FAlertColumn* Alert = nullptr;
		
		// First check the active alert.
		if (PreviousAlert)
		{
			// This is the currently active alert so replace it with the next one if it exists.
			if (PreviousAlert->Name == Name)
			{
				Alert = DataStorage.GetColumn<FAlertColumn>(PreviousAlert->NextAlert);
				if (Alert)
				{
					// There's another message in the queue so use that message. Remove the copied alert from the alerts table.
					RowHandle NextAlert = PreviousAlert->NextAlert;
					*PreviousAlert = MoveTemp(*Alert);

					FAlertActionColumn* Action = DataStorage.GetColumn<FAlertActionColumn>(PreviousAlert->NextAlert);
					if (Action)
					{
						if (FAlertActionColumn* PreviousAction = DataStorage.GetColumn<FAlertActionColumn>(TargetRow))
						{
							*PreviousAction = MoveTemp(*Action);
						}
						else
						{
							DataStorage.AddColumn(TargetRow, MoveTemp(*Action));
						}

					}
					else
					{
						DataStorage.RemoveColumn<FAlertActionColumn>(TargetRow);
					}

					DataStorage.RemoveRow(NextAlert);
				}
				else
				{
					// There are no more messages so delete the current alert.
					DataStorage.RemoveColumn<FAlertColumn>(TargetRow);
					DataStorage.RemoveColumn<FAlertActionColumn>(TargetRow);
				}
				DataStorage.AddColumns<FTypedElementSyncBackToWorldTag>(TargetRow);
				return;
			}

			// The alert is in the alerts table so walk the chain and remove it when found, updating the links as they're found.
			Alert = DataStorage.GetColumn<FAlertColumn>(PreviousAlert->NextAlert);
			while (Alert)
			{
				if (Alert->Name == Name)
				{
					RowHandle CurrentRow = PreviousAlert->NextAlert;
					PreviousAlert->NextAlert = Alert->NextAlert;
					DataStorage.RemoveRow(CurrentRow);
					return;
				}
				PreviousAlert = Alert;
				Alert = DataStorage.GetColumn<FAlertColumn>(PreviousAlert->NextAlert);
			}
		}

		// The alert might be queued up for processing so search the unsorted alerts table and remove it from there.
		if (const UTedsAlertsFactory* AlertsFactory = DataStorage.FindFactory<UTedsAlertsFactory>())
		{
			TArray<RowHandle> RemoveRows;
			DataStorage.RunQuery(AlertsFactory->GetUnsortedAlertChainTable(), CreateDirectQueryCallbackBinding(
				[&Name, &RemoveRows](RowHandle Row, const FAlertColumn& Alert)
				{
					if (Alert.Name == Name)
					{
						RemoveRows.Add(Row);
					}
				}));
			for (RowHandle Row : RemoveRows)
			{
				DataStorage.RemoveRow(Row);
			}
		}
	}

	void RemoveAlert(IQueryContext& Context, RowHandle TargetRow, const FName& Name)
	{
		using namespace UE::Editor::DataStorage;

		Context.PushCommand([TargetRow, Name]() mutable
			{
				if (ICoreProvider* DataStorage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName))
				{
					RemoveAlert(*DataStorage, TargetRow, Name);
				}
			});
	}
} //namespace UE::Editor::DataStorage::Alerts
