// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataStorage/Features.h"
#include "Elements/Columns/TypedElementHiearchyColumns.h"
#include "Elements/Columns/TypedElementMiscColumns.h"
#include "Elements/Columns/TypedElementSelectionColumns.h"
#include "Elements/Framework/TypedElementQueryBuilder.h"
#include "Elements/Interfaces/TypedElementDataStorageInterface.h"
#include "HAL/IConsoleManager.h"
#include "TedsAlerts.h"
#include "TedsAlertColumns.h"

FAutoConsoleCommand AddRandomAlertToRowConsoleCommand(
	TEXT("TEDS.Debug.AddRandomAlertToSelectedRows"),
	TEXT("Add random alert to all selected rows."),
	FConsoleCommandDelegate::CreateLambda(
		[]()
		{
			using namespace UE::Editor::DataStorage::Columns;
			using namespace UE::Editor::DataStorage::Queries;

			TRACE_CPUPROFILER_EVENT_SCOPE(TEDS.Debug.AddRandomAlertToSelectedRows);

			if (ICoreProvider* DataStorage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName))
			{
				static uint32 Count = 0;
				static FName TestMessageName = FName("Test message");
				static QueryHandle Query = [DataStorage]
					{
						return DataStorage->RegisterQuery(
							Select()
							.Where()
							.All<FTypedElementSelectionColumn>()
							.Compile());
					}();

				TArray<RowHandle> Rows;
				DataStorage->RunQuery(Query, CreateDirectQueryCallbackBinding(
					[&Rows](IDirectQueryContext& Context, RowHandle Row)
					{
						Rows.Add(Row);
					}));
				for (RowHandle Row : Rows)
				{
					uint8 Priority = static_cast<uint8>(FMath::RandRange(0, 255));
					int32 Config = FMath::RandRange(0, 4);
					FString Message = FString::Printf(TEXT("Test alert %u has priority %u"), Count++, Priority);
					FAlertColumnType Type = (Config & 0x1) ? FAlertColumnType::Error : FAlertColumnType::Warning;
					TFunction<void(RowHandle)> Action;
					if ((Config >> 1) & 0x1)
					{
						Action = [Message](RowHandle)
							{
								FPlatformMisc::MessageBoxExt(
									EAppMsgType::Ok,
									*FString::Printf(TEXT("Example of an alert action for message: \n`%s`"), *Message),
									TEXT("TEDS.Debug.AddRandomAlertToSelectedRows"));
							};
					}
					Alerts::AddAlert(*DataStorage, Row, TestMessageName, FText::FromString(MoveTemp(Message)), Type, Priority, MoveTemp(Action));
				}
			}
		}
	));

FAutoConsoleCommand RemoveSelectedAlertsConsoleCommand(
	TEXT("TEDS.Debug.RemoveSelectedAlerts"),
	TEXT("Removes the alert from all selected rows. Any queued alerts will replace the removed alert."),
	FConsoleCommandDelegate::CreateLambda(
		[]()
		{
			using namespace UE::Editor::DataStorage::Columns;
			using namespace UE::Editor::DataStorage::Queries;

			TRACE_CPUPROFILER_EVENT_SCOPE(TEDS.Debug.ClearSelectedAlerts);
			static FName TestMessageName = FName("Test message");

			if (ICoreProvider* DataStorage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName))
			{
				static QueryHandle AlertQuery = [DataStorage]
					{
						return DataStorage->RegisterQuery(
							Select()
							.Where()
							.All<FAlertColumn, FTypedElementSelectionColumn>()
							.Compile());
					}();
				TArray<RowHandle> Rows;
				DataStorage->RunQuery(AlertQuery, CreateDirectQueryCallbackBinding(
					[&Rows](IDirectQueryContext& Context, RowHandle Row)
					{
						Rows.Add(Row);
					}));
				for (RowHandle Row : Rows)
				{
					Alerts::RemoveAlert(*DataStorage, Row, TestMessageName);
				}
			}
		}
	));

FAutoConsoleCommand ClearAllAlertsConsoleCommand(
	TEXT("TEDS.Debug.ClearAllAlertInfo"),
	TEXT("Removes all alerts and child alerts."),
	FConsoleCommandDelegate::CreateLambda(
		[]()
		{
			using namespace UE::Editor::DataStorage::Columns;
			using namespace UE::Editor::DataStorage::Queries;

			TRACE_CPUPROFILER_EVENT_SCOPE(TEDS.Debug.ClearAllAlertInfo);

			if (ICoreProvider* DataStorage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName))
			{
				// Remove all alerts on existing rows.
				{
					static QueryHandle AlertInfoQuery = [DataStorage]
						{
							return DataStorage->RegisterQuery(
								Select()
								.Where()
								.Any<FAlertColumn, FChildAlertColumn>()
								.Compile());
						}();
					TArray<RowHandle> Rows;
					DataStorage->RunQuery(AlertInfoQuery, CreateDirectQueryCallbackBinding(
						[&Rows](IDirectQueryContext& Context, RowHandle Row)
						{
							Rows.Add(Row);
						}));
					for (RowHandle Row : Rows)
					{
						DataStorage->RemoveColumn<FAlertColumn>(Row);
						DataStorage->RemoveColumn<FChildAlertColumn>(Row);
						DataStorage->RemoveColumn<FAlertActionColumn>(Row);
						DataStorage->AddColumn<FTypedElementSyncBackToWorldTag>(Row);
					}
				}

				// Remove all rows with alert chains.
				{
					static QueryHandle AlertChainQuery = [DataStorage]
						{
							return DataStorage->RegisterQuery(
								Select()
								.Where()
								.Any<FAlertChainTag>()
								.Compile());
						}();
					TArray<RowHandle> Rows;
					DataStorage->RunQuery(AlertChainQuery, CreateDirectQueryCallbackBinding(
						[&Rows](IDirectQueryContext& Context, RowHandle Row)
						{
							Rows.Add(Row);
						}));
					for (RowHandle Row : Rows)
					{
						DataStorage->RemoveRow(Row);
					}
				}
			}
		}
	));
