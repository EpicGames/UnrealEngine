// Copyright Epic Games, Inc. All Rights Reserved.

#include "TedsAlertsFactory.h"

#include "Elements/Columns/TypedElementMiscColumns.h"
#include "Elements/Framework/TypedElementQueryBuilder.h"
#include "Elements/Interfaces/TypedElementDataStorageInterface.h"
#include "TedsAlerts.h"
#include "Templates/UnrealTemplate.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TedsAlertsFactory)

const FName UTedsAlertsFactory::AlertChainTableName = "Alerts chain";
const FName UTedsAlertsFactory::UnsortedAlertChainTableName = "Alerts chain (unsorted)";

void UTedsAlertsFactory::RegisterTables(UE::Editor::DataStorage::ICoreProvider& DataStorage)
{
	using namespace UE::Editor::DataStorage;

	ChainTable = DataStorage.RegisterTable<FTedsAlertColumn, FTedsAlertChainTag>(AlertChainTableName);
	UnsortedChainTable = DataStorage.RegisterTable<FTedsUnsortedAlertChainTag>(UnsortedAlertChainTableName, 
		FTableRegistrationOptions{.SourceTable = ChainTable});
}

void UTedsAlertsFactory::RegisterQueries(UE::Editor::DataStorage::ICoreProvider& DataStorage)
{
	RegisterSubQueries(DataStorage);
	RegisterSortUnsortedAlertsQuery(DataStorage);
	RegisterOnRemoveQueries(DataStorage);
}

UE::Editor::DataStorage::TableHandle UTedsAlertsFactory::GetAlertChainTable() const
{
	return ChainTable;
}

UE::Editor::DataStorage::TableHandle UTedsAlertsFactory::GetUnsortedAlertChainTable() const
{
	return UnsortedChainTable;
}

UE::Editor::DataStorage::QueryHandle UTedsAlertsFactory::GetSortedAlertsQuery() const
{
	return SortedAlertsQuery;
}

UE::Editor::DataStorage::QueryHandle UTedsAlertsFactory::GetUnsortedAlertsQuery() const
{
	return UnsortedAlertsQuery;
}

void UTedsAlertsFactory::RegisterAlertHierarchy(UE::Editor::DataStorage::ICoreProvider& DataStorage, const FName& InHierarchyName)
{
	using namespace UE::Editor::DataStorage;
	FHierarchyHandle HierarchyHandle = DataStorage.FindHierarchyByName(InHierarchyName);

	if (!DataStorage.IsValidHierarchyHandle(HierarchyHandle) || HierarchyConditions.Contains(InHierarchyName))
	{
		return;
	}

	TUniquePtr<FAlertHierarchyConditions> Conditions = MakeUnique<FAlertHierarchyConditions>();
	Conditions->AlertPropagationStartConditionName = FName(FString::Printf(TEXT("Alerts_%s"), *InHierarchyName.ToString()));
	Conditions->AlertPropagationConditionName = FName(FString::Printf(TEXT("AlertsPropagation_%s"), *InHierarchyName.ToString()));
	Conditions->AlertPropagationFinishedConditionName = FName(FString::Printf(TEXT("AlertsPropagationFinished_%s"), *InHierarchyName.ToString()));
	AllAlertPropagationStartConditionNames.Add(Conditions->AlertPropagationStartConditionName);

	RegisterOnAddHierarchyQueries(DataStorage, HierarchyHandle, Conditions->AlertPropagationStartConditionName);
	RegisterOnRemoveHierarchyQueries(DataStorage, HierarchyHandle, Conditions->AlertPropagationStartConditionName);
	RegisterParentUpdatesHierarchyQueries(DataStorage, HierarchyHandle, Conditions->AlertPropagationStartConditionName);
	RegisterChildAlertHierarchyUpdatesQueries(DataStorage, InHierarchyName, HierarchyHandle, Conditions.Get());

	HierarchyConditions.Add(InHierarchyName, MoveTemp(Conditions));
}

void UTedsAlertsFactory::RegisterSubQueries(UE::Editor::DataStorage::ICoreProvider& DataStorage)
{
	using namespace UE::Editor::DataStorage::Queries;

	SortedAlertsQuery = DataStorage.RegisterQuery(
		Select()
			.ReadWrite<FTedsAlertColumn>()
		.Where()
			.None<FTedsUnsortedAlertChainTag>()
		.Compile());

	UnsortedAlertsQuery = DataStorage.RegisterQuery(
		Select()
			.ReadWrite<FTedsAlertColumn>()
		.Where()
			.All<FTedsUnsortedAlertChainTag>()
		.Compile());
	
	AlertActionQuery = DataStorage.RegisterQuery(
		Select()
			.ReadWrite<FTedsAlertActionColumn>()
		.Compile());

	AlertHierarchyQuery = DataStorage.RegisterQuery(
		Select()
			.ReadWrite<FTedsAlertHierarchyColumn>()
		.Compile());

	ChildAlertColumnReadWriteQuery = DataStorage.RegisterQuery(
		Select()
			.ReadWrite<FTedsChildAlertColumn>()
		.Compile());
}

void UTedsAlertsFactory::RegisterSortUnsortedAlertsQuery(UE::Editor::DataStorage::ICoreProvider& DataStorage)
{
	using namespace UE::Editor::DataStorage::Queries;

	constexpr int32 AlertActionQueryIndex = 1;
	constexpr int32 AlertHierarchyQueryIndex = 2;

	DataStorage.RegisterQuery(
		Select("Sort unsorted alerts", FProcessor(EQueryTickPhase::PrePhysics, DataStorage.GetQueryTickGroupName(EQueryTickGroups::Update)),
			[AllConditions = &AllAlertPropagationStartConditionNames](IQueryContext& NewAlertContext, RowHandle NewAlertRow, FTedsAlertColumn& NewAlert)
			{
				RowHandle NextRow = NewAlert.NextAlert;
				while (NewAlertContext.IsRowAssigned(NextRow))
				{
					// Walk the chain and add link at the appropriate spot. This can include the row with the active alert.
					NewAlertContext.RunSubquery(0, NextRow, CreateSubqueryCallbackBinding([NewAlertRow, &NewAlert, &NewAlertContext, &NextRow, AllConditions]
						(ISubqueryContext& Context, RowHandle TargetRow, FTedsAlertColumn& TargetAlert)
						{
							if (NewAlert.AlertType > TargetAlert.AlertType ||
								(NewAlert.AlertType == TargetAlert.AlertType && NewAlert.Priority >= TargetAlert.Priority))
							{
								// if the row is not the active alert, then simply link up.
								// Note that "NextAlert" at this point still contains the row that has the active alert.
								if (NewAlert.NextAlert == TargetRow)
								{
									if (TargetAlert.Message.IsEmpty())
									{
										// This is a placeholder so override it.
										AssignAlert(Context, TargetRow, TargetAlert, NewAlertContext, NewAlertRow, NewAlert, AlertActionQueryIndex, AlertHierarchyQueryIndex);
									}
									else
									{
										// The active alert needs to be replaced so swap the active alert with the new alert.
										SwapAlerts(Context, TargetRow, TargetAlert, NewAlertContext, NewAlertRow, NewAlert, AlertActionQueryIndex, AlertHierarchyQueryIndex);
									}
									// (Re)calculate child alerts for all registered hierarchies.
									for (const FName& ConditionName : *AllConditions)
									{
										Context.ActivateQueries(ConditionName);
									}
								}
								else
								{
									// Found a spot in the chain. To avoid having to track the previous entry, just swap the
									// alert at that spot with the new one and chain them up.
									SwapAlerts(Context, TargetRow, TargetAlert, NewAlertContext, NewAlertRow, NewAlert, AlertActionQueryIndex, AlertHierarchyQueryIndex);
								}
								NextRow = InvalidRowHandle;
							}
							else
							{
								if (TargetAlert.NextAlert == InvalidRowHandle)
								{
									// End of the chain so append at the end.
									AppendAlert(TargetAlert, NewAlertContext, NewAlertRow, NewAlert);
									NextRow = InvalidRowHandle;
								}
								else
								{
									// Go to the next row in the chain to check that one.
									NextRow = TargetAlert.NextAlert;
								}
							}
						}));
				}
			})
		.Where()
			.Any<FTedsUnsortedAlertChainTag>()
		.DependsOn()
			.SubQuery(SortedAlertsQuery)
			.SubQuery(AlertActionQuery)
			.SubQuery(AlertHierarchyQuery)
		.Compile());
}

void UTedsAlertsFactory::RegisterOnRemoveQueries(UE::Editor::DataStorage::ICoreProvider& DataStorage)
{
	using namespace UE::Editor::DataStorage::Queries;


	DataStorage.RegisterQuery(
		Select("Remove active alert", FObserver::OnRemove<FTedsAlertColumn>(),
			[AllConditions = &AllAlertPropagationStartConditionNames](IQueryContext& Context, RowHandle Row, const FTedsAlertColumn& Alert)
			{
				// Delete all entries in the alert chain.
				RowHandle NextRow = Alert.NextAlert;
				while (Context.RunSubquery(0, NextRow, CreateSubqueryCallbackBinding(
					[&NextRow](ISubqueryContext& Context, RowHandle Row, FTedsAlertColumn& Alert)
					{
						NextRow = Alert.NextAlert;
						Context.RemoveRow(Row);
					})).Count > 0);


				// Remove any alerts that haven't been processed yet.
				// Also check NextAlert == Row to avoid removing unsorted entries that share the same
				// name but target a different row (e.g. when multiple rows use the same alert name).
				Context.RunSubquery(1, CreateSubqueryCallbackBinding(
					[&Alert, Row](ISubqueryContext& Context, RowHandle PendingAlertRow, FTedsAlertColumn& PendingAlert)
					{
						if (Alert.Name == PendingAlert.Name && PendingAlert.NextAlert == Row)
						{
							Context.RemoveRow(PendingAlertRow);
						}
					}));

				// FTedsChildAlertColumn is added to leaf alert rows by "Add missing child alerts";
				// clean it up here alongside the other associated columns.
				Context.RemoveColumns<FTedsAlertHierarchyColumn, FTedsAlertActionColumn, FTedsChildAlertColumn>(Row);

				// Update any alert parents in all registered hierarchies.
				for (const FName& ConditionName : *AllConditions)
				{
					Context.ActivateQueries(ConditionName);
				}
			})
		.Where()
			.None<FTedsAlertChainTag>()
		.DependsOn()
			.SubQuery(SortedAlertsQuery)
			.SubQuery(UnsortedAlertsQuery)
		.Compile());
}



void UTedsAlertsFactory::RegisterOnAddHierarchyQueries(UE::Editor::DataStorage::ICoreProvider& DataStorage, UE::Editor::DataStorage::FHierarchyHandle InHierarchyHandle, const FName& InAlertPropagationStartConditionName)
{
	using namespace UE::Editor::DataStorage::Queries;

	DataStorage.RegisterQuery(
		Select("Register alert with parent on alert hierarchy add", FObserver::OnAdd<FTedsAlertHierarchyColumn>(),
			[InHierarchyHandle, InAlertPropagationStartConditionName](IQueryContext& Context, RowHandle Row, const FTedsAlertHierarchyColumn& AlertHierarchyColumn)
			{
				if (AlertHierarchyColumn.HierarchyHandle == InHierarchyHandle)
				{
					Context.ActivateQueries(InAlertPropagationStartConditionName);
				}
			})
		.Where()
			.All(DataStorage.GetChildTagType(InHierarchyHandle)) // Only need to do an update pass if there are parents.
			.None<FTedsAlertChainTag>()
		.Compile());

	DataStorage.RegisterQuery(
		Select("Register alert with parent on parent add (FHierarchyHandle)",
			FObserver(FObserver::EEvent::Add, DataStorage.GetChildTagType(InHierarchyHandle)),
			[InHierarchyHandle, InAlertPropagationStartConditionName](IQueryContext& Context, RowHandle Row, const FTedsAlertHierarchyColumn& AlertHierarchyColumn)
			{
				if (AlertHierarchyColumn.HierarchyHandle == InHierarchyHandle)
				{
					Context.ActivateQueries(InAlertPropagationStartConditionName);
				}
			})
		.Where()
			.Any<FTedsAlertColumn, FTedsChildAlertColumn>()
			.None<FTedsAlertChainTag>()
		.Compile());
}

void UTedsAlertsFactory::RegisterOnRemoveHierarchyQueries(UE::Editor::DataStorage::ICoreProvider& DataStorage,
	UE::Editor::DataStorage::FHierarchyHandle InHierarchyHandle, const FName& InAlertPropagationStartConditionName)
{
	using namespace UE::Editor::DataStorage::Queries;

	DataStorage.RegisterQuery(
		Select("Update alert upon parent removal", FObserver(FObserver::EEvent::Remove, DataStorage.GetChildTagType(InHierarchyHandle)),
			[InHierarchyHandle, InAlertPropagationStartConditionName](IQueryContext& Context, RowHandle Row, const FTedsAlertHierarchyColumn& AlertHierarchyColumn)
			{
				if (AlertHierarchyColumn.HierarchyHandle == InHierarchyHandle)
				{
					Context.ActivateQueries(InAlertPropagationStartConditionName);
				}
			})
		.Where()
			.Any<FTedsAlertColumn, FTedsChildAlertColumn>()
			.None<FTedsAlertChainTag>()
		.Compile());
}

void UTedsAlertsFactory::RegisterParentUpdatesHierarchyQueries(UE::Editor::DataStorage::ICoreProvider& DataStorage, UE::Editor::DataStorage::FHierarchyHandle InHierarchyHandle, const FName& InAlertPropagationStartConditionName)
{
	using namespace UE::Editor::DataStorage::Queries;

	if (const UScriptStruct* ParentChangedColumnType = DataStorage.GetParentChangedColumnType(InHierarchyHandle))
	{
		DataStorage.RegisterQuery(
			Select(
			"Trigger alert update if alert or child alert's parent changed",
			FProcessor(EQueryTickPhase::PostPhysics, DataStorage.GetQueryTickGroupName(EQueryTickGroups::Default)),
			[InHierarchyHandle, InAlertPropagationStartConditionName](IQueryContext& Context, RowHandle Row, const FTedsAlertHierarchyColumn& AlertHierarchyColumn)
			{
				if (AlertHierarchyColumn.HierarchyHandle == InHierarchyHandle)
				{
					Context.ActivateQueries(InAlertPropagationStartConditionName);
				}
			})
		.Where()
			.All(ParentChangedColumnType)
			.Any<FTedsAlertColumn, FTedsChildAlertColumn>()
			.None<FTedsAlertChainTag>()
		.Compile());
	}
}

void UTedsAlertsFactory::RegisterChildAlertHierarchyUpdatesQueries(UE::Editor::DataStorage::ICoreProvider& DataStorage, const FName& InHierarchyName, UE::Editor::DataStorage::FHierarchyHandle InHierarchyHandle, FAlertHierarchyConditions* InConditions)
{
	using namespace UE::Editor::DataStorage::Queries;

	const FName AlertPropagationStartConditionName = InConditions->AlertPropagationStartConditionName;
	const FName AlertPropagationConditionName = InConditions->AlertPropagationConditionName;
	const FName AlertPropagationFinishedConditionName = InConditions->AlertPropagationFinishedConditionName;

	// Ensure all ancestor rows and the leaf alert row itself have the FTedsChildAlertColumn before counting begins.
	DataStorage.RegisterQuery(
		Select(
			"Add missing child alerts",
			FPhaseAmble(FPhaseAmble::ELocation::Preamble, EQueryTickPhase::PostPhysics)
				.MakeActivatable(AlertPropagationStartConditionName),
			[InHierarchyHandle](IQueryContext& Context, RowHandle Row, const FTedsAlertHierarchyColumn& AlertHierarchyColumn)
			{
				if (AlertHierarchyColumn.HierarchyHandle == InHierarchyHandle)
				{
					// Add FTedsChildAlertColumn to this leaf alert row so it participates in the per-frame propagation pass alongside intermediate nodes.
					if (!Context.HasColumn<FTedsChildAlertColumn>(Row))
					{
						FTedsChildAlertColumn LeafChildAlert;
						ResetChildAlertCounters(LeafChildAlert);
						Context.AddColumn(Row, MoveTemp(LeafChildAlert));
					}

					RowHandle ParentRow = Context.GetParentRow(Row);
					if (Context.IsRowAssigned(ParentRow))
					{
						AddChildAlertsToHierarchy(Context, ParentRow, AlertHierarchyColumn.HierarchyHandle);
					}
				}
			})
		.AccessesHierarchy(InHierarchyName)
		.Where()
			.All<FTedsAlertColumn>()
			.None<FTedsAlertChainTag>()
		.Compile());

	// Count every node that will propagate.
	// For each node, record it in PendingPropagations and increment its parent's NumAlertChildren so the parent knows how many to wait for.
	// Activate the per-frame propagation condition for the next frame.
	DataStorage.RegisterQuery(
		Select(
			"Count alert children",
			FProcessor(EQueryTickPhase::PostPhysics, DataStorage.GetQueryTickGroupName(EQueryTickGroups::Update))
				.MakeActivatable(AlertPropagationStartConditionName),
			[InHierarchyHandle, PendingPropagations = &InConditions->PendingPropagations, AlertPropagationConditionName](
				IQueryContext& Context, RowHandle Row, const FTedsAlertHierarchyColumn& AlertHierarchyColumn)
			{
				if (AlertHierarchyColumn.HierarchyHandle != InHierarchyHandle)
				{
					return;
				}

				PendingPropagations->fetch_add(1);

				RowHandle ParentRow = Context.GetParentRow(Row);
				if (Context.IsRowAssigned(ParentRow))
				{
					Context.RunSubquery(0, ParentRow, CreateSubqueryCallbackBinding(
						[](ISubqueryContext&, RowHandle, FTedsChildAlertColumn& ParentChildAlert)
						{
							++ParentChildAlert.NumAlertChildren;
						}));
				}

				// Schedule the propagation wave for next frame.
				Context.ActivateQueries(AlertPropagationConditionName);
			})
		.AccessesHierarchy(InHierarchyName)
		.Where()
			.Any<FTedsAlertColumn, FTedsChildAlertColumn>()
			.None<FTedsAlertChainTag>()
		.DependsOn()
			.SubQuery(ChildAlertColumnReadWriteQuery)
		.Compile());

	// Reset the per-hierarchy propagation counter once at the start of each AlertPropagationStartConditionName wave,
	// before "Count alert children" starts incrementing it.
	DataStorage.RegisterQuery(
		Select(
			"Reset pending propagations",
			FPhaseAmble(FPhaseAmble::ELocation::Preamble, EQueryTickPhase::PostPhysics)
				.MakeActivatable(AlertPropagationStartConditionName),
			[PendingPropagations = &InConditions->PendingPropagations](IQueryContext&, const RowHandle*)
			{
				PendingPropagations->store(0);
			})
		.Where()
			.All<FTedsChildAlertColumn>()
			.None<FTedsAlertChainTag>()
		.Compile());

	DataStorage.RegisterQuery(
		Select(
			"Clear child alerts",
			FProcessor(EQueryTickPhase::PostPhysics, DataStorage.GetQueryTickGroupName(EQueryTickGroups::PreUpdate))
				.MakeActivatable(AlertPropagationStartConditionName),
			[InHierarchyHandle](IQueryContext&, RowHandle, FTedsChildAlertColumn& ChildAlert, const FTedsAlertHierarchyColumn& AlertHierarchyColumn)
			{
				if (AlertHierarchyColumn.HierarchyHandle == InHierarchyHandle)
				{
					ResetChildAlertCounters(ChildAlert);
				}
			})
		.Where()
			.None<FTedsAlertChainTag>()
		.Compile());

	DataStorage.RegisterQuery(
		Select(
			"Remove unused child alerts",
			FProcessor(EQueryTickPhase::PostPhysics, DataStorage.GetQueryTickGroupName(EQueryTickGroups::PostUpdate))
				.MakeActivatable(AlertPropagationFinishedConditionName),
			[InHierarchyHandle](IQueryContext& Context, RowHandle Row, FTedsChildAlertColumn& ChildAlert, const FTedsAlertHierarchyColumn& AlertHierarchyColumn)
			{
				if (AlertHierarchyColumn.HierarchyHandle != InHierarchyHandle)
				{
					return;
				}
				for (int32 It = 0; It < static_cast<int32>(FTedsAlertColumnType::MAX); ++It)
				{
					if (ChildAlert.Counts[It] != 0)
					{
						return;
					}
				}
				Context.RemoveColumns<FTedsChildAlertColumn, FTedsAlertHierarchyColumn>(Row);
				Context.AddColumns<FTypedElementSyncBackToWorldTag>(Row);
			})
		.Where()
			.None<FTedsAlertChainTag, FTedsAlertColumn>()
		.Compile());

	// For each node whose direct contributing children have all reported in, push its accumulated counts plus its own alert, if any to its parent,
	// then decrement the per-hierarchy counter.
	DataStorage.RegisterQuery(
		Select(
			"Propagate child alerts",
			FProcessor(EQueryTickPhase::PostPhysics, DataStorage.GetQueryTickGroupName(EQueryTickGroups::PostUpdate))
				.MakeActivatable(AlertPropagationConditionName),
			[InHierarchyHandle, PendingPropagations = &InConditions->PendingPropagations](
				IQueryContext& Context, RowHandle Row, FTedsChildAlertColumn& ChildAlert,
				const FTedsAlertHierarchyColumn& AlertHierarchyColumn)
			{
				if (AlertHierarchyColumn.HierarchyHandle != InHierarchyHandle)
				{
					return;
				}

				// Not ready yet: still waiting for some children to report in.
				if (ChildAlert.ProcessedChildCount != ChildAlert.NumAlertChildren)
				{
					return;
				}

				if (const RowHandle ParentRow = Context.GetParentRow(Row); Context.IsRowAssigned(ParentRow))
				{
					Context.RunSubquery(0, ParentRow, CreateSubqueryCallbackBinding(
						[&ChildAlert, ParentRow, &Context](ISubqueryContext& SubCtx, RowHandle, FTedsChildAlertColumn& ParentChildAlert)
						{
							// Only propagate child count to parent if row has an Alert.
							if (const FTedsAlertColumn* OwnAlert = Context.GetColumn<FTedsAlertColumn>())
							{	
								for (int32 AlertType = 0; AlertType < static_cast<int32>(FTedsAlertColumnType::MAX); ++AlertType)
								{
									ParentChildAlert.Counts[AlertType] += ChildAlert.Counts[AlertType];
								}
								++ParentChildAlert.Counts[static_cast<int32>(OwnAlert->AlertType)];
							}
							++ParentChildAlert.ProcessedChildCount;
							SubCtx.AddColumns<FTypedElementSyncBackToWorldTag>(ParentRow);
						}));
				}
				Context.AddColumns<FTypedElementSyncBackToWorldTag>(Row);

				PendingPropagations->fetch_sub(1);

				// Sentinel: move ProcessedChildCount past NumAlertChildren so this node does not match the condition again in subsequent frames.
				++ChildAlert.ProcessedChildCount;
			})
		.AccessesHierarchy(InHierarchyName)
		.ReadOnly<FTedsAlertColumn>(EOptional::Yes)
		.Where()
			.None<FTedsAlertChainTag>()
		.DependsOn()
			.SubQuery(ChildAlertColumnReadWriteQuery)
		.Compile());

	// If there are still nodes waiting to propagate, re-activate for the next frame.
	DataStorage.RegisterQuery(
		Select(
			"Check alert propagation complete",
			FPhaseAmble(FPhaseAmble::ELocation::Postamble, EQueryTickPhase::PostPhysics)
				.MakeActivatable(AlertPropagationConditionName),
			[PendingPropagations = &InConditions->PendingPropagations, AlertPropagationConditionName, AlertPropagationFinishedConditionName](IQueryContext& Context, const RowHandle*)
			{
				if (PendingPropagations->load() > 0)
				{
					Context.ActivateQueries(AlertPropagationConditionName);
				}
				else
				{
					Context.ActivateQueries(AlertPropagationFinishedConditionName);
				}
			})
		.Where()
			.All<FTedsChildAlertColumn>()
		.Compile());
}

void UTedsAlertsFactory::AssignAlert(
	UE::Editor::DataStorage::ISubqueryContext& TargetContext, UE::Editor::DataStorage::RowHandle TargetRow, FTedsAlertColumn& Target,
	UE::Editor::DataStorage::IQueryContext& SourceContext, UE::Editor::DataStorage::RowHandle SourceRow, FTedsAlertColumn& Source,
	int32 AlertActionQueryIndex, int32 AlertHierarchyQueryIndex)
{
	Target = MoveTemp(Source);
	Target.NextAlert = UE::Editor::DataStorage::InvalidRowHandle;

	// Assign Alert Action
	FTedsAlertActionColumn* TargetAction = GetAction(SourceContext, TargetRow, AlertActionQueryIndex);
	FTedsAlertActionColumn* SourceAction = GetAction(SourceContext, SourceRow, AlertActionQueryIndex);
	if (SourceAction)
	{
		if (TargetAction)
		{
			*TargetAction = MoveTemp(*SourceAction);
		}
		else
		{
			SourceContext.AddColumn(TargetRow, MoveTemp(*SourceAction));
		}
	}
	else
	{
		if (TargetAction)
		{
			TargetContext.RemoveColumns<FTedsAlertActionColumn>(TargetRow);
		}
	}

	// Assign Alert Hierarchy
	FTedsAlertHierarchyColumn* TargetHierarchy = GetHierarchy(SourceContext, TargetRow, AlertHierarchyQueryIndex);
	FTedsAlertHierarchyColumn* SourceHierarchy = GetHierarchy(SourceContext, SourceRow, AlertHierarchyQueryIndex);
	if (SourceHierarchy)
	{
		if (TargetHierarchy)
		{
			*TargetHierarchy = MoveTemp(*SourceHierarchy);
		}
		else
		{
			SourceContext.AddColumn(TargetRow, MoveTemp(*SourceHierarchy));
		}
	}
	else
	{
		if (TargetHierarchy)
		{
			TargetContext.RemoveColumns<FTedsAlertHierarchyColumn>(TargetRow);
		}
	}
	
	// Notify UI.
	TargetContext.AddColumns<FTypedElementSyncBackToWorldTag>(TargetRow);
	// Fully absorbed into the placeholder so the addition to the chain is no longer needed.
	SourceContext.RemoveRow(SourceRow);
}

void UTedsAlertsFactory::SwapAlerts(
	UE::Editor::DataStorage::ISubqueryContext& OriginalContext, UE::Editor::DataStorage::RowHandle OriginalRow, FTedsAlertColumn& Original,
	UE::Editor::DataStorage::IQueryContext& NewContext, UE::Editor::DataStorage::RowHandle NewRow, FTedsAlertColumn& New,
	int32 AlertActionQueryIndex, int32 AlertHierarchyQueryIndex)
{
	Swap(Original, New);
	
	// Swap the alert action columns if they exist.
	{
		FTedsAlertActionColumn* OriginalAction = GetAction(NewContext, OriginalRow, AlertActionQueryIndex);
		FTedsAlertActionColumn* NewAction = GetAction(NewContext, NewRow, AlertActionQueryIndex);
		if (NewAction)
		{
			if (OriginalAction)
			{
				Swap(*OriginalAction, *NewAction);
			}
			else
			{
				OriginalContext.AddColumn(OriginalRow, MoveTemp(*NewAction));
				NewContext.RemoveColumns<FTedsAlertActionColumn>(NewRow);
			}
		}
		else
		{
			if (OriginalAction)
			{
				NewContext.AddColumn(NewRow, MoveTemp(*OriginalAction));
				OriginalContext.RemoveColumns<FTedsAlertActionColumn>(OriginalRow);
			}
		}
	}

	// Swap the alert hierarchy columns if they exist.
	{
		FTedsAlertHierarchyColumn* OriginalHierarchy = GetHierarchy(NewContext, OriginalRow, AlertHierarchyQueryIndex);
		FTedsAlertHierarchyColumn* NewHierarchy = GetHierarchy(NewContext, NewRow, AlertHierarchyQueryIndex);
		if (NewHierarchy)
		{
			if (OriginalHierarchy)
			{
				Swap(*OriginalHierarchy, *NewHierarchy);
			}
			else
			{
				OriginalContext.AddColumn(OriginalRow, MoveTemp(*NewHierarchy));
				NewContext.RemoveColumns<FTedsAlertHierarchyColumn>(NewRow);
			}
		}
		else
		{
			if (OriginalHierarchy)
			{
				NewContext.AddColumn(NewRow, MoveTemp(*OriginalHierarchy));
				OriginalContext.RemoveColumns<FTedsAlertHierarchyColumn>(OriginalRow);
			}
		}
	}

	Original.NextAlert = NewRow;
	// Notify UI. Most of the time this is not needed as only the active alert should be used in the UI,
	// but the TEDS Debugger might be showing the other alerts, so make sure they're updated to prevent
	// present invalid data or worse in the case of a action which can cause a crash.
	OriginalContext.AddColumns<FTypedElementSyncBackToWorldTag>(OriginalRow);
	NewContext.AddColumns<FTypedElementSyncBackToWorldTag>(NewRow);
	NewContext.RemoveColumns<FTedsUnsortedAlertChainTag>(NewRow);
}

void UTedsAlertsFactory::AppendAlert(
	FTedsAlertColumn& LastAlert,
	UE::Editor::DataStorage::IQueryContext& AdditionalAlertContext,
	UE::Editor::DataStorage::RowHandle AdditionalAlertRow,
	FTedsAlertColumn& AdditionalAlert)
{
	AdditionalAlert.NextAlert = UE::Editor::DataStorage::InvalidRowHandle;
	LastAlert.NextAlert = AdditionalAlertRow;
	AdditionalAlertContext.RemoveColumns<FTedsUnsortedAlertChainTag>(AdditionalAlertRow);
}

FTedsAlertActionColumn* UTedsAlertsFactory::GetAction(UE::Editor::DataStorage::IQueryContext& Context,
	UE::Editor::DataStorage::RowHandle Row, int32 ChildAlertQueryIndex)
{
	using namespace UE::Editor::DataStorage::Queries;
	
	FTedsAlertActionColumn* Result = nullptr;
	Context.RunSubquery(ChildAlertQueryIndex, Row, CreateSubqueryCallbackBinding(
		[&Result](ISubqueryContext& Context, FTedsAlertActionColumn& Action)
		{
			Result = &Action;
		}));
	return Result;
}

FTedsAlertHierarchyColumn* UTedsAlertsFactory::GetHierarchy(UE::Editor::DataStorage::IQueryContext& Context, UE::Editor::DataStorage::RowHandle Row,
	int32 ChildAlertQueryIndex)
{
	using namespace UE::Editor::DataStorage::Queries;
	
	FTedsAlertHierarchyColumn* Result = nullptr;
	Context.RunSubquery(ChildAlertQueryIndex, Row, CreateSubqueryCallbackBinding(
		[&Result](ISubqueryContext& Context, FTedsAlertHierarchyColumn& HierarchyColumn)
		{
			Result = &HierarchyColumn;
		}));
	return Result;
}

void UTedsAlertsFactory::AddChildAlertsToHierarchy(
	UE::Editor::DataStorage::IQueryContext& Context, UE::Editor::DataStorage::RowHandle Parent, UE::Editor::DataStorage::FHierarchyHandle HierarchyHandle)
{
	using namespace UE::Editor::DataStorage;

	// Only add to the direct parent. The gray warning indicator is intentionally limited to
	// the immediate parent of an alert. Grandparents and higher do not receive the column.
	if (!Context.HasColumn<FTedsChildAlertColumn>(Parent))
	{
		FTedsChildAlertColumn ChildAlert;
		ResetChildAlertCounters(ChildAlert);
		Context.AddColumn(Parent, MoveTemp(ChildAlert));

		// Child alerts also need the hierarchy column to walk up during propagation.
		Context.AddColumn(Parent, FTedsAlertHierarchyColumn{ .HierarchyHandle = HierarchyHandle });
	}
}



void UTedsAlertsFactory::ResetChildAlertCounters(FTedsChildAlertColumn& ChildAlert)
{
	for (size_t It = 0; It < static_cast<size_t>(FTedsAlertColumnType::MAX); ++It)
	{
		ChildAlert.Counts[It] = 0;
	}
	ChildAlert.ProcessedChildCount = 0;
	ChildAlert.NumAlertChildren = 0;
}