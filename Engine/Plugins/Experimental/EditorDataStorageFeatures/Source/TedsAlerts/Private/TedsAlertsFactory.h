// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <atomic>

#include "DataStorage/Handles.h"
#include "Elements/Interfaces/TypedElementDataStorageFactory.h"
#include "TedsAlertColumns.h"
#include "UObject/ObjectMacros.h"

#include "TedsAlertsFactory.generated.h"

/**
 * Factory that manages tables, queries and any other data for alerts.
 */
UCLASS()
class UTedsAlertsFactory final : public UEditorDataStorageFactory
{
	GENERATED_BODY()

public:
	static const FName AlertChainTableName;
	static const FName UnsortedAlertChainTableName;

	~UTedsAlertsFactory() override = default;
	
	void RegisterTables(UE::Editor::DataStorage::ICoreProvider& DataStorage) override;
	void RegisterQueries(UE::Editor::DataStorage::ICoreProvider& DataStorage) override;

	UE::Editor::DataStorage::TableHandle GetAlertChainTable() const;
	UE::Editor::DataStorage::TableHandle GetUnsortedAlertChainTable() const;

	UE::Editor::DataStorage::QueryHandle GetSortedAlertsQuery() const;
	UE::Editor::DataStorage::QueryHandle GetUnsortedAlertsQuery() const;

	// Register a new hierarchy as being part of alerts. This adds observers to track changes for this specific hierarchy so rows with alerts
	// beloging to this hierarchy have can update their parent
	void RegisterAlertHierarchy(UE::Editor::DataStorage::ICoreProvider& DataStorage, const FName& InHierarchyName);

private:
	// Per-hierarchy condition names and propagation counter.
	// Stored as a unique_ptr so the address of PendingPropagations is stable after the map grows.
	struct FAlertHierarchyConditions
	{
		FName AlertPropagationStartConditionName;
		FName AlertPropagationConditionName;
		FName AlertPropagationFinishedConditionName;
		// Counts how many nodes still need to propagate their alert counts to their parent.
		// Decremented by "Propagate child alerts"; when it hits 0 propagation is complete for this wave.
		std::atomic<int32> PendingPropagations{0};
	};
	TMap<FName, TUniquePtr<FAlertHierarchyConditions>> HierarchyConditions;

	// All per-hierarchy AlertPropagationStartConditionNames, captured by non-hierarchy queries so they can
	// activate every registered hierarchy when a global alert change occurs.
	TArray<FName> AllAlertPropagationStartConditionNames;

	UE::Editor::DataStorage::TableHandle ChainTable = UE::Editor::DataStorage::InvalidTableHandle;
	UE::Editor::DataStorage::TableHandle UnsortedChainTable = UE::Editor::DataStorage::InvalidTableHandle;

	UE::Editor::DataStorage::QueryHandle SortedAlertsQuery = UE::Editor::DataStorage::InvalidQueryHandle;
	UE::Editor::DataStorage::QueryHandle UnsortedAlertsQuery = UE::Editor::DataStorage::InvalidQueryHandle;
	UE::Editor::DataStorage::QueryHandle AlertActionQuery = UE::Editor::DataStorage::InvalidQueryHandle;
	UE::Editor::DataStorage::QueryHandle AlertHierarchyQuery = UE::Editor::DataStorage::InvalidQueryHandle;
	UE::Editor::DataStorage::QueryHandle ChildAlertColumnReadWriteQuery = UE::Editor::DataStorage::InvalidQueryHandle;

	void RegisterSubQueries(UE::Editor::DataStorage::ICoreProvider& DataStorage);
	void RegisterSortUnsortedAlertsQuery(UE::Editor::DataStorage::ICoreProvider& DataStorage);
	void RegisterOnRemoveQueries(UE::Editor::DataStorage::ICoreProvider& DataStorage);

	// Hierarchy update queries, these currently have to be registered per hierarchy since a query can only access one hierarchy and it isn't possible
	// to right a query that can dynamically pick which hierarchy to use.
	void RegisterOnAddHierarchyQueries(UE::Editor::DataStorage::ICoreProvider& DataStorage, UE::Editor::DataStorage::FHierarchyHandle InHierarchyHandle, const FName& InAlertPropagationStartConditionName);
	void RegisterOnRemoveHierarchyQueries(UE::Editor::DataStorage::ICoreProvider& DataStorage, UE::Editor::DataStorage::FHierarchyHandle InHierarchyHandle, const FName& InAlertPropagationStartConditionName);
	void RegisterParentUpdatesHierarchyQueries(UE::Editor::DataStorage::ICoreProvider& DataStorage, UE::Editor::DataStorage::FHierarchyHandle InHierarchyHandle, const FName& InAlertPropagationStartConditionName);
	void RegisterChildAlertHierarchyUpdatesQueries(UE::Editor::DataStorage::ICoreProvider& DataStorage, const FName& InHierarchyName, UE::Editor::DataStorage::FHierarchyHandle InHierarchyHandle, FAlertHierarchyConditions* InConditions);

	static void AssignAlert(
		UE::Editor::DataStorage::ISubqueryContext& TargetContext, UE::Editor::DataStorage::RowHandle TargetRow, FTedsAlertColumn& Target,
		UE::Editor::DataStorage::IQueryContext& SourceContext, UE::Editor::DataStorage::RowHandle SourceRow, FTedsAlertColumn& Source,
		int32 AlertActionQueryIndex, int32 AlertHierarchyQueryIndex);
	static void SwapAlerts(
		UE::Editor::DataStorage::ISubqueryContext& OriginalContext, UE::Editor::DataStorage::RowHandle OriginalRow, FTedsAlertColumn& Original,
		UE::Editor::DataStorage::IQueryContext& NewContext, UE::Editor::DataStorage::RowHandle NewRow, FTedsAlertColumn& New,
		int32 AlertActionQueryIndex, int32 AlertHierarchyQueryIndex);
	static void AppendAlert(
		FTedsAlertColumn& LastAlert, 
		UE::Editor::DataStorage::IQueryContext& AdditionalAlertContext, 
		UE::Editor::DataStorage::RowHandle AdditionalAlertRow, 
		FTedsAlertColumn& AdditionalAlert);
	static FTedsAlertActionColumn* GetAction(UE::Editor::DataStorage::IQueryContext& Context,
		UE::Editor::DataStorage::RowHandle Row, int32 ChildAlertQueryIndex);
	static FTedsAlertHierarchyColumn* GetHierarchy(UE::Editor::DataStorage::IQueryContext& Context,
		UE::Editor::DataStorage::RowHandle Row, int32 ChildAlertQueryIndex);

	static void AddChildAlertsToHierarchy(
		UE::Editor::DataStorage::IQueryContext& Context, UE::Editor::DataStorage::RowHandle Parent, UE::Editor::DataStorage::FHierarchyHandle HierarchyHandle);
	static void ResetChildAlertCounters(FTedsChildAlertColumn& ChildAlert);
};
