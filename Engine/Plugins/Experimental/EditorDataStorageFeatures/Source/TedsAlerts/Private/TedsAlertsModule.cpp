// Copyright Epic Games, Inc. All Rights Reserved.

#include "TedsAlertsModule.h"

#include "Modules/ModuleManager.h"

namespace UE::Editor::DataStorage
{
	void FTedsAlertsModule::StartupModule()
	{
		IModuleInterface::StartupModule();

		FModuleManager::Get().LoadModule(TEXT("TypedElementFramework"));	
	}

	void FTedsAlertsModule::ShutdownModule()
	{
		IModuleInterface::ShutdownModule();
	}
} // namespace UE::Editor::DataStorage

IMPLEMENT_MODULE(UE::Editor::DataStorage::FTedsAlertsModule, TedsAlerts);
