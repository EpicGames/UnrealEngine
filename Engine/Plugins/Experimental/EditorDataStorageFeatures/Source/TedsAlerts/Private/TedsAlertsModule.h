// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"

namespace UE::Editor::DataStorage
{
	class FTedsAlertsModule
		: public IModuleInterface
	{
	public:

		FTedsAlertsModule() = default;

		// IModuleInterface interface
		virtual void StartupModule() override;
		virtual void ShutdownModule() override;
	};
} // namespace UE::Editor::DataStorage
