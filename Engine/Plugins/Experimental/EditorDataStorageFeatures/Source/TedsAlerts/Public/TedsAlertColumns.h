// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataStorage/Handles.h"
#include "Elements/Interfaces/TypedElementDataStorageInterface.h"
#include "Internationalization/Text.h"
#include "UObject/ObjectMacros.h"

#include "TedsAlertColumns.generated.h"

UENUM()
enum class FTedsAlertColumnType : uint8
{
	Warning,
	Error,

	MAX
};

using FTedsAlertActionCallback = TFunction<void(UE::Editor::DataStorage::RowHandle)>;

/**
 * Column containing information a user needs to be alerted of.
 */
USTRUCT(meta = (DisplayName = "Alert"))
struct FTedsAlertColumn final : public FEditorDataStorageColumn
{
	GENERATED_BODY()

	UPROPERTY()
	FText Message;

	/**
	 * If valid, points to the next alert in the chain.The next alert will take the place of the current
	 * alert if this alert is cleared.
	 */
	UE::Editor::DataStorage::RowHandle NextAlert;

	/** Unique name to identify the message with. */
	UPROPERTY()
	FName Name;

	/** The type of alert. This is used for ordering and to show appropriate icons. */
	UPROPERTY(meta = (IgnoreForMemberInitializationTest))
	FTedsAlertColumnType AlertType;

	/**
	 * A chain of priorities is sorted by errors, then warnings. If either group has multiple alerts, alerts are sorted by priority
	 * with the lowest value (0) given the later location and higher value (up to 255) the earlier locations in the chain.
	 */
	UPROPERTY(meta = (IgnoreForMemberInitializationTest))
	uint8 Priority;
};

/**
 * Column containing a count for the number of alerts any child rows have.
 */
USTRUCT(meta = (DisplayName = "Child alert"))
struct FTedsChildAlertColumn final : public FEditorDataStorageColumn
{
	GENERATED_BODY()

	uint16 Counts[static_cast<size_t>(FTedsAlertColumnType::MAX)];

	/** How many direct contributing children (leaf alerts or intermediate nodes) have propagated to this row. */
	uint16 ProcessedChildCount = 0;

	/** Expected number of direct contributing children. Set at reset time by the "Count alert children" processor. */
	uint16 NumAlertChildren = 0;
};

/**
 * Column that holds info about the hierarchy an alert is participating in
 */
USTRUCT(meta = (DisplayName = "Alert Hierarchy Column"))
struct FTedsAlertHierarchyColumn final : public FEditorDataStorageColumn
{
	GENERATED_BODY()
	
	UE::Editor::DataStorage::FHierarchyHandle HierarchyHandle;
};

/**
 * Column that can be added to an alert column to have it trigger an action when the alert is clicked.
 */
USTRUCT(meta = (DisplayName = "Alert action"))
struct FTedsAlertActionColumn final : public FEditorDataStorageColumn
{
	GENERATED_BODY()

	FTedsAlertActionCallback Action;
};

/**
 * Tag to indicate this row contains alert chain information.
 */
USTRUCT(meta = (DisplayName = "Alert chain"))
struct FTedsAlertChainTag final : public FEditorDataStorageTag
{
	GENERATED_BODY()
};

/**
 * Tag to indicate this row contains alert chain information that has not been added/sorted yet. This means that the
 * "NextAlert" in the Alert column points to the target row, not the row for the next alert.
 */
USTRUCT(meta = (DisplayName = "Unsorted alert chain"))
struct FTedsUnsortedAlertChainTag final : public FEditorDataStorageTag
{
	GENERATED_BODY()
};

namespace UE::Editor::DataStorage::Columns
{
	using FAlertColumnType = FTedsAlertColumnType;
	using FAlertActionCallback = FTedsAlertActionCallback;
	
	using FAlertColumn = FTedsAlertColumn;
	using FChildAlertColumn = FTedsChildAlertColumn;
	using FAlertHierarchyColumn = FTedsAlertHierarchyColumn;
	using FAlertActionColumn = FTedsAlertActionColumn;
	using FAlertChainTag = FTedsAlertChainTag;
	using FUnsortedAlertChainTag = FTedsUnsortedAlertChainTag;
}
