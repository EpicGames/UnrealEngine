// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataStorage/Handles.h"
#include "Internationalization/Text.h"
#include "TedsAlertColumns.h"
#include "UObject/NameTypes.h"

namespace UE::Editor::DataStorage
{
	struct IQueryContext;
	class ICoreProvider;
}

namespace UE::Editor::DataStorage::Alerts
{
	/**
	 * Add an alert to the provided row. Various tools like the Outliner monitor for alerts and can show them for instance as
	 * a colored triangle with a exclamation count. If a row also has a parent row handle, the alerts will propagate their count
	 * to the parent.
	 * @DataStorage A reference to the data storage instance to store the alert in.
	 * @TargetRow The target row to store the alert on. If the row already has an alert, an ordered chain of alerts will be created.
	 * @Name A name to uniquely identify the alert. If there's a chain of alerts this is used to find the correct alert to remove.
	 * @Message The message shown in the alert, e.g. as a tool tip on the alert icon.
	 * @Type The type of alerts. Widgets will use separate child counts per type and use different colors such as red for errors and 
	 *		yellow for warnings.
	 * @Priority If there are multiple alerts for the same type, the priority is used to order the alerts so the most important
	 *		one is shown when the active alert is removed. Higher values indicate higher priority.
	 * @Action An optional callback that gets triggered when the user presses the alert icon.
	 * @HierarchyName Optional hierarchy name to use when resolving parent alerts. The alert queries will walk up the parent chain in the provided
	 *		hierarchy to add the child alert column
	 */
	TEDSALERTS_API void AddAlert(ICoreProvider& DataStorage, RowHandle TargetRow, const FName& Name, FText Message, 
		Columns::FAlertColumnType Type, uint8 Priority = 127, Columns::FAlertActionCallback Action = {}, const FName& HierarchyName = NAME_None);
	/**
	 * Add an alert to the provided row. Various tools like the Outliner monitor for alerts and can show them for instance as
	 * a colored triangle with a exclamation count. If a row also has a parent row handle, the alerts will propagate their count
	 * to the parent.
	 * @Context A reference to the query callback context.
	 * @TargetRow The target row to store the alert on. If the row already has an alert, an ordered chain of alerts will be created.
	 * @Name A name to uniquely identify the alert. If there's a chain of alerts this is used to find the correct alert to remove.
	 * @Message The message shown in the alert, e.g. as a tool tip on the alert icon.
	 * @Type The type of alerts. Widgets will use separate child counts per type and use different colors such as red for errors and
	 *		yellow for warnings.
	 * @Priority If there are multiple alerts for the same type, the priority is used to order the alerts so the most important
	 *		one is shown when the active alert is removed. Higher values indicate higher priority.
	 * @Action An optional callback that gets triggered when the user presses the alert icon.
	 * @HierarchyName Optional hierarchy name to use when resolving parent alerts. The alert queries will walk up the parent chain in the provided
	 *		hierarchy to add the child alert column
	 */
	TEDSALERTS_API void AddAlert(IQueryContext& Context, RowHandle TargetRow, const FName& Name, FText Message,
		Columns::FAlertColumnType Type, uint8 Priority = 127, Columns::FAlertActionCallback Action = {}, const FName& HierarchyName = NAME_None);
	
	/**
	 * Locates the first alerts with the provided name in the alert chain found at the provided row and updates its text.
	 * @DataStorage A reference to the data storage instance to store the alert in.
	 * @TargetRow The row the alert is stored on.
	 * @Name The name that uniquely identifies the alert.
	 * @Message The new message for the alert.
	 */
	TEDSALERTS_API void UpdateAlertText(ICoreProvider& DataStorage, RowHandle TargetRow, const FName& Name, FText Message);
	/**
	 * Locates the first alerts with the provided name in the alert chain found at the provided row and updates its text.
	 * @Context A reference to the query callback context.
	 * @TargetRow The row the alert is stored on.
	 * @Name The name that uniquely identifies the alert.
	 * @Message The new message for the alert.
	 */
	TEDSALERTS_API void UpdateAlertText(IQueryContext& Context, RowHandle TargetRow, const FName& Name, FText Message);

	/**
	 * Locates the first alerts with the provided name in the alert chain found at the provided row and updates its action.
	 * If there's no action it will be added. If the provided action isn't bound, the alert action will be removed.
	 * @DataStorage A reference to the data storage instance to store the alert in.
	 * @TargetRow The row the alert is stored on.
	 * @Name The name that uniquely identifies the alert.
	 * @Action The new action to trigger.
	 */
	TEDSALERTS_API void UpdateAlertAction(
		ICoreProvider& DataStorage, RowHandle TargetRow, const FName& Name, Columns::FAlertActionCallback Action);
	/**
	 * Locates the first alerts with the provided name in the alert chain found at the provided row and updates its action.
	 * If there's no action it will be added. If the provided action isn't bound, the alert action will be removed.
	 * @Context A reference to the query callback context.
	 * @TargetRow The row the alert is stored on.
	 * @Name The name that uniquely identifies the alert.
	 * @Action The new action to trigger.
	 */
	TEDSALERTS_API void UpdateAlertAction(IQueryContext& Context, RowHandle TargetRow, const FName& Name, Columns::FAlertActionCallback Action);
	
	/**
	 * Searches the alert chain found at the target row for the first alert with the provided name and removes it. The chain will
	 * automatically be patched, keeping the chain in order of priority. If the root alert is removed, the UI will be automatically
	 * updated with the next alert in line or be cleared if there are no more alerts.
	 * @DataStorage A reference to the data storage instance to store the alert in.
	 * @TargetRow The row the alert is stored on.
	 * @Name The name that uniquely identifies the alert.
	 */
	TEDSALERTS_API void RemoveAlert(ICoreProvider& DataStorage, RowHandle TargetRow, const FName& Name);
	/**
	 * Searches the alert chain found at the target row for the first alert with the provided name and removes it. The chain will
	 * automatically be patched, keeping the chain in order of priority. If the root alert is removed, the UI will be automatically
	 * updated with the next alert in line or be cleared if there are no more alerts.
	 * @Context A reference to the query callback context.
	 * @TargetRow The row the alert is stored on.
	 * @Name The name that uniquely identifies the alert.
	 */
	TEDSALERTS_API void RemoveAlert(IQueryContext& Context, RowHandle TargetRow, const FName& Name);
} // namespace UE::Editor::DataStorage::Alerts
