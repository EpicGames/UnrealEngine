// Copyright Epic Games, Inc. All Rights Reserved.

#include "CB/TagsMetadataCache.h"

#include "UObject/AssetRegistryTagsContext.h"
#include "UObject/LinkerLoad.h"
#include "UObject/NameTypes.h"
#include "UObject/UnrealType.h"

namespace UE::Editor::AssetData::Private
{

void FTagsMetadataCache::CacheClass(FTopLevelAssetPath InClassName)
{
	UClass* FoundClass = FindObject<UClass>(InClassName);

	if (!FoundClass)
	{
		// Look for class redirectors
		FString NewPath = FLinkerLoad::FindNewPathNameForClass(InClassName.ToString(), false);
		if (!NewPath.IsEmpty())
		{
			FoundClass = FindObject<UClass>(nullptr, *NewPath);
		}
	}

	if (FoundClass)
	{
		FClassPropertiesCache ClassProperties;
		FTopLevelAssetPath ClassPath(FoundClass);

		const UObject* CDO = FoundClass->GetDefaultObject();
		FAssetRegistryTagsContextData TagsContext(CDO, EAssetRegistryTagsCaller::Uncategorized);
		CDO->GetAssetRegistryTags(TagsContext);

		TMap<FName, UObject::FAssetRegistryTagMetadata> TagsMetadata;
		CDO->GetAssetRegistryTagMetadata(TagsMetadata);

		ClassProperties.TagNameToCachedProperty.Reserve(TagsContext.Tags.Num());

		for (const TPair<FName, UObject::FAssetRegistryTag>& TagPair : TagsContext.Tags)
		{
			TSharedPtr< UE::Editor::AssetData::FItemAttributeMetadata> TagCache = MakeShared<UE::Editor::AssetData::FItemAttributeMetadata>();
			ClassProperties.TagNameToCachedProperty.Add(TagPair.Key, TagCache);
			TagCache->TagType = TagPair.Value.Type;
			TagCache->DisplayFlags = TagPair.Value.DisplayFlags;

			if (const UObject::FAssetRegistryTagMetadata* TagMetaData = TagsMetadata.Find(TagPair.Key))
			{
				TagCache->DisplayName = TagMetaData->DisplayName;
				TagCache->TooltipText = TagMetaData->TooltipText;
				TagCache->Suffix = TagMetaData->Suffix;
				TagCache->ImportantValue = TagMetaData->ImportantValue;
			}
			else
			{
				// If the tag name corresponds to a property name, use the property tooltip
				const FProperty* Property = FindFProperty<FProperty>(FoundClass, TagPair.Key);
				TagCache->TooltipText = Property ? Property->GetToolTipText() : FText::FromString(FName::NameToDisplayString(TagPair.Key.ToString(), false));
			}

			// Ensure a display name for this tag
			if (TagCache->DisplayName.IsEmpty())
			{
				if (const FProperty* TagField = FindFProperty<FProperty>(FoundClass, TagPair.Key))
				{
					// Take the display name from the corresponding property if possible
					TagCache->DisplayName = TagField->GetDisplayNameText();
				}
				else
				{
					// We have no type information by this point, so no idea if it's a bool :(
					TagCache->DisplayName = FText::AsCultureInvariant(FName::NameToDisplayString(TagPair.Key.ToString(), /*bIsBool*/false));
				}
			}
		}

		ClassPathToCachedClass.Add(ClassPath, MoveTemp(ClassProperties));
	}
}

const FTagsMetadataCache::FClassPropertiesCache* FTagsMetadataCache::FindCacheForClass(FTopLevelAssetPath InClassName) const
{
	return ClassPathToCachedClass.Find(InClassName);
}

TSharedPtr<UE::Editor::AssetData::FItemAttributeMetadata> FTagsMetadataCache::FClassPropertiesCache::GetCacheForTag(const FName InTagName) const
{
	const TSharedPtr<UE::Editor::AssetData::FItemAttributeMetadata>* PropertyCachePtr = TagNameToCachedProperty.Find(InTagName);
	return PropertyCachePtr ? *PropertyCachePtr : TSharedPtr<UE::Editor::AssetData::FItemAttributeMetadata>();
}

}