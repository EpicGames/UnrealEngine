// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "TedsAssetDataStructs.h"

#include "Containers/Map.h"
#include "Containers/Set.h"
#include "Templates/SharedPointer.h"
#include "UObject/TopLevelAssetPath.h"
#include "UObject/NameTypes.h"

namespace UE::Editor::AssetData::Private
{
class FTagsMetadataCache
{
public:

	struct FClassPropertiesCache
	{
	public:
		TSharedPtr<UE::Editor::AssetData::FItemAttributeMetadata> GetCacheForTag(const FName InTagName) const;

	private:
		friend class FTagsMetadataCache;

		TMap<FName, TSharedPtr<UE::Editor::AssetData::FItemAttributeMetadata>> TagNameToCachedProperty;
	};


	void CacheClass(FTopLevelAssetPath InClassName);

	const FClassPropertiesCache* FindCacheForClass(FTopLevelAssetPath InClassName) const;

	template<typename TContainer>
	void BatchCacheClasses(const TContainer& InClasses)
	{
		ClassPathToCachedClass.Reserve(ClassPathToCachedClass.Num() + InClasses.Num());

		for (const FTopLevelAssetPath& ClassPath : InClasses)
		{
			CacheClass(ClassPath);
		}
	}

	private:
		TMap<FTopLevelAssetPath, FClassPropertiesCache> ClassPathToCachedClass;
};

}