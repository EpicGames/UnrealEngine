// Copyright Epic Games, Inc. All Rights Reserved.

#include "TedsAssetDataCBDataSource.h"

#include "TedsAssetDataColumns.h"
#include "TedsAssetData.h"

#include "AssetDefinitionRegistry.h"
#include "AssetRegistry/IAssetRegistry.h"
#include "AssetViewTypes.h"
#include "AssetViewUtils.h"
#include "Blueprint/BlueprintSupport.h"
#include "ContentBrowserDataUtils.h"
#include "DataStorage/Queries/Types.h"
#include "Elements/Framework/TypedElementQueryBuilder.h"
#include "Elements/Interfaces/TypedElementDataStorageInterface.h"
#include "Factories/Factory.h"
#include "HAL/IConsoleManager.h"
#include "Interfaces/IPluginManager.h"
#include "Internationalization/Text.h"
#include "Misc/PathViews.h"
#include "PluginDescriptor.h"
#include "Settings/ContentBrowserSettings.h"
#include "TedsAssetDataModule.h"
#include "Templates/Function.h"
#include "UObject/CoreRedirects.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectMacros.h"
#include "UObject/SoftObjectPtr.h"
#include "UObject/TopLevelAssetPath.h"
#include "UObject/UObjectIterator.h"

namespace UE::Editor::AssetData::Private
{

TAutoConsoleVariable<bool> CVarTEDSAssetDataCBSourceIncludeTagsAndValues(TEXT("TEDS.AssetDataStorage.Metadata"), false, TEXT("When true we will add the meta data for the asset showable in the CB")
	, FConsoleVariableDelegate::CreateLambda([](IConsoleVariable* Variable)
	{
		const bool bIsEnabled = Variable->GetBool();
		FTedsAssetDataModule& Module = FTedsAssetDataModule::GetChecked();
		if (bIsEnabled)
		{
			Module.EnableAssetDataMetadataStorage();
		}
		else
		{
			Module.DisableAssetDataMetadataStorage();
		}
	}));

FTedsAssetDataCBDataSource::FTedsAssetDataCBDataSource(UE::Editor::DataStorage::ICoreProvider& InDatabase)
	: Database(InDatabase)
{
	using namespace UE::Editor::DataStorage;
	using namespace Queries;


	bPopulateMetadataColumns = CVarTEDSAssetDataCBSourceIncludeTagsAndValues.GetValueOnGameThread();
	AssetRegistry = IAssetRegistry::Get();

	InitVirtualPathProcessor();

	if (bPopulateMetadataColumns)
	{
		TagsMetadataCache = MakeUnique<FTagsMetadataCache>();
		PrepopulateTagsMetadataCache();
	}

	ProcessPathQuery = Database.RegisterQuery(
		Select(
			TEXT("FTedsAssetDataCBDataSource: Process Path updates"),
			FProcessor(EQueryTickPhase::DuringPhysics, Database.GetQueryTickGroupName(EQueryTickGroups::Update))
				.BatchModifications(true),
			[DataSource = const_cast<const FTedsAssetDataCBDataSource*>(this)](IQueryContext& Context, const RowHandle* Rows, const FAssetPathColumn_Experimental* PathColumn)
			{
				DataSource->ProcessPathQueryCallback(Context, Rows, PathColumn);
			})
		.Where()
			.All<FUpdatedPathTag>()
		.Compile());

	ProcessAssetDataPathUpdateQuery = Database.RegisterQuery(
		Select(
			TEXT("FTedsAssetDataCBDataSource:: Process Asset Data Path Update"),
			FProcessor(EQueryTickPhase::DuringPhysics, Database.GetQueryTickGroupName(EQueryTickGroups::Update))
				.BatchModifications(true),
			[DataSource = const_cast<const FTedsAssetDataCBDataSource*>(this)](IQueryContext& Context, const RowHandle* Row, const FAssetDataColumn_Experimental* AssetDataColumn)
			{
				DataSource->ProcessAssetDataPathUpdateQueryCallback(Context, Row, AssetDataColumn);
			})
		.Where()
			.All<FUpdatedPathTag>()
			.None<FUpdatedAssetDataTag>()
		.Compile());

	ProcessAssetDataAndPathUpdateQuery = Database.RegisterQuery(
		Select(
			TEXT("FTedsAssetDataCBDataSource: Process Asset Data and Path Updates"),
			FProcessor(EQueryTickPhase::DuringPhysics, Database.GetQueryTickGroupName(EQueryTickGroups::Update))
				.BatchModifications(true),
			[DataSource = const_cast<const FTedsAssetDataCBDataSource*>(this)](IQueryContext& Context, const RowHandle* Rows, const FAssetDataColumn_Experimental* AssetDataColumn)
			{
				DataSource->ProcessAssetDataAndPathUpdateQueryCallback(Context, Rows, AssetDataColumn);
			})
		.Where()
			.All<FUpdatedAssetDataTag, FUpdatedPathTag>()
			.Compile());

	ProcessAssetDataUpdateQuery = Database.RegisterQuery(
		Select(
			TEXT("FTedsAssetDataCBDataSource: Process Asset Data updates"),
			FProcessor(EQueryTickPhase::DuringPhysics, Database.GetQueryTickGroupName(EQueryTickGroups::Update))
				.BatchModifications(true),
			[DataSource = const_cast<const FTedsAssetDataCBDataSource*>(this)](IQueryContext& Context, const RowHandle* Rows, const FAssetDataColumn_Experimental* AssetDataColumn)
			{
				DataSource->ProcessAssetDataUpdateQueryCallback(Context, Rows, AssetDataColumn);
			}
		)
		.Where()
			.All<FUpdatedAssetDataTag, FVirtualPathColumn_Experimental>()
			.None<FUpdatedPathTag>()
		.Compile()
		);


	ReprocessesAssetDataColumns = Database.RegisterQuery(
		Select(
			TEXT("FTedsAssetData: Remove Updated Asset Tag"),
			FPhaseAmble(FPhaseAmble::ELocation::Preamble, EQueryTickPhase::PrePhysics)
				.MakeActivatable(RepopulateAssetDataColumns),
			[DataSource = const_cast<const FTedsAssetDataCBDataSource*>(this)](IQueryContext& Context, const RowHandle* Rows)
			{
				TConstArrayView<RowHandle> RowsArrayView(Rows, Context.GetRowCount());
				if (!DataSource->bPopulateMetadataColumns)
				{
					Context.RemoveColumns<FItemTextAttributeColumn_Experimental, FItemStringAttributeColumn_Experimental>(RowsArrayView);
				}
				Context.AddColumns<FUpdatedAssetDataTag>(RowsArrayView);
			})
		.Where()
			.All<FAssetTag>()
			.Compile());
}

FTedsAssetDataCBDataSource::~FTedsAssetDataCBDataSource()
{
	// Not needed on a editor shut down
	if (!IsEngineExitRequested())
	{
		Database.UnregisterQuery(ReprocessesAssetDataColumns);
		Database.UnregisterQuery(ProcessAssetDataUpdateQuery);
		Database.UnregisterQuery(ProcessAssetDataAndPathUpdateQuery);
		Database.UnregisterQuery(ProcessAssetDataPathUpdateQuery);
		Database.UnregisterQuery(ProcessPathQuery);

		IPluginManager& PluginManager = IPluginManager::Get();
		PluginManager.OnNewPluginContentMounted().RemoveAll(this);
		PluginManager.OnPluginEdited().RemoveAll(this);
		PluginManager.OnPluginUnmounted().RemoveAll(this);
	}
}

void FTedsAssetDataCBDataSource::InitVirtualPathProcessor()
{
	IPluginManager& PluginManager = IPluginManager::Get();

	PluginManager.OnNewPluginContentMounted().AddRaw(this, &FTedsAssetDataCBDataSource::OnPluginContentMounted);
	PluginManager.OnPluginEdited().AddRaw(this, &FTedsAssetDataCBDataSource::OnPluginContentMounted);
	PluginManager.OnPluginUnmounted().AddRaw(this, &FTedsAssetDataCBDataSource::OnPluginUnmounted);

	TArray<TSharedRef<IPlugin>> EnabledPluginsWithContent = PluginManager.GetEnabledPluginsWithContent();
	VirtualPathProcessor.PluginNameToCachedData.Reserve(EnabledPluginsWithContent.Num());

	for (const TSharedRef<IPlugin>& Plugin : EnabledPluginsWithContent)
	{
		FVirtualPathProcessor::FCachedPluginData& Data = VirtualPathProcessor.PluginNameToCachedData.FindOrAdd(Plugin->GetName());
		Data.LoadedFrom = Plugin->GetLoadedFrom();
		Data.EditorCustomVirtualPath = Plugin->GetDescriptor().EditorCustomVirtualPath;
	}

	const UContentBrowserSettings* ContentBrowserSettings = GetDefault<UContentBrowserSettings>();
	VirtualPathProcessor.bShowAllFolder = ContentBrowserSettings->bShowAllFolder;
	VirtualPathProcessor.bOrganizeFolders = ContentBrowserSettings->bOrganizeFolders;
}

void FTedsAssetDataCBDataSource::OnPluginContentMounted(IPlugin& InPlugin)
{
	FVirtualPathProcessor::FCachedPluginData& Data = VirtualPathProcessor.PluginNameToCachedData.FindOrAdd(InPlugin.GetName());
	Data.LoadedFrom = InPlugin.GetLoadedFrom();
	Data.EditorCustomVirtualPath = InPlugin.GetDescriptor().EditorCustomVirtualPath;
}

void FTedsAssetDataCBDataSource::OnPluginUnmounted(IPlugin& InPlugin)
{
	VirtualPathProcessor.PluginNameToCachedData.Remove(InPlugin.GetName());
}

void FTedsAssetDataCBDataSource::EnableMetadataStorage(bool bEnable)
{
	if (bEnable != bPopulateMetadataColumns)
	{
		bPopulateMetadataColumns = bEnable;

		if (bPopulateMetadataColumns)
		{ 
			TagsMetadataCache = MakeUnique<FTagsMetadataCache>();
			PrepopulateTagsMetadataCache();
		}
		else
		{
			TagsMetadataCache.Reset();
		}

		// Make sure the CVar match the current state
		CVarTEDSAssetDataCBSourceIncludeTagsAndValues.AsVariable()->Set(bEnable);

		// Force a update of the asset data columns
		Database.ActivateQueries(RepopulateAssetDataColumns);
	}
}

bool FTedsAssetDataCBDataSource::GenerateVirtualPath(const FStringView InAssetPath, FNameBuilder& OutVirtualizedPath) const
{
	if (!ContentBrowserDataUtils::PathPassesAttributeFilter(InAssetPath, 0, EContentBrowserItemAttributeFilter::IncludeAll))
	{
		return false;
	}

	VirtualPathProcessor.ConvertInternalPathToVirtualPath(InAssetPath, OutVirtualizedPath);
	return true;
}

void FTedsAssetDataCBDataSource::AddAssetDataColumns(UE::Editor::DataStorage::IQueryContext& Context, DataStorage::RowHandle Row, const FAssetData& AssetData, const FAssetPackageData* OptionalPackageData) const
{
	// For now just add the columns one by one but this should be rework to work in batch
	// Not optimized at all but we would like to have the data in sooner for testing purposes.
	const EAssetAccessSpecifier AssetAccessSpecifier = AssetData.GetAssetAccessSpecifier();
	if (AssetAccessSpecifier == EAssetAccessSpecifier::Public)
	{
		Context.AddColumns<FAssetTag, FPublicAssetTag>(Row);
	}
	else if (AssetAccessSpecifier == EAssetAccessSpecifier::EpicInternal)
	{
		Context.AddColumns<FAssetTag, FEpicInternalAssetTag>(Row);
	}
	else
	{
		Context.AddColumns<FAssetTag, FPrivateAssetTag>(Row);
	}

	if (OptionalPackageData)
	{
		FDiskSizeColumn DiskSizeColumn;
		DiskSizeColumn.DiskSize = OptionalPackageData->DiskSize;
		Context.AddColumn(Row, MoveTemp(DiskSizeColumn));
	}
	else
	{
		Context.RemoveColumns<FDiskSizeColumn>(Row);
	}

	FAssetClassColumn AssetClassColumn;
	AssetClassColumn.ClassPath = AssetData.AssetClassPath;
	Context.AddColumn(Row, MoveTemp(AssetClassColumn));

	FAssetNameColumn ItemNameColumn;
	ItemNameColumn.Name = AssetData.AssetName;
	Context.AddColumn(Row, MoveTemp(ItemNameColumn));

	if (bPopulateMetadataColumns)
	{
		static const FTopLevelAssetPath BlueprintAssetClass = FTopLevelAssetPath(TEXT("/Script/Engine"), TEXT("Blueprint"));

		// The population of the cache still need some work.
		const FTagsMetadataCache::FClassPropertiesCache* ClassPropertyTagCache = TagsMetadataCache->FindCacheForClass(AssetData.AssetClassPath);
		const FTagsMetadataCache::FClassPropertiesCache* ParentClassPropertyTagCache = nullptr;

		if (!ClassPropertyTagCache)
		{
			FCoreRedirectObjectName RedirectedName = FCoreRedirects::GetRedirectedName(ECoreRedirectFlags::Type_Class, AssetData.AssetClassPath);
			if (RedirectedName.IsValid())
			{
				ClassPropertyTagCache = TagsMetadataCache->FindCacheForClass(FTopLevelAssetPath(RedirectedName.PackageName, RedirectedName.ObjectName));
			}
		}


		if (AssetData.AssetClassPath == BlueprintAssetClass)
		{
			// Non functional at the moment need to revisit the caching for those.
			FAssetTagValueRef ParentClassRef = AssetData.TagsAndValues.FindTag(FBlueprintTags::ParentClassPath);
			if (ParentClassRef.IsSet())
			{
				ParentClassPropertyTagCache = TagsMetadataCache->FindCacheForClass(ParentClassRef.AsExportPath().ToTopLevelAssetPath());
			}

			if (!ParentClassPropertyTagCache)
			{
				FAssetTagValueRef NativeParentClassRef = AssetData.TagsAndValues.FindTag(FBlueprintTags::NativeParentClassPath);
				if (NativeParentClassRef.IsSet())
				{
					ParentClassPropertyTagCache = TagsMetadataCache->FindCacheForClass(NativeParentClassRef.AsExportPath().ToTopLevelAssetPath());
				}
			}
		}

		
		for (const TPair<FName, FAssetTagValueRef>& TagAndValue : AssetData.TagsAndValues)
		{ 
			TSharedPtr<const UE::Editor::AssetData::FItemAttributeMetadata> AttributeMetadata;
			if (ParentClassPropertyTagCache)
			{
				AttributeMetadata = ParentClassPropertyTagCache->GetCacheForTag(TagAndValue.Key);
			}

			if (!AttributeMetadata && ClassPropertyTagCache)
			{
				AttributeMetadata = ClassPropertyTagCache->GetCacheForTag(TagAndValue.Key);
			}

			// Todo revisit to see if we can save some memory here. 
			FString TagValue = TagAndValue.Value.AsString();
			bool bAddedColumn = false;
			if (FTextStringHelper::IsComplexText(*TagValue))
			{
				FText TmpText;
				if (FTextStringHelper::ReadFromBuffer(*TagValue, TmpText))
				{
					FItemTextAttributeColumn_Experimental AttributeColumn;
					AttributeColumn.Value = MoveTemp(TmpText);
					AttributeColumn.AttributeMetadata = MoveTemp(AttributeMetadata);
					Context.AddColumn(Row, TagAndValue.Key, MoveTemp(AttributeColumn));
					bAddedColumn = true;
				}
			}

			if (!bAddedColumn)
			{
				FItemStringAttributeColumn_Experimental AttributeColumn;
				AttributeColumn.Value = MoveTemp(TagValue);
				AttributeColumn.AttributeMetadata = MoveTemp(AttributeMetadata);
				Context.AddColumn(Row, TagAndValue.Key, MoveTemp(AttributeColumn));
			}
		}


	}
}

void FTedsAssetDataCBDataSource::ProcessPathQueryCallback(DataStorage::IQueryContext& Context, const DataStorage::RowHandle* Rows, const FAssetPathColumn_Experimental* PathColumn) const
{
	uint64 StartTime = FTedsAssetData::ProcessPathAddStartTime.load();
	uint64 CurrentTime = FPlatformTime::Cycles64();

	// Processors run in chunks on a worker thread. 
	// Using compare_exchange_weak allows us to take the min start time and the max end time of all chunks in a thread safe way
	while (CurrentTime < StartTime)
	{
		if (FTedsAssetData::ProcessPathAddStartTime.compare_exchange_weak(StartTime, CurrentTime))
		{
			break;
		}
	}

	int32 NumOfRowToProcess = Context.GetRowCount();
	FNameBuilder InternalPath;
	FNameBuilder VirtualPath;
	FString Path;

	for (int32 Index = 0; Index < NumOfRowToProcess; ++Index)
	{
		Path = PathColumn[Index].Path.ToString();

		InternalPath.Reset();
		InternalPath.Append(PathColumn[Index].Path.ToString());

		if (GenerateVirtualPath(InternalPath, VirtualPath))
		{
			FVirtualPathColumn_Experimental VirtualPathColumn;
			VirtualPathColumn.VirtualPath = *VirtualPath;
			// Todo investigate for a batch add Maybe?
			Context.AddColumn(Rows[Index], MoveTemp(VirtualPathColumn));

			FFolderTypeColumn_Experimental FolderType;

			if (AssetViewUtils::IsPluginFolder(Path))
			{
				FolderType.FolderType = EFolderType::PluginRoot;
			}
			else if (AssetViewUtils::IsDevelopersFolder(Path))
			{
				FolderType.FolderType = EFolderType::Developer;
			}
			// TODO: Missing CPP Folders, need further change and conversion from the old CBClassDataSource (see FNativeClassHierarchy)
			// TODO: Missing Virtual Folders, need further change and conversion from the old CBAssetDataCore (see GetItemAttribute)
			else
			{
				FolderType.FolderType = EFolderType::Normal;
			}

			Context.AddColumn(Rows[Index], MoveTemp(FolderType));
		}
	}

	uint64 EndTime = FTedsAssetData::ProcessPathAddEndTime.load();
	CurrentTime = FPlatformTime::Cycles64();

	while (CurrentTime > EndTime)
	{
		if (FTedsAssetData::ProcessPathAddEndTime.compare_exchange_weak(EndTime, CurrentTime))
		{
			break;
		}
	}

	FTedsAssetData::ProcessedAddedPathBatchSize.fetch_add(NumOfRowToProcess);
}

void FTedsAssetDataCBDataSource::ProcessAssetDataPathUpdateQueryCallback(DataStorage::IQueryContext& Context, const DataStorage::RowHandle* Rows, const FAssetDataColumn_Experimental* AssetDataColumn) const
{
	int32 NumOfRowToProcess = Context.GetRowCount();

	FNameBuilder InternalPath;
	FNameBuilder VirtualPath;

	for (int32 Index = 0; Index < NumOfRowToProcess; ++Index)
	{ 
		InternalPath.Reset();
		AssetDataColumn[Index].AssetData.AppendObjectPath(InternalPath);
		if (GenerateVirtualPath(InternalPath, VirtualPath))
		{
			FVirtualPathColumn_Experimental VirtualPathColumn;
			VirtualPathColumn.VirtualPath = *VirtualPath;
			// Todo investigate for a batch add Maybe?
			Context.AddColumn(Rows[Index], MoveTemp(VirtualPathColumn));
		}
	}
}

void FTedsAssetDataCBDataSource::ProcessAssetDataAndPathUpdateQueryCallback(DataStorage::IQueryContext& Context, const DataStorage::RowHandle* Rows, const FAssetDataColumn_Experimental* AssetDataColumn) const
{
	uint64 StartTime = FTedsAssetData::ProcessAssetAddStartTime.load();
	uint64 CurrentTime = FPlatformTime::Cycles64();

	// Processors run in chunks on a worker thread. 
	// Using compare_exchange_weak allows us to take the min start time and the max end time of all chunks in a thread safe way
	while (CurrentTime < StartTime)
	{
		if (FTedsAssetData::ProcessAssetAddStartTime.compare_exchange_weak(StartTime, CurrentTime))
		{
			break;
		}
	}

	const int32 RowCount = Context.GetRowCount();
	TArray<FName, TInlineAllocator<32>> PackageNames;
	TArray<TPair<DataStorage::RowHandle, const FAssetData*>, TInlineAllocator<32>> RowsAndAssetData;
	PackageNames.Reserve(RowCount);
	RowsAndAssetData.Reserve(RowCount);

	FNameBuilder InternalPath;
	FNameBuilder VirtualPath;

	for (int32 Index = 0; Index < RowCount; ++Index)
	{ 
		const FAssetData& AssetData = AssetDataColumn[Index].AssetData;

		InternalPath.Reset();
		AssetData.AppendObjectPath(InternalPath);

		if (GenerateVirtualPath(InternalPath, VirtualPath))
		{
			const DataStorage::RowHandle Row = Rows[Index];

			FVirtualPathColumn_Experimental VirtualPathColumn;
			VirtualPathColumn.VirtualPath = *VirtualPath;
			// Todo investigate for a batch add Maybe?
			Context.AddColumn(Row, MoveTemp(VirtualPathColumn));

			PackageNames.Add(AssetData.PackageName);
			RowsAndAssetData.Emplace(Row, &AssetData);
		}
	}

	TArray<TOptional<FAssetPackageData>> AssetPackageDatas = AssetRegistry->GetAssetPackageDatasCopy(PackageNames);

	for (int32 Index = 0; Index < RowsAndAssetData.Num(); ++Index)
	{
		const TPair<DataStorage::RowHandle, const FAssetData*>& Pair = RowsAndAssetData[Index];
		const TOptional<FAssetPackageData>& AssetPackageData = AssetPackageDatas[Index];
		AddAssetDataColumns(Context, Pair.Key, *Pair.Value, AssetPackageData.GetPtrOrNull());
	}

	uint64 EndTime = FTedsAssetData::ProcessAssetAddEndTime.load();
	CurrentTime = FPlatformTime::Cycles64();

	while (CurrentTime > EndTime)
	{
		if (FTedsAssetData::ProcessAssetAddEndTime.compare_exchange_weak(EndTime, CurrentTime))
		{
			break;
		}
	}

	FTedsAssetData::ProcessedAddedAssetBatchSize.fetch_add(RowCount);
}

void FTedsAssetDataCBDataSource::ProcessAssetDataUpdateQueryCallback(DataStorage::IQueryContext& Context, const DataStorage::RowHandle* Rows, const FAssetDataColumn_Experimental* AssetDataColumn) const
{
	uint64 StartTime = FTedsAssetData::ProcessAssetUpdateStartTime.load();
	uint64 CurrentTime = FPlatformTime::Cycles64();

	// Processors run in chunks on a worker thread. 
	// Using compare_exchange_weak allows us to take the min start time and the max end time of all chunks in a thread safe way
	while (CurrentTime < StartTime)
	{
		if (FTedsAssetData::ProcessAssetUpdateStartTime.compare_exchange_weak(StartTime, CurrentTime))
		{
			break;
		}
	}

	const int32 RowCount = Context.GetRowCount();
	TArray<FName, TInlineAllocator<32>> PackageNames;
	PackageNames.Reserve(RowCount);

	for (int32 Index = 0; Index < RowCount; ++Index)
	{
		PackageNames.Add(AssetDataColumn[Index].AssetData.PackageName);
	}

	TArray<TOptional<FAssetPackageData>> AssetPackageDatas = AssetRegistry->GetAssetPackageDatasCopy(PackageNames);

	for (int32 Index = 0; Index < RowCount; ++Index)
	{
		AddAssetDataColumns(Context, Rows[Index], AssetDataColumn[Index].AssetData, AssetPackageDatas[Index].GetPtrOrNull());
	}

	uint64 EndTime = FTedsAssetData::ProcessAssetUpdateEndTime.load();
	CurrentTime = FPlatformTime::Cycles64();

	while (CurrentTime > EndTime)
	{
		if (FTedsAssetData::ProcessAssetUpdateEndTime.compare_exchange_weak(EndTime, CurrentTime))
		{
			break;
		}
	}

	FTedsAssetData::ProcessedUpdatedAssetBatchSize.fetch_add(RowCount);
}

void FTedsAssetDataCBDataSource::PrepopulateTagsMetadataCache()
{
	if (FTagsMetadataCache* TagsMetadata = TagsMetadataCache.Get())
	{
		TSet<FTopLevelAssetPath> ClassesPath;

		// Try to populate the tags meta cache to avoid some costly operations when doing the initial population by using the knows asset types
		TArray<TSoftClassPtr<UObject>> ClassesWithAssetDefinition = UAssetDefinitionRegistry::Get()->GetAllRegisteredAssetClasses();
		ClassesPath.Reserve(ClassesWithAssetDefinition.Num());

		TArray<UClass*> ChildrenClasses;

		for (const TSoftClassPtr<UObject>& SoftClass : ClassesWithAssetDefinition)
		{
			// Check 
			if (UClass* Class = SoftClass.Get())
			{
				if (!Class->HasAllClassFlags(EClassFlags::CLASS_Abstract))
				{
					ClassesPath.Add(Class->GetClassPathName());
				}

				ChildrenClasses.Reset();
				GetDerivedClasses(Class, ChildrenClasses);
				
				for (UClass* ChildClass : ChildrenClasses)
				{
					if (!ChildClass->HasAllClassFlags(EClassFlags::CLASS_Abstract))
					{
						ClassesPath.Add(ChildClass->GetClassPathName());
					}
				}
			}
		}

		// Also check the factories for asset types
		for (TObjectIterator<UClass> It; It; ++It)
		{
			UClass* Class = *It;
			if (!Class->IsChildOf<UFactory>() || Class->HasAnyClassFlags(CLASS_Abstract))
			{
				continue;
			}

			if (UClass* AssetClass = Class->GetDefaultObject<UFactory>()->GetSupportedClass())
			{
				ClassesPath.Add(AssetClass->GetClassPathName());
			}
		}

		TagsMetadata->BatchCacheClasses(ClassesPath);
	}
}

void FTedsAssetDataCBDataSource::FVirtualPathProcessor::ConvertInternalPathToVirtualPath(const FStringView InternalPath, FStringBuilderBase& OutVirtualPath) const
{
	OutVirtualPath.Reset();

	if (bShowAllFolder)
	{
		OutVirtualPath.Append(TEXT("/All"));
		if (InternalPath.Len() == 1 && InternalPath[0] == TEXT('/'))
		{
			return;
		}
	}

	if (bOrganizeFolders && InternalPath.Len() > 1)
	{
		const FStringView MountPoint = FPathViews::GetMountPointNameFromPath(InternalPath);
		const int32 MountPointHash = GetTypeHash(MountPoint);
		if (const FTedsAssetDataCBDataSource::FVirtualPathProcessor::FCachedPluginData* Plugin = PluginNameToCachedData.FindByHash(MountPointHash, MountPoint))
		{
			if (Plugin->LoadedFrom == EPluginLoadedFrom::Engine)
			{
				OutVirtualPath.Append(TEXT("/EngineData/Plugins"));
			}
			else
			{
				OutVirtualPath.Append(TEXT("/Plugins"));
			}

			if (!Plugin->EditorCustomVirtualPath.IsEmpty())
			{
				int32 NumCharsToCopy = Plugin->EditorCustomVirtualPath.Len();
				if (Plugin->EditorCustomVirtualPath[NumCharsToCopy - 1] == TEXT('/'))
				{
					--NumCharsToCopy;
				}

				if (NumCharsToCopy > 0)
				{
					if (Plugin->EditorCustomVirtualPath[0] != TEXT('/'))
					{
						OutVirtualPath.AppendChar(TEXT('/'));
					}

					OutVirtualPath.Append(*Plugin->EditorCustomVirtualPath, NumCharsToCopy);
				}
			}
		}
		else if (MountPoint.Equals(TEXT("Engine")))
		{
			OutVirtualPath.Append(TEXT("/EngineData"));
		}

	}

	OutVirtualPath.Append(InternalPath.GetData(), InternalPath.Len());
}

} // End of Namespace UE::Editor::AssetData::Private
