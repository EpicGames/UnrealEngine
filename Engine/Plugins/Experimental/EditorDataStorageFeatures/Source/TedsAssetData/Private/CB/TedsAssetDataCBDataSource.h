// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CB/TagsMetadataCache.h"
#include "DataStorage/Handles.h"
#include "DataStorage/CommonTypes.h"
#include "Containers/StringFwd.h"
#include "Containers/UnrealString.h"
#include "Interfaces/IPluginManager.h"

class FAssetPackageData;
class FAssetPropertyTagCache;
class IAssetRegistry;
class IPlugin;

struct FAssetData;
struct FAssetDataColumn_Experimental;
struct FAssetPathColumn_Experimental;

namespace UE::Editor::DataStorage
{
	struct IQueryContext;
	class ICoreProvider;
}

namespace UE::Editor::AssetData::Private
{

class FTedsAssetDataCBDataSource
{
public:
	explicit FTedsAssetDataCBDataSource(UE::Editor::DataStorage::ICoreProvider& InDatabase);
	~FTedsAssetDataCBDataSource();
	
	// Toggle the storage of the metadata from the asset registry into teds 
	void EnableMetadataStorage(bool bEnable);

private:
	struct FVirtualPathProcessor
	{
		struct FCachedPluginData
		{
			EPluginLoadedFrom LoadedFrom;
			FString EditorCustomVirtualPath;
		};

		TMap<FString, FCachedPluginData> PluginNameToCachedData;

		bool bShowAllFolder = false;
		bool bOrganizeFolders = false;

		void ConvertInternalPathToVirtualPath(const FStringView InternalPath, FStringBuilderBase& OutVirtualPath) const;
	};

	void InitVirtualPathProcessor();
	void PrepopulateTagsMetadataCache();


	void OnPluginContentMounted(IPlugin& InPlugin);
	void OnPluginUnmounted(IPlugin& InPlugin);

	/**
	 * Check if we should generate a virtual a path to the path and if so do generate a virtualized path for it.
	 * @param InAssetPath The internal path to virtualize
	 * @param OutVirtualizedPath The virtualized path generated if needed.
	 * @return True if the path virtual path was generated.
	 */
	bool GenerateVirtualPath(const FStringView InAssetPath, FNameBuilder& OutVirtualizedPath) const;

	void AddAssetDataColumns(DataStorage::IQueryContext& Context, DataStorage::RowHandle Row, const FAssetData& AssetData, const FAssetPackageData* OptionalPackageData) const;

	void ProcessPathQueryCallback(DataStorage::IQueryContext& Context, const DataStorage::RowHandle* Rows, const FAssetPathColumn_Experimental* PathColumn) const;
	void ProcessAssetDataPathUpdateQueryCallback(DataStorage::IQueryContext& Context, const DataStorage::RowHandle* Rows, const FAssetDataColumn_Experimental* AssetDataColumn) const;
	void ProcessAssetDataAndPathUpdateQueryCallback(DataStorage::IQueryContext& Context, const DataStorage::RowHandle* Rows, const FAssetDataColumn_Experimental* AssetDataColumn) const;
	void ProcessAssetDataUpdateQueryCallback(DataStorage::IQueryContext& Context, const DataStorage::RowHandle*Rows, const FAssetDataColumn_Experimental* AssetDataColumn) const;

	UE::Editor::DataStorage::ICoreProvider& Database;
	DataStorage::QueryHandle ProcessPathQuery;
	DataStorage::QueryHandle ProcessAssetDataPathUpdateQuery;
	DataStorage::QueryHandle ProcessAssetDataAndPathUpdateQuery;
	DataStorage::QueryHandle ProcessAssetDataUpdateQuery;
	DataStorage::QueryHandle ReprocessesAssetDataColumns;

	FVirtualPathProcessor VirtualPathProcessor; 
	bool bPopulateMetadataColumns = false;
	FName RepopulateAssetDataColumns = TEXT("RepopulateAssetDataColumnsQuery");

	TUniquePtr<FTagsMetadataCache> TagsMetadataCache;

	const IAssetRegistry* AssetRegistry = nullptr;

};

} // End of namespace UE::Editor
