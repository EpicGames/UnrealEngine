// Copyright Epic Games, Inc. All Rights Reserved.

#include "Processors/AssetProcessors.h"

#include "AssetDefinition.h"
#include "AssetDefinitionRegistry.h"
#include "CollectionManagerTypes.h"
#include "ContentBrowserDataSubsystem.h"
#include "ContentBrowserModule.h"
#include "IContentBrowserDataModule.h"
#include "IContentBrowserSingleton.h"
#include "Elements/Columns/TypedElementFolderColumns.h"
#include "Elements/Columns/TypedElementSlateWidgetColumns.h"
#include "Elements/Framework/TypedElementQueryBuilder.h"
#include "Elements/Interfaces/TypedElementQueryStorageInterfaces.h"
#include "Experimental/ContentBrowserExtensionUtils.h"
#include "TedsAssetDataColumns.h"
#include "Elements/Columns/TypedElementMiscColumns.h"
#include "String/LexFromString.h"
#include "TedsAlerts.h"
#include "TedsAssetDataModule.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AssetProcessors)

#define LOCTEXT_NAMESPACE "UTedsAssetDataFactory"

namespace UE::TedsAssetDataFactory::Private
{
	TAutoConsoleVariable<bool> CVarTedsAssetDataFactory(TEXT("TEDS.TedsAssetDataFactory"),
		true,
		TEXT("When true this will enable some experimental features that are not optimized to work at scale yet. Note: The value need to be set a boot time to see the effect of this cvar."));

	static FName UnresolvedFavoritesTable(TEXT("UnresolvedFavorites"));
}

void UTedsAssetDataFactory::RegisterTables(UE::Editor::DataStorage::ICoreProvider& DataStorage)
{
	DataStorage.RegisterTable<FUnresolvedFavoriteColumn>(UE::TedsAssetDataFactory::Private::UnresolvedFavoritesTable);
}

void UTedsAssetDataFactory::RegisterQueries(UE::Editor::DataStorage::ICoreProvider& DataStorage)
{
	using namespace UE::Editor::DataStorage;
	using namespace UE::Editor::DataStorage::Columns;
	using namespace UE::Editor::DataStorage::Queries;

	if (!UE::TedsAssetDataFactory::Private::CVarTedsAssetDataFactory.GetValueOnGameThread())
	{
		return;
	}

	DataStorage.RegisterQuery(
		Select(
			TEXT("TedsAssetDataFactory: Sync folder color from world"),
			FProcessor(EQueryTickPhase::PostPhysics, DataStorage.GetQueryTickGroupName(EQueryTickGroups::SyncExternalToDataStorage))
			.SetExecutionMode(EExecutionMode::GameThread),
			[](IQueryContext& Context, const RowHandle* Rows, const FAssetPathColumn_Experimental* AssetPathColumn, FSlateColorColumn* ColorColumn)
			{
				const int32 NumOfRowToProcess = Context.GetRowCount();

				for (int32 Index = 0; Index < NumOfRowToProcess; ++Index)
				{
					if (TOptional<FLinearColor> Color = UE::Editor::ContentBrowser::ExtensionUtils::GetFolderColor(AssetPathColumn[Index].Path))
					{
						ColorColumn[Index].Color = Color.GetValue();
					}
				}
			}
		)
		.Where()
			.All<FFolderTag, FTypedElementSyncFromWorldTag, FVirtualPathColumn_Experimental>()
		.Compile()
		);
	
	struct FSetFolderColorCommand
	{
		void operator()()
		{
			
			UE::Editor::ContentBrowser::ExtensionUtils::SetFolderColor(FolderPath, NewFolderColor);
		}
		FName FolderPath;
		FLinearColor NewFolderColor;;
	};

	DataStorage.RegisterQuery(
		Select(
			TEXT("TedsAssetDataFactory: Sync folder color back to world"),
			FProcessor(EQueryTickPhase::PrePhysics, DataStorage.GetQueryTickGroupName(EQueryTickGroups::SyncDataStorageToExternal))
			.SetExecutionMode(EExecutionMode::GameThread),
			[](IQueryContext& Context, const RowHandle* Rows, const FAssetPathColumn_Experimental* PathColumn, const FSlateColorColumn* ColorColumn)
			{
				const int32 NumOfRowToProcess = Context.GetRowCount();

				for (int32 Index = 0; Index < NumOfRowToProcess; ++Index)
				{
					FSlateColor FolderColor = ColorColumn[Index].Color.GetSpecifiedColor();

					if (FolderColor.IsColorSpecified())
					{
						// Defer the add because it will fire a CB delegate that UTedsAssetDataFactory::OnSetFolderColor registers to and ends up
						// accessing the data storage in the middle of a processor callback which is not allowed
						Context.PushCommand(FSetFolderColorCommand
						{
							.FolderPath = PathColumn[Index].Path,
							.NewFolderColor = ColorColumn[Index].Color.GetSpecifiedColor()
						});
					}
				}
			}
		)
		.Where()
			.All<FFolderTag, FTypedElementSyncBackToWorldTag, FVirtualPathColumn_Experimental>()
		.Compile()
		);

	struct FSetFolderFavoriteCommand
	{
		void operator()()
		{
			UE::Editor::ContentBrowser::ExtensionUtils::SetFolderFavorite(FolderPath, bFavorite);
		}
		FString FolderPath;
		bool bFavorite;
	};

	DataStorage.RegisterQuery(
		Select(
			TEXT("TedsAssetDataFactory: Sync favorite folder back to world"),
			FProcessor(EQueryTickPhase::PrePhysics, DataStorage.GetQueryTickGroupName(EQueryTickGroups::SyncDataStorageToExternal))
			.SetExecutionMode(EExecutionMode::GameThread),
			[](IQueryContext& Context, RowHandle Row, const FAssetPathColumn_Experimental& AssetPathColumn)
			{
				// Only do work if the state in the world doesn't match TEDS already
				if (!UE::Editor::ContentBrowser::ExtensionUtils::IsFolderFavorite(FContentBrowserItemPath(AssetPathColumn.Path.ToString(), EContentBrowserPathType::Internal)))
				{
					Context.PushCommand(FSetFolderFavoriteCommand{.FolderPath = AssetPathColumn.Path.ToString(), .bFavorite = true});
				}
			}
		)
		.Where()
			.All<FFolderTag, FTypedElementSyncBackToWorldTag, FFavoriteTag>()
		.Compile()
		);

		DataStorage.RegisterQuery(
			Select(
				TEXT("TedsAssetDataFactory: Sync non-favorite folder back to world"),
				FProcessor(EQueryTickPhase::PrePhysics, DataStorage.GetQueryTickGroupName(EQueryTickGroups::SyncDataStorageToExternal))
				.SetExecutionMode(EExecutionMode::GameThread),
				[](IQueryContext& Context, RowHandle Row, const FAssetPathColumn_Experimental& AssetPathColumn)
				{
					// Only do work if the state in the world doesn't match TEDS already
					if (UE::Editor::ContentBrowser::ExtensionUtils::IsFolderFavorite(FContentBrowserItemPath(AssetPathColumn.Path.ToString(), EContentBrowserPathType::Internal)))
					{
						Context.PushCommand(FSetFolderFavoriteCommand{.FolderPath = AssetPathColumn.Path.ToString(), .bFavorite = false});
					}
				}
			)
			.Where()
				.All<FFolderTag, FTypedElementSyncBackToWorldTag>()
				.None<FFavoriteTag>()
			.Compile()
			);

	// Query to resolve any favorites that come in before TEDS asset data has a chance to process them
	// TODO: Make Activable after the processing is complete
	DataStorage.RegisterQuery(
		Select(
			TEXT("Resolve Favorites"),
			FProcessor(EQueryTickPhase::PostPhysics, DataStorage.GetQueryTickGroupName(EQueryTickGroups::SyncExternalToDataStorage)),
			[](IQueryContext& Context, const RowHandle Row, const FUnresolvedFavoriteColumn& UnresolvedFavoriteColumn)
			{
				RowHandle AssetRow = Context.LookupMappedRow(UE::Editor::AssetData::MappingDomain, UnresolvedFavoriteColumn.MappingIndex);
				if (Context.IsRowAvailable(AssetRow))
				{
					Context.AddColumns<FTypedElementSyncFromWorldTag>(AssetRow);
					Context.AddColumns<FFavoriteTag>(AssetRow);
					Context.RemoveRow(Row);
				}
			}
		)
		.Compile()
		);
}

void UTedsAssetDataFactory::PreRegister(UE::Editor::DataStorage::ICoreProvider& DataStorage)
{
	if (!UE::TedsAssetDataFactory::Private::CVarTedsAssetDataFactory.GetValueOnGameThread())
	{
		return;
	}

	if (FContentBrowserModule* ContentBrowserModule = FModuleManager::Get().GetModulePtr<FContentBrowserModule>("ContentBrowser"))
	{
		ContentBrowserModule->GetOnSetFolderColor().AddUObject(this, &UTedsAssetDataFactory::OnSetFolderColor, &DataStorage);
	}
	
	OnFavoritesChangedDelegateHandle = IContentBrowserSingleton::Get().RegisterOnFavoritesChangedHandler(
		FOnFavoritesChanged::FDelegate::CreateUObject(this, &UTedsAssetDataFactory::OnFavoritesChanged, &DataStorage));
}

void UTedsAssetDataFactory::PostRegister(UE::Editor::DataStorage::ICoreProvider& DataStorage)
{
	if (UE::Editor::AssetData::FTedsAssetDataModule* TedsAssetDataModule = FModuleManager::Get().GetModulePtr<UE::Editor::AssetData::FTedsAssetDataModule>("TedsAssetData"))
	{
		if (TedsAssetDataModule->IsTedsAssetRegistryStorageEnabled())
		{
			OnAssetDataStorageEnabled(&DataStorage);
		}
		else
		{
			TedsAssetDataModule->OnAssetRegistryStorageInit().AddUObject(this, &UTedsAssetDataFactory::OnAssetDataStorageEnabled, &DataStorage);
		}
	}
}

void UTedsAssetDataFactory::PreShutdown(UE::Editor::DataStorage::ICoreProvider& DataStorage)
{
	if (FContentBrowserModule* ContentBrowserModule = FModuleManager::Get().GetModulePtr<FContentBrowserModule>("ContentBrowser"))
	{
		ContentBrowserModule->GetOnSetFolderColor().RemoveAll(this);
		IContentBrowserSingleton::Get().UnregisterOnFavoritesChangedDelegate(OnFavoritesChangedDelegateHandle);
	}
	
	if (UE::Editor::AssetData::FTedsAssetDataModule* TedsAssetDataModule = FModuleManager::Get().GetModulePtr<UE::Editor::AssetData::FTedsAssetDataModule>("TedsAssetData"))
	{
		TedsAssetDataModule->OnAssetRegistryStorageInit().RemoveAll(this);
	}
}

void UTedsAssetDataFactory::OnSetFolderColor(const FString& Path, UE::Editor::DataStorage::ICoreProvider* DataStorage)
{
	const UE::Editor::DataStorage::FMapKey PathKey = UE::Editor::DataStorage::FMapKey(FName(Path));
	const UE::Editor::DataStorage::RowHandle Row = DataStorage->LookupMappedRow(UE::Editor::AssetData::MappingDomain, PathKey);

	if (DataStorage->IsRowAvailable(Row))
	{
		DataStorage->AddColumn<FTypedElementSyncFromWorldTag>(Row);
	}
}

void UTedsAssetDataFactory::OnFavoritesChanged(const FContentBrowserItemPath& ItemPath, bool bAdded, UE::Editor::DataStorage::ICoreProvider* DataStorage)
{
	if (ItemPath.HasInternalPath())
	{
		const UE::Editor::DataStorage::FMapKey PathKey = UE::Editor::DataStorage::FMapKey(ItemPath.GetInternalPathName());
		const UE::Editor::DataStorage::RowHandle Row = DataStorage->LookupMappedRow(UE::Editor::AssetData::MappingDomain, PathKey);

		if (DataStorage->IsRowAvailable(Row))
		{
			// Since we are adding the sync tag which could end up doing other expensive processing, only add it when the data actually changed
			// from what TEDS has
			if (bAdded)
			{
				if (!DataStorage->HasColumns<FFavoriteTag>(Row))
				{
					DataStorage->AddColumn<FTypedElementSyncFromWorldTag>(Row);
					DataStorage->AddColumn<FFavoriteTag>(Row);
				}
			}
			else
			{
				if (DataStorage->HasColumns<FFavoriteTag>(Row))
				{
					DataStorage->AddColumn<FTypedElementSyncFromWorldTag>(Row);
					DataStorage->RemoveColumn<FFavoriteTag>(Row);
				}
			}
		}
		else if (bAdded) // if it's being removed and doesn't exist we don't have anything to do
		{
			UE::Editor::DataStorage::RowHandle TempRow = DataStorage->AddRow(
				DataStorage->FindTable(UE::TedsAssetDataFactory::Private::UnresolvedFavoritesTable));

			if (FUnresolvedFavoriteColumn* Column = DataStorage->GetColumn<FUnresolvedFavoriteColumn>(TempRow))
			{
				Column->MappingIndex = PathKey;
			}
		}
	}
	
}

void UTedsAssetDataFactory::OnAssetDataStorageEnabled(UE::Editor::DataStorage::ICoreProvider* DataStorage)
{
	IContentBrowserSingleton::Get().ForEachFavoriteFolder([DataStorage, this](const FContentBrowserItemPath& Path)
	{
		OnFavoritesChanged(Path, true, DataStorage);
	});
}

#undef LOCTEXT_NAMESPACE
