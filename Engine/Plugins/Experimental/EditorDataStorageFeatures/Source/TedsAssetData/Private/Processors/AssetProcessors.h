// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataStorage/CommonTypes.h"
#include "DataStorage/MapKey.h"
#include "Elements/Interfaces/TypedElementDataStorageFactory.h"

#include "AssetProcessors.generated.h"

struct FContentBrowserItemPath;

namespace UE::Editor::DataStorage
{
	class ICoreProvider;
} // namespace UE::Editor::DataStorage

/**
 * Temp tag added to a row when it is a favorite before TEDS asset data has processed it
 */
USTRUCT(meta = (DisplayName = "Unresolved Favorite"))
struct FUnresolvedFavoriteColumn final : public FEditorDataStorageColumn
{
	GENERATED_BODY()

	UE::Editor::DataStorage::FMapKey MappingIndex;
};

UCLASS()
class UTedsAssetDataFactory : public UEditorDataStorageFactory
{
	GENERATED_BODY()

public:
	~UTedsAssetDataFactory() override = default;

	virtual void RegisterTables(UE::Editor::DataStorage::ICoreProvider& DataStorage) override;
	void RegisterQueries(UE::Editor::DataStorage::ICoreProvider& DataStorage) override;
	virtual void PreRegister(UE::Editor::DataStorage::ICoreProvider& DataStorage) override;
	virtual void PostRegister(UE::Editor::DataStorage::ICoreProvider& DataStorage) override;
	virtual void PreShutdown(UE::Editor::DataStorage::ICoreProvider& DataStorage) override;

protected:

	void OnSetFolderColor(const FString& Path, UE::Editor::DataStorage::ICoreProvider* DataStorage);
	void OnFavoritesChanged(const FContentBrowserItemPath& ItemPath, bool bAdded, UE::Editor::DataStorage::ICoreProvider* DataStorage);
	void OnAssetDataStorageEnabled(UE::Editor::DataStorage::ICoreProvider* DataStorage);

protected:
	FDelegateHandle OnFavoritesChangedDelegateHandle;
};
