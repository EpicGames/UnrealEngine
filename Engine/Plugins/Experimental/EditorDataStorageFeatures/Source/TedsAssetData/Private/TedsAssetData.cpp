// Copyright Epic Games, Inc. All Rights Reserved.

#include "TedsAssetData.h"

#include "TedsAssetDataColumns.h"

#include "AssetRegistry/IAssetRegistry.h"
#include "Async/ParallelFor.h"
#include "Containers/Array.h"
#include "Containers/ChunkedArray.h"
#include "Containers/Queue.h"
#include "DataStorage/Queries/Types.h"
#include "Elements/Columns/TypedElementFolderColumns.h"
#include "Elements/Columns/TypedElementSlateWidgetColumns.h"
#include "Elements/Framework/TypedElementQueryBuilder.h"
#include "Elements/Interfaces/TypedElementDataStorageInterface.h"
#include "Engine/Engine.h"
#include "DataStorage/Debug/Log.h"

#include "ContentBrowserDataUtils.h"

#include "ProfilingDebugging/CountersTrace.h"

#include <limits>

#define TRACK_TEDSASSETDATA_MEMORY 0

#if COUNTERSTRACE_ENABLED

DECLARE_STATS_GROUP(TEXT("Teds Asset Data"), STATGROUP_TedsAssetData, STATCAT_Advanced);

DECLARE_DWORD_COUNTER_STAT(TEXT("Path Adds Remaining"), STAT_TedsAssetData_PathAddsRemaining, STATGROUP_TedsAssetData);
DECLARE_DWORD_COUNTER_STAT(TEXT("Asset Adds Remaining"), STAT_TedsAssetData_AssetAddsRemaining, STATGROUP_TedsAssetData);
DECLARE_DWORD_COUNTER_STAT(TEXT("Asset Updates Remaining"), STAT_TedsAssetData_AssetUpdatesRemaining, STATGROUP_TedsAssetData);

DECLARE_FLOAT_COUNTER_STAT(TEXT("Path Add Avg Time"), STAT_TedsAssetData_PathAddAvgTime, STATGROUP_TedsAssetData);
DECLARE_FLOAT_COUNTER_STAT(TEXT("Asset Add Avg Time"), STAT_TedsAssetData_AssetAddAvgTime, STATGROUP_TedsAssetData);
DECLARE_FLOAT_COUNTER_STAT(TEXT("Asset Update Avg Time"), STAT_TedsAssetData_AssetUpdateAvgTime, STATGROUP_TedsAssetData);

DECLARE_DWORD_COUNTER_STAT(TEXT("Path Add Batch Size"), STAT_TedsAssetData_PathAddBatchSize, STATGROUP_TedsAssetData);
DECLARE_DWORD_COUNTER_STAT(TEXT("Asset Add Batch Size"), STAT_TedsAssetData_AssetAddBatchSize, STATGROUP_TedsAssetData);
DECLARE_DWORD_COUNTER_STAT(TEXT("Asset Update Batch Size"), STAT_TedsAssetData_AssetUpdateBatchSize, STATGROUP_TedsAssetData);

DECLARE_DWORD_COUNTER_STAT(TEXT("Path Add Batch Size Received"), STAT_TedsAssetData_PathAddBatchSizeReceived, STATGROUP_TedsAssetData);
DECLARE_DWORD_COUNTER_STAT(TEXT("Asset Add Batch Size Received"), STAT_TedsAssetData_AssetAddBatchSizeReceived, STATGROUP_TedsAssetData);
DECLARE_DWORD_COUNTER_STAT(TEXT("Asset Update Batch Size Received"), STAT_TedsAssetData_AssetUpdateBatchSizeReceived, STATGROUP_TedsAssetData);

DECLARE_FLOAT_COUNTER_STAT(TEXT("Path Add Batch Last Duration"), STAT_TedsAssetData_PathAddBatchLastDuration, STATGROUP_TedsAssetData);
DECLARE_FLOAT_COUNTER_STAT(TEXT("Asset Add Batch Last Duration"), STAT_TedsAssetData_AssetAddBatchLastDuration, STATGROUP_TedsAssetData);
DECLARE_FLOAT_COUNTER_STAT(TEXT("Asset Update Batch Last Duration"), STAT_TedsAssetData_AssetUpdateBatchLastDuration, STATGROUP_TedsAssetData);

TRACE_DECLARE_INT_COUNTER(COUNTER_TedsAssetData_PathAddsRemaining, TEXT("TedsAssetData/PathAdd/Remaining"));
TRACE_DECLARE_INT_COUNTER(COUNTER_TedsAssetData_AssetAddsRemaining, TEXT("TedsAssetData/AssetAdd/Remaining"));
TRACE_DECLARE_INT_COUNTER(COUNTER_TedsAssetData_AssetUpdatesRemaining, TEXT("TedsAssetData/AssetUpdate/Remaining"));

TRACE_DECLARE_FLOAT_COUNTER(COUNTER_TedsAssetData_PathAddAvgTime, TEXT("TedsAssetData/PathAdd/AvgTime"));
TRACE_DECLARE_FLOAT_COUNTER(COUNTER_TedsAssetData_AssetAddAvgTime, TEXT("TedsAssetData/AssetAdd/AvgTime"));
TRACE_DECLARE_FLOAT_COUNTER(COUNTER_TedsAssetData_AssetUpdateAvgTime, TEXT("TedsAssetData/AssetUpdate/AvgTime"));

TRACE_DECLARE_INT_COUNTER(COUNTER_TedsAssetData_PathAddBatchSize, TEXT("TedsAssetData/PathAdd/Batch/Size"));
TRACE_DECLARE_INT_COUNTER(COUNTER_TedsAssetData_AssetAddBatchSize, TEXT("TedsAssetData/AssetAdd/Batch/Size"));
TRACE_DECLARE_INT_COUNTER(COUNTER_TedsAssetData_AssetUpdateBatchSize, TEXT("TedsAssetData/AssetUpdate/Batch/Size"));

TRACE_DECLARE_INT_COUNTER(COUNTER_TedsAssetData_PathAddBatchSizeReceived, TEXT("TedsAssetData/PathAdd/Batch/Size/Received"));
TRACE_DECLARE_INT_COUNTER(COUNTER_TedsAssetData_AssetAddBatchSizeReceived, TEXT("TedsAssetData/AssetAdd/Batch/Size/Received"));
TRACE_DECLARE_INT_COUNTER(COUNTER_TedsAssetData_AssetUpdateBatchSizeReceived, TEXT("TedsAssetData/AssetUpdate/Batch/Size/Received"));

TRACE_DECLARE_FLOAT_COUNTER(COUNTER_TedsAssetData_PathAddBatchLastDuration, TEXT("TedsAssetData/PathAdd/Batch/LastDuration"));
TRACE_DECLARE_FLOAT_COUNTER(COUNTER_TedsAssetData_AssetAddBatchLastDuration, TEXT("TedsAssetData/AssetAdd/Batch/LastDuration"));
TRACE_DECLARE_FLOAT_COUNTER(COUNTER_TedsAssetData_AssetUpdateBatchLastDuration, TEXT("TedsAssetData/AssetUpdate/Batch/LastDuration"));

DECLARE_FLOAT_COUNTER_STAT(TEXT("Total Asset Data processing time"), STAT_TedsAssetData_TotalProcessingTime, STATGROUP_TedsAssetData);

TRACE_DECLARE_FLOAT_COUNTER(COUNTER_TedsAssetData_TotalProcessingTime, TEXT("TedsAssetData/TotalProcessingTime"));

#endif

namespace UE::Editor::AssetData::Private
{

std::atomic<uint32> FTedsAssetData::ProcessedAddedPathBatchSize = 0;
std::atomic<uint32> FTedsAssetData::ProcessedAddedAssetBatchSize = 0;
std::atomic<uint32> FTedsAssetData::ProcessedUpdatedAssetBatchSize = 0;

std::atomic<uint64> FTedsAssetData::ProcessPathAddStartTime = std::numeric_limits<uint64>::max();
std::atomic<uint64> FTedsAssetData::ProcessPathAddEndTime = 0;

std::atomic<uint64> FTedsAssetData::ProcessAssetAddStartTime = std::numeric_limits<uint64>::max();
std::atomic<uint64> FTedsAssetData::ProcessAssetAddEndTime = 0;

std::atomic<uint64> FTedsAssetData::ProcessAssetUpdateStartTime = std::numeric_limits<uint64>::max();
std::atomic<uint64> FTedsAssetData::ProcessAssetUpdateEndTime = 0;

using namespace DataStorage;

constexpr int32 ParallelForMinBatchSize = 1024 * 4;

struct FPopulateAssetDataRowArgs
{
	FAssetData AssetData;
	FMapKey ObjectPathKey;
};

// Only safe if the GT is blocked during the operation
template<typename TAssetData>
FPopulateAssetDataRowArgs ThreadSafe_PopulateAssetDataTableRow(TAssetData&& InAssetData, const ICoreProvider& Database)
{
	FPopulateAssetDataRowArgs Output;
	Output.ObjectPathKey = FMapKey(InAssetData.GetSoftObjectPath());

	if (Database.IsRowAssigned(Database.LookupMappedRow(MappingDomain, Output.ObjectPathKey)))
	{
		// No need to initialize the rest of the row here. The invalid's asset data will be used as flag to skip the data generated here.
		return Output;
	}

	Output.AssetData = Forward<TAssetData>(InAssetData);

	return Output;
}

void PopulateAssetDataTableRow(FPopulateAssetDataRowArgs&& InAssetDataRowArgs, ICoreProvider& Database, RowHandle RowHandle)
{
	Database.GetColumn<FAssetDataColumn_Experimental>(RowHandle)->AssetData = MoveTemp(InAssetDataRowArgs.AssetData);
}

struct FPopulatePathRowArgs
{
	FName AssetRegistryPath;
	FMapKey AssetRegistryPathKey;
	FName AssetName;

	operator bool() const 
	{
		return !AssetRegistryPath.IsNone();
	}

	void MarkAsInvalid()
	{
		AssetRegistryPath = FName();
	}
};

void GetParentFolderIndex(FStringView Path, int32& OutParentFolderIndex)
{
	OutParentFolderIndex = INDEX_NONE;

	if (Path.Len() > 1)
	{
		OutParentFolderIndex = 1;
	}

	// Skip the first '/'
	for (int32 Index = 1; Index < Path.Len(); ++Index)
	{
		if (Path[Index] == TEXT('/'))
		{
			OutParentFolderIndex = Index;
		}
	}
}

// Only thread safe if the game thread is blocked 
FPopulatePathRowArgs ThreadSafe_PopulatePathRowArgs(FMapKey&& AssetRegistryPathKey, FName InAssetRegistryPath, FStringView PathAsString)
{
	int32 CharacterIndex;
	FStringView FolderName;
	GetParentFolderIndex(PathAsString, CharacterIndex);
	if (CharacterIndex != INDEX_NONE)
	{
		FolderName = PathAsString.RightChop(CharacterIndex);
	}

	FPopulatePathRowArgs Args;
	Args.AssetRegistryPath = InAssetRegistryPath;
	Args.AssetName = FName(FolderName);
	Args.AssetRegistryPathKey = MoveTemp(AssetRegistryPathKey);

	return Args;
}

void PopulatePathDataTableRow(FPopulatePathRowArgs&& InPopulatePathRowArgs, ICoreProvider& Database, RowHandle InRowHandle)
{
	Database.GetColumn<FAssetPathColumn_Experimental>(InRowHandle)->Path = InPopulatePathRowArgs.AssetRegistryPath;
	Database.GetColumn<FAssetNameColumn>(InRowHandle)->Name = InPopulatePathRowArgs.AssetName;
}


FTedsAssetData::FTedsAssetData(UE::Editor::DataStorage::ICoreProvider& InDatabase)
	: Database(InDatabase)
{
	IAssetRegistry& AssetRegistry = IAssetRegistry::GetChecked();

	AssetRegistry.OnScanStarted().AddRaw(this, &FTedsAssetData::OnARScanStarted);
	AssetRegistry.OnScanEnded().AddRaw(this, &FTedsAssetData::OnARScanEnded);

	// Start processing immediately if Asset Registry is not gathering and has not completed its initial scan
	// Otherwise wait for OnScanEnded
	if (AssetRegistry.IsSearchAllAssets() && !AssetRegistry.IsGathering())
	{
		StartProcessing();
	}
}

FTedsAssetData::~FTedsAssetData()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FTedsAssetData::~FTedsAssetData);

#if TRACK_TEDSASSETDATA_MEMORY
	LLM_SCOPE_BYNAME(TEXT("FTedsAssetData"))
#endif

	// Not needed on a editor shut down
	if (!IsEngineExitRequested())
	{
		Database.UnregisterCooperativeUpdate(TEXT("ProcessAssetTasksCooperatively"));

		Database.OnUpdate().RemoveAll(this);

		if (IAssetRegistry* AssetRegistry = IAssetRegistry::Get())
		{
			Database.UnregisterQuery(RemoveUpdatedAssetDataTagQuery);
			Database.UnregisterQuery(RemoveUpdatedPathTagQuery);

			AssetRegistry->OnScanStarted().RemoveAll(this);
			AssetRegistry->OnScanEnded().RemoveAll(this);
			AssetRegistry->OnAssetsAdded().RemoveAll(this);
			AssetRegistry->OnAssetsRemoved().RemoveAll(this);
			AssetRegistry->OnAssetsUpdated().RemoveAll(this);
			AssetRegistry->OnAssetsUpdatedOnDisk().RemoveAll(this);
			AssetRegistry->OnAssetRenamed().RemoveAll(this);
			AssetRegistry->OnPathsAdded().RemoveAll(this);
			AssetRegistry->OnPathsRemoved().RemoveAll(this);
	
			AssetRegistry->EnumerateAllCachedPaths([this](FName InPath)
				{
					const FMapKeyView PathKey = FMapKeyView(InPath);
					const RowHandle Row = Database.LookupMappedRow(MappingDomain, PathKey);
					Database.RemoveRow(Row);
					return true;
				});

			AssetRegistry->EnumerateAllAssets([this](const FAssetData& InAssetData)
				{
					const FMapKey AssetPathKey = FMapKey(InAssetData.GetSoftObjectPath());
					const RowHandle Row = Database.LookupMappedRow(MappingDomain, AssetPathKey);
					Database.RemoveRow(Row);
					return true;
				});
		}
	}
}

void FTedsAssetData::Initialize()
{
	using namespace UE::Editor::DataStorage::Queries;
	TRACE_CPUPROFILER_EVENT_SCOPE(FTedsAssetData::Initialize);

#if TRACK_TEDSASSETDATA_MEMORY
	LLM_SCOPE_BYNAME(TEXT("FTedsAssetData"))
#endif

	ProcessingStartTime = FPlatformTime::Cycles64();

	// Register to events from asset registry
	IAssetRegistry& AssetRegistry = IAssetRegistry::GetChecked();

	AssetRegistry.OnAssetsAdded().AddRaw(this, &FTedsAssetData::OnAssetsAdded);
	AssetRegistry.OnAssetsRemoved().AddRaw(this, &FTedsAssetData::OnAssetsRemoved);
	AssetRegistry.OnAssetsUpdated().AddRaw(this, &FTedsAssetData::OnAssetsUpdated);
	AssetRegistry.OnAssetRenamed().AddRaw(this, &FTedsAssetData::OnAssetRenamed);
	AssetRegistry.OnAssetsUpdatedOnDisk().AddRaw(this, &FTedsAssetData::OnAssetsUpdatedOnDisk);
	AssetRegistry.OnPathsAdded().AddRaw(this, &FTedsAssetData::OnPathsAdded);
	AssetRegistry.OnPathsRemoved().AddRaw(this, &FTedsAssetData::OnPathsRemoved);

	Database.OnUpdate().AddRaw(this, &FTedsAssetData::OnDatabaseUpdate);

	Database.RegisterCooperativeUpdate(TEXT("ProcessAssetTasksCooperatively"), DataStorage::ICoreProvider::ECooperativeTaskPriority::High,
		[this](FTimespan TimeAllowance)
		{
			ProcessTaskQueues(TimeAllowance);
		});

	// Register data types to TEDS
	PathsTable = Database.FindTable(FName(TEXT("Editor_AssetRegistryPathsTable")));
	if (PathsTable == InvalidTableHandle)
	{
		PathsTable = Database.RegisterTable<FFolderTag, FAssetPathColumn_Experimental, FAssetNameColumn, FUpdatedPathTag>(FName(TEXT("Editor_AssetRegistryPathsTable")));
	}

	AssetsDataTable = Database.FindTable(FName(TEXT("Editor_AssetRegistryAssetDataTable")));
	if (AssetsDataTable == InvalidTableHandle)
	{
		AssetsDataTable = Database.RegisterTable<FAssetDataColumn_Experimental, FUpdatedPathTag, FUpdatedAssetDataTag>(FName(TEXT("Editor_AssetRegistryAssetDataTable")));
	}

	RemoveUpdatedPathTagQuery = Database.RegisterQuery(
		Select(
			TEXT("FTedsAssetData: Remove Updated Path Tag"),
			FPhaseAmble(FPhaseAmble::ELocation::Postamble, EQueryTickPhase::FrameEnd),
			[](IQueryContext& Context, const RowHandle* Rows)
			{
				Context.RemoveColumns<FUpdatedPathTag>(TConstArrayView<RowHandle>(Rows, Context.GetRowCount()));
			})
		.Where()
		.All<FUpdatedPathTag>()
		.Compile());

	RemoveUpdatedAssetDataTagQuery = Database.RegisterQuery(
		Select(
			TEXT("FTedsAssetData: Remove Updated Asset Data Tag"),
			FPhaseAmble(FPhaseAmble::ELocation::Postamble, EQueryTickPhase::FrameEnd),
			[](IQueryContext& Context, const RowHandle* Rows)
			{
				Context.RemoveColumns<FUpdatedAssetDataTag>(TConstArrayView<RowHandle>(Rows, Context.GetRowCount()));
			})
		.Where()
		.All<FUpdatedAssetDataTag>()
		.Compile());

	// Init with the data existing at moment in asset registry
	UE::AssetRegistry::FFiltering::InitializeShouldSkipAsset();
	TQueue<FAssetData, EQueueMode::Mpsc> CachedAssets;
	std::atomic<int32> CachedAssetsCount = 0;
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FTedsAssetData::Initialize_EnumerateAllAssets);
		AssetRegistry.EnumerateAllAssets([&CachedAssets, &CachedAssetsCount](const FAssetData& AssetData)
			{
				if (!UE::AssetRegistry::FFiltering::ShouldSkipAsset(AssetData.AssetClassPath, AssetData.PackageFlags) &&
					ContentBrowserDataUtils::IsPrimaryAsset(AssetData))
				{
					CachedAssets.Enqueue(AssetData);
					++CachedAssetsCount;
				}
				return true;
			}, UE::AssetRegistry::EEnumerateAssetsFlags::Parallel);
	}


	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FTedsAssetData::Initialize_DrainQueue);
		AddedAssets.Reserve(AddedAssets.Num() + CachedAssetsCount);
		FAssetData AssetDataTemp;
		while (CachedAssets.Dequeue(AssetDataTemp))
		{
			AddedAssets.Add(MoveTemp(AssetDataTemp));
		}
	}


	TChunkedArray<FName> CachedPaths;
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FTedsAssetData::Initialize_EnumerateAllCachedPaths);
		AssetRegistry.EnumerateAllCachedPaths([&CachedPaths](FName Name)
			{
				CachedPaths.AddElement(Name);
				return true;
			});
	}

	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FTedsAssetData::Initialize_CachedPaths.MoveToLinearArray);
		CachedPaths.MoveToLinearArray(AddedPaths);
	}
	

	bIsInitialized = true;
}

void FTedsAssetData::StartProcessing()
{
	if (!bIsInitialized)
	{
		// Do initial bulk asset grab and register to Asset Registry events
		Initialize();
	}
	bIsPaused = false;
}

void FTedsAssetData::PauseProcessing()
{
	bIsPaused = true;
}

void FTedsAssetData::ProcessAllEvents()
{
	if (IAssetRegistry* AssetRegistry = IAssetRegistry::Get())
	{
		AssetRegistry->Tick(-1.f);
	}
}

void FTedsAssetData::OnARScanStarted()
{
	PauseProcessing();
}

void FTedsAssetData::OnARScanEnded()
{
	StartProcessing();
}

void FTedsAssetData::OnAssetsAdded(TConstArrayView<FAssetData> InAssetsAdded)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FTedsAssetData::OnAssetsAdded);

	AddedAssets.Reserve(AddedAssets.Num() + InAssetsAdded.Num());

	UE::AssetRegistry::FFiltering::InitializeShouldSkipAsset();
	for (const FAssetData& AssetData : InAssetsAdded)
	{
		if (!UE::AssetRegistry::FFiltering::ShouldSkipAsset(AssetData.AssetClassPath, AssetData.PackageFlags) &&
			ContentBrowserDataUtils::IsPrimaryAsset(AssetData))
		{
			AddedAssets.Emplace(AssetData);
		}
	}
}

FTimespan FTedsAssetData::ProcessAddedAssets(const FTimespan& AllowedTime)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FTedsAssetData::ProcessAddedAssets);

#if TRACK_TEDSASSETDATA_MEMORY
	LLM_SCOPE_BYNAME(TEXT("FTedsAssetData"))
#endif

	if(AddedAssets.IsEmpty())
	{
		LastAddedAssetBatchSize = 0;
		return AllowedTime;
	}

	int32 BatchSize = static_cast<int32>((AllowedTime / AvgTimePerProcessedAssetAdd).GetTicks());

	// Regardless of AvgTime always process at least the minimum batch size
	uint32 BatchMin = FMath::Min(MinimumBatchSize, AddedAssets.Num());
	BatchSize = static_cast<int32>(FMath::Clamp(BatchSize, BatchMin, AddedAssets.Num()));

	uint32 BatchStartIndex = AddedAssets.Num() - BatchSize;

	TConstArrayView<FAssetData> BatchView(&AddedAssets[BatchStartIndex], BatchSize);
	TArray<FAssetData> AssetBatch = TSet<FAssetData>(BatchView).Array(); // TODO: Profile set dedupe perf and explore other options such as sorting the main queue

	TArray<int32> Contexts;
	TArray<FPopulateAssetDataRowArgs> PopulateRowArgs;
	PopulateRowArgs.AddDefaulted(AssetBatch.Num());

	ParallelForWithTaskContext(TEXT("Populating TEDS Asset Registry Asset Data"), Contexts, AssetBatch.Num(),
		ParallelForMinBatchSize, [&PopulateRowArgs, &AssetBatch, this](int32& ValidAssetCount, int32 Index)
		{
			FPopulateAssetDataRowArgs RowArgs;
			const FAssetData& AssetData = AssetBatch[Index];

			RowArgs = ThreadSafe_PopulateAssetDataTableRow(AssetData, Database);
			if (RowArgs.AssetData.IsValid())
			{
				++ValidAssetCount;
			}
			PopulateRowArgs[Index] = MoveTemp(RowArgs);
		});


	int32 NewRowsCount = 0;
	for (int32 Context : Contexts)
	{
		NewRowsCount += Context;
	}

	int32 Index = 0;
	if (NewRowsCount > 0)
	{
		TArray<TPair<FMapKey, RowHandle>> KeyToRow;
		KeyToRow.Reserve(NewRowsCount);
		Database.BatchAddRow(AssetsDataTable, NewRowsCount, [RowArgs = MoveTemp(PopulateRowArgs), Index = 0, &KeyToRow, this](RowHandle InRowHandle) mutable
			{
				FPopulateAssetDataRowArgs ARowArgs = MoveTemp(RowArgs[Index]);
				while (!ARowArgs.AssetData.IsValid())
				{
					++Index;
					ARowArgs = MoveTemp(RowArgs[Index]);
				}

				KeyToRow.Emplace(MoveTemp(ARowArgs.ObjectPathKey), InRowHandle);
				PopulateAssetDataTableRow(MoveTemp(ARowArgs), Database, InRowHandle);
				++Index;
			});

		Database.BatchMapRows(MappingDomain, KeyToRow);
	}

	// Cut the consumed batch off of the back of our queue
	AddedAssets.RemoveAt(BatchStartIndex, BatchSize);

	FTimespan ConsumedTime(static_cast<int64>(AvgTimePerProcessedAssetAdd * BatchSize));
	LastAddedAssetBatchSize += BatchSize; // Accumulate total batch size since we can be called multiple times per tick
	return AllowedTime - ConsumedTime;
}

void FTedsAssetData::OnAssetsRemoved(TConstArrayView<FAssetData> InAssetsRemoved)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FTedsAssetData::OnAssetsRemoved);

#if TRACK_TEDSASSETDATA_MEMORY
	LLM_SCOPE_BYNAME(TEXT("FTedsAssetData"))
#endif

	TArray<RowHandle> RowsToRemove;
	RowsToRemove.Reserve(InAssetsRemoved.Num());

	for (const FAssetData& Asset : InAssetsRemoved)
	{
		const FMapKey AssetKey = FMapKey(Asset.GetSoftObjectPath());
		const RowHandle AssetRow = Database.LookupMappedRow(MappingDomain, AssetKey);
		if (Database.IsRowAssigned(AssetRow))
		{
			RowsToRemove.Add(AssetRow);
		}
		AddedAssets.Remove(Asset);
		UpdatedAssets.Remove(Asset);
	}

	Database.BatchRemoveRows(RowsToRemove);
}

void FTedsAssetData::OnAssetsUpdated(TConstArrayView<FAssetData> InAssetsUpdated)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FTedsAssetData::OnAssetsUpdated);

	OnAssetsUpdated_Impl(InAssetsUpdated);
}

void FTedsAssetData::OnAssetsUpdatedOnDisk(TConstArrayView<FAssetData> InAssetsUpdated)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FTedsAssetData::OnAssetsUpdatedOnDisk);

	OnAssetsUpdated_Impl(InAssetsUpdated);
}

void FTedsAssetData::OnAssetsUpdated_Impl(TConstArrayView<FAssetData> InAssetsUpdated)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FTedsAssetData::OnAssetsUpdated_Impl);

	UpdatedAssets.Append(InAssetsUpdated.GetData(), InAssetsUpdated.Num());
}

FTimespan FTedsAssetData::ProcessUpdatedAssets(const FTimespan& AllowedTime)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FTedsAssetData::ProcessUpdatedAssets);

#if TRACK_TEDSASSETDATA_MEMORY
	LLM_SCOPE_BYNAME(TEXT("FTedsAssetData"))
#endif

	if (UpdatedAssets.IsEmpty())
	{
		LastUpdatedAssetBatchSize = 0;
		return AllowedTime;
	}

	int32 BatchSize = static_cast<int32>((AllowedTime / AvgTimePerProcessedAssetUpdate).GetTicks());

	// Regardless of AvgTime always process at least the minimum batch size
	uint32 BatchMin = FMath::Min(MinimumBatchSize, UpdatedAssets.Num());
	BatchSize = static_cast<int32>(FMath::Clamp(BatchSize, BatchMin, UpdatedAssets.Num()));

	uint32 BatchStartIndex = UpdatedAssets.Num() - BatchSize;

	TConstArrayView<FAssetData> AssetBatch(&UpdatedAssets[BatchStartIndex], BatchSize);

	for (const FAssetData& Asset : AssetBatch)
	{
		const FMapKey AssetKey = FMapKey(Asset.GetSoftObjectPath());
		const RowHandle Row = Database.LookupMappedRow(MappingDomain, AssetKey);
		if (Database.IsRowAssigned(Row))
		{
			Database.GetColumn<FAssetDataColumn_Experimental>(Row)->AssetData = Asset;
			Database.AddColumn<FUpdatedAssetDataTag>(Row);
		}
	}

	// Cut the consumed batch off of the back of our queue
	UpdatedAssets.RemoveAt(BatchStartIndex, BatchSize);

	FTimespan ConsumedTime(static_cast<int64>(AvgTimePerProcessedAssetUpdate * BatchSize));
	LastUpdatedAssetBatchSize += BatchSize; // Accumulate total batch size since we can be called multiple times per tick
	return AllowedTime - ConsumedTime;
}

void FTedsAssetData::OnAssetRenamed(const FAssetData& InAsset, const FString& InOldObjectPath)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FTedsAssetData::OnAssetRenamed);

#if TRACK_TEDSASSETDATA_MEMORY
	LLM_SCOPE_BYNAME(TEXT("FTedsAssetData"))
#endif

	using namespace UE::Editor::DataStorage;

	FMapKey NewAssetHash = FMapKey(InAsset.GetSoftObjectPath());
	const FMapKey OldAssetHash = FMapKey(FSoftObjectPath(InOldObjectPath));
	const RowHandle Row = Database.LookupMappedRow(MappingDomain, OldAssetHash);
	if (Database.IsRowAssigned(Row))
	{
		Database.GetColumn<FAssetDataColumn_Experimental>(Row)->AssetData = InAsset;

		// Update the asset in folder columns
		const FMapKeyView NewFolderHash(InAsset.PackagePath);
		FStringView OldPackagePath(InOldObjectPath);

		{
			int32 CharacterIndex = 0;
			OldPackagePath.FindLastChar(TEXT('/'), CharacterIndex);
			OldPackagePath.LeftInline(CharacterIndex);
		}

		const FMapKey OldFolderHash = FMapKey(FName(OldPackagePath));

		Database.AddColumn<FUpdatedPathTag>(Row);
		Database.RemapRow(MappingDomain, OldAssetHash, MoveTemp(NewAssetHash));
	}
}

void FTedsAssetData::OnPathsAdded(TConstArrayView<FStringView> InPathsAdded)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FTedsAssetData::OnPathsAdded);

	int32 AddedPathsSize = AddedPaths.Num();
	AddedPaths.AddDefaulted(InPathsAdded.Num());

	// Would normally append InPathsAdded to AddedPaths, but we need to convert the FStringView to FName
	ParallelFor(TEXT("Staging TEDS Asset Registry Path"), InPathsAdded.Num(), ParallelForMinBatchSize, [&InPathsAdded, AddedPathsSize, this](int32 Index)
		{
			AddedPaths[AddedPathsSize + Index] = FName(InPathsAdded[Index]);
		});
}

FTimespan FTedsAssetData::ProcessAddedPaths(const FTimespan& AllowedTime)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FTedsAssetData::ProcessAddedPaths);

#if TRACK_TEDSASSETDATA_MEMORY
	LLM_SCOPE_BYNAME(TEXT("FTedsAssetData"));
#endif

	if (AddedPaths.IsEmpty())
	{
		LastAddedPathBatchSize = 0;
		return AllowedTime;
	}

	int32 BatchSize = static_cast<int32>((AllowedTime / AvgTimePerProcessedPathAdd).GetTicks());

	// Regardless of AvgTime always process at least the minimum batch size
	uint32 BatchMin = FMath::Min(MinimumBatchSize, AddedPaths.Num());
	BatchSize = static_cast<int32>(FMath::Clamp(BatchSize, BatchMin, AddedPaths.Num()));

	uint32 BatchStartIndex = AddedPaths.Num() - BatchSize;

	TConstArrayView<FName> PathView(&AddedPaths[BatchStartIndex], BatchSize); // TODO: Profile set dedupe perf and explore other options such as sorting the main queue
	TArray<FName> PathBatch(TSet<FName>(PathView).Array());

	TArray<int32> PreparePathRowContexts;
	TArray<FPopulatePathRowArgs> PopulateRowArgs;
	PopulateRowArgs.AddDefaulted(PathBatch.Num());

	ParallelForWithTaskContext(TEXT("Populating TEDS Asset Registry Path"), PreparePathRowContexts, PathBatch.Num(), ParallelForMinBatchSize, [&PopulateRowArgs, &PathBatch, this](int32& WorkerValidCount, int32 Index)
		{
			FName PathName = PathBatch[Index];
			FNameBuilder Path;
			PathName.ToString(Path);

			FMapKey AssetRegistryPathKey = FMapKey(PathName);
			FPopulatePathRowArgs RowArgs;

			if (Database.LookupMappedRow(MappingDomain, AssetRegistryPathKey) != InvalidRowHandle)
			{
				RowArgs.MarkAsInvalid();
			}
			else
			{
				RowArgs = ThreadSafe_PopulatePathRowArgs(MoveTemp(AssetRegistryPathKey), PathName, Path);
				++WorkerValidCount;
			}

			PopulateRowArgs[Index] = MoveTemp(RowArgs);
		});

	int32 NewRowsCount = 0;
	for (const int32& Context : PreparePathRowContexts)
	{
		NewRowsCount += Context;
	}

	if (NewRowsCount > 0)
	{
		TArray<RowHandle> ReservedRow;
		ReservedRow.AddUninitialized(NewRowsCount);
		Database.BatchReserveRows(ReservedRow);

		TArray<TPair<FMapKey, RowHandle>> KeysAndRows;
		KeysAndRows.Reserve(NewRowsCount);

		int32 RowCount = 0;
		for (int32 Index = 0; Index < NewRowsCount; ++Index)
		{
			const FPopulatePathRowArgs* RowArgs = &PopulateRowArgs[RowCount];
			while (!(*RowArgs))
			{
				++RowCount;
				RowArgs = &PopulateRowArgs[RowCount];
			}

			KeysAndRows.Emplace(RowArgs->AssetRegistryPathKey, ReservedRow[Index]);
			++RowCount;
		}

		Database.BatchMapRows(MappingDomain, KeysAndRows);

		int32 Index = 0;
		Database.BatchAddRow(PathsTable, ReservedRow, [&PopulateRowArgs, &Index, this](RowHandle InRowHandle)
			{
				FPopulatePathRowArgs RowArgs = MoveTemp(PopulateRowArgs[Index]);
				while (!RowArgs)
				{
					++Index;
					RowArgs = MoveTemp(PopulateRowArgs[Index]);
				}

				PopulatePathDataTableRow(MoveTemp(RowArgs), Database, InRowHandle);
				++Index;
			});
	}

	// Cut the consumed batch off of the back of our queue
	AddedPaths.RemoveAt(BatchStartIndex, BatchSize);

	FTimespan ConsumedTime(static_cast<int64>(AvgTimePerProcessedPathAdd * BatchSize));
	LastAddedPathBatchSize += BatchSize; // Accumulate total batch size since we can be called multiple times per tick
	return AllowedTime - ConsumedTime;
}
 
void FTedsAssetData::OnPathsRemoved(TConstArrayView<FStringView> InPathsRemoved)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FTedsAssetData::OnPathsRemoved);

#if TRACK_TEDSASSETDATA_MEMORY
	LLM_SCOPE_BYNAME(TEXT("FTedsAssetData"))
#endif

	for (const FStringView Path : InPathsRemoved)
	{
		FName PathName(Path);
		FMapKey PathKey = FMapKey(PathName);
		Database.RemoveRow(Database.LookupMappedRow(MappingDomain, PathKey));
		AddedPaths.Remove(PathName);
	}
}

void FTedsAssetData::OnDatabaseUpdate()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FTedsAssetData::OnDatabaseUpdate);

	// Asset Registry is currently gathering. Don't process batches at this time to avoid competing on the Game Thread
	if (bIsPaused)
	{
		return;
	}

	UpdateAvgProcessedTime(AvgTimePerProcessedPathAdd, ProcessPathAddStartTime, ProcessPathAddEndTime, LastAddedPathBatchSize);
	UpdateAvgProcessedTime(AvgTimePerProcessedAssetAdd, ProcessAssetAddStartTime, ProcessAssetAddEndTime, LastAddedAssetBatchSize);
	UpdateAvgProcessedTime(AvgTimePerProcessedAssetUpdate, ProcessAssetUpdateStartTime, ProcessAssetUpdateEndTime, LastUpdatedAssetBatchSize);

#if COUNTERSTRACE_ENABLED
	RecordAssetDataStats();
#endif

	LastAddedPathBatchSize = 0;
	LastAddedAssetBatchSize = 0;
	LastUpdatedAssetBatchSize = 0;

	ProcessedAddedPathBatchSize = 0;
	ProcessedAddedAssetBatchSize = 0;
	ProcessedUpdatedAssetBatchSize = 0;

	ProcessTaskQueues(MaxAllowedTimePerTick);

	ProcessPathAddStartTime = std::numeric_limits<uint64>::max();
	ProcessPathAddEndTime = 0;

	ProcessAssetAddStartTime = std::numeric_limits<uint64>::max();
	ProcessAssetAddEndTime = 0;

	ProcessAssetUpdateStartTime = std::numeric_limits<uint64>::max();
	ProcessAssetUpdateEndTime = 0;
}

void FTedsAssetData::UpdateAvgProcessedTime(double& OutAverageTime, uint64 LatestStartTime, uint64 LatestEndTime, int32 LastBatchSize)
{
	constexpr double PreviousTicksWeight = 0.7;
	constexpr double LatestTickWeight = 0.3;

	static_assert(PreviousTicksWeight + LatestTickWeight == 1.0);

	if (LastBatchSize != 0 && LatestEndTime != 0)
	{
		FTimespan ProcessorDuration = FTimespan::FromSeconds(FPlatformTime::ToSeconds64(LatestEndTime - LatestStartTime));
		double LatestTimePerUnit = static_cast<double>((ProcessorDuration / LastBatchSize).GetTicks());
		LatestTimePerUnit = FMath::Max(LatestTimePerUnit, 1); // Timespan can truncate to 0 so give it a min of 1

		OutAverageTime = (OutAverageTime * PreviousTicksWeight) + (LatestTimePerUnit * LatestTickWeight);
		OutAverageTime = FMath::Max(OutAverageTime, 1);
	}
}

void FTedsAssetData::ProcessTaskQueues(const FTimespan AllowedTime)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FTedsAssetData::ProcessTaskQueues);

	if (AllowedTime.IsZero())
	{
		return;
	}

	int32 TaskTypeCount = AddedPaths.IsEmpty() ? 0 : 1;
	TaskTypeCount += AddedAssets.IsEmpty() ? 0 : 1;
	TaskTypeCount += UpdatedAssets.IsEmpty() ? 0 : 1;

	if (TaskTypeCount == 0)
	{
		return;
	}

	// By dividing allowed time by number of filled task queues we allocate time to each task type instead of all to the first type
	FTimespan AllowedTimePerTaskType = AllowedTime / TaskTypeCount;

	FTimespan UnusedTime = 0;
	if (!AddedPaths.IsEmpty())
	{
		UnusedTime = ProcessAddedPaths(AllowedTimePerTaskType);
	}

	if (!AddedAssets.IsEmpty())
	{
		UnusedTime = ProcessAddedAssets(AllowedTimePerTaskType + UnusedTime);
	}

	if (!UpdatedAssets.IsEmpty())
	{
		ProcessUpdatedAssets(AllowedTimePerTaskType + UnusedTime);
	}
}

void FTedsAssetData::RecordAssetDataStats()
{
#if COUNTERSTRACE_ENABLED

	TRACE_CPUPROFILER_EVENT_SCOPE(FTedsAssetData::RecordAssetDataStats);

	SET_DWORD_STAT(STAT_TedsAssetData_PathAddsRemaining, AddedPaths.Num());
	SET_DWORD_STAT(STAT_TedsAssetData_AssetAddsRemaining, AddedAssets.Num());
	SET_DWORD_STAT(STAT_TedsAssetData_AssetUpdatesRemaining, UpdatedAssets.Num());

	TRACE_COUNTER_SET(COUNTER_TedsAssetData_PathAddsRemaining, AddedPaths.Num());
	TRACE_COUNTER_SET(COUNTER_TedsAssetData_AssetAddsRemaining, AddedAssets.Num());
	TRACE_COUNTER_SET(COUNTER_TedsAssetData_AssetUpdatesRemaining, UpdatedAssets.Num());

	SET_DWORD_STAT(STAT_TedsAssetData_PathAddBatchSize, LastAddedPathBatchSize);
	SET_DWORD_STAT(STAT_TedsAssetData_AssetAddBatchSize, LastAddedAssetBatchSize);
	SET_DWORD_STAT(STAT_TedsAssetData_AssetUpdateBatchSize, LastUpdatedAssetBatchSize);

	TRACE_COUNTER_SET(COUNTER_TedsAssetData_PathAddBatchSize, LastAddedPathBatchSize);
	TRACE_COUNTER_SET(COUNTER_TedsAssetData_AssetAddBatchSize, LastAddedAssetBatchSize);
	TRACE_COUNTER_SET(COUNTER_TedsAssetData_AssetUpdateBatchSize, LastUpdatedAssetBatchSize);

	SET_DWORD_STAT(STAT_TedsAssetData_PathAddBatchSizeReceived, ProcessedAddedPathBatchSize.load());
	SET_DWORD_STAT(STAT_TedsAssetData_AssetAddBatchSizeReceived, ProcessedAddedAssetBatchSize.load());
	SET_DWORD_STAT(STAT_TedsAssetData_AssetUpdateBatchSizeReceived, ProcessedUpdatedAssetBatchSize.load());

	TRACE_COUNTER_SET(COUNTER_TedsAssetData_PathAddBatchSizeReceived, ProcessedAddedPathBatchSize.load());
	TRACE_COUNTER_SET(COUNTER_TedsAssetData_AssetAddBatchSizeReceived, ProcessedAddedAssetBatchSize.load());
	TRACE_COUNTER_SET(COUNTER_TedsAssetData_AssetUpdateBatchSizeReceived, ProcessedUpdatedAssetBatchSize.load());

	SET_FLOAT_STAT(STAT_TedsAssetData_PathAddAvgTime, AvgTimePerProcessedPathAdd / ETimespan::TicksPerMillisecond);
	SET_FLOAT_STAT(STAT_TedsAssetData_AssetAddAvgTime, AvgTimePerProcessedAssetAdd / ETimespan::TicksPerMillisecond);
	SET_FLOAT_STAT(STAT_TedsAssetData_AssetUpdateAvgTime, AvgTimePerProcessedAssetUpdate / ETimespan::TicksPerMillisecond);

	TRACE_COUNTER_SET(COUNTER_TedsAssetData_PathAddAvgTime, AvgTimePerProcessedPathAdd / ETimespan::TicksPerMillisecond);
	TRACE_COUNTER_SET(COUNTER_TedsAssetData_AssetAddAvgTime, AvgTimePerProcessedAssetAdd / ETimespan::TicksPerMillisecond);
	TRACE_COUNTER_SET(COUNTER_TedsAssetData_AssetUpdateAvgTime, AvgTimePerProcessedAssetUpdate / ETimespan::TicksPerMillisecond);

	SET_FLOAT_STAT(STAT_TedsAssetData_PathAddBatchLastDuration, FTimespan::FromSeconds(FPlatformTime::ToSeconds64(ProcessPathAddEndTime - ProcessPathAddStartTime)).GetTotalMilliseconds());
	SET_FLOAT_STAT(STAT_TedsAssetData_AssetAddBatchLastDuration, FTimespan::FromSeconds(FPlatformTime::ToSeconds64(ProcessAssetAddEndTime - ProcessAssetAddStartTime)).GetTotalMilliseconds());
	SET_FLOAT_STAT(STAT_TedsAssetData_AssetUpdateBatchLastDuration, FTimespan::FromSeconds(FPlatformTime::ToSeconds64(ProcessAssetUpdateEndTime - ProcessAssetUpdateStartTime)).GetTotalMilliseconds());

	TRACE_COUNTER_SET(COUNTER_TedsAssetData_PathAddBatchLastDuration, FTimespan::FromSeconds(FPlatformTime::ToSeconds64(ProcessPathAddEndTime - ProcessPathAddStartTime)).GetTotalMilliseconds());
	TRACE_COUNTER_SET(COUNTER_TedsAssetData_AssetAddBatchLastDuration, FTimespan::FromSeconds(FPlatformTime::ToSeconds64(ProcessAssetAddEndTime - ProcessAssetAddStartTime)).GetTotalMilliseconds());
	TRACE_COUNTER_SET(COUNTER_TedsAssetData_AssetUpdateBatchLastDuration, FTimespan::FromSeconds(FPlatformTime::ToSeconds64(ProcessAssetUpdateEndTime - ProcessAssetUpdateStartTime)).GetTotalMilliseconds());

	if ((AddedPaths.Num() + AddedAssets.Num() + UpdatedAssets.Num()) > 0)
	{
		SET_FLOAT_STAT(STAT_TedsAssetData_TotalProcessingTime, FPlatformTime::ToSeconds64(FPlatformTime::Cycles64() - ProcessingStartTime));

		TRACE_COUNTER_SET(COUNTER_TedsAssetData_TotalProcessingTime, FPlatformTime::ToSeconds64(FPlatformTime::Cycles64() - ProcessingStartTime));
	}

#endif
}

} // namespace UE::Editor::AssetData::Private

#undef TRACK_TEDSASSETDATA_MEMORY
