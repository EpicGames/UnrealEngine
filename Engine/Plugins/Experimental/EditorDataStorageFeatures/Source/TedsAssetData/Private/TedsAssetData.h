// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ContainersFwd.h"
#include "Containers/StringFwd.h"
#include "DataStorage/Handles.h"

struct FAssetData;

namespace UE::Editor::DataStorage
{
	class ICoreProvider;
} // UE::Editor::DataStorage

namespace UE::Editor::AssetData::Private
{
	/**
	 * Manage the registration and life cycle of the row related representing the data from the asset registry into TEDS.
	 */
	class FTedsAssetData
	{
		friend class FTedsAssetDataCBDataSource;

	public:
		FTedsAssetData(UE::Editor::DataStorage::ICoreProvider& InDatabase);

		~FTedsAssetData();

		void Initialize();
		void StartProcessing();
		void PauseProcessing();

		void ProcessAllEvents();

	private:
		void OnARScanStarted();
		void OnARScanEnded();

		void OnAssetsAdded(TConstArrayView<FAssetData> InAssetsAdded);
		FTimespan ProcessAddedAssets(const FTimespan& AllowedTime);

		void OnAssetsRemoved(TConstArrayView<FAssetData> InAssetsRemoved);

		void OnAssetsUpdated(TConstArrayView<FAssetData> InAssetsUpdated);
		void OnAssetsUpdatedOnDisk(TConstArrayView<FAssetData> InAssetsUpdated);
		void OnAssetsUpdated_Impl(TConstArrayView<FAssetData> InAssetsUpdated);
		FTimespan ProcessUpdatedAssets(const FTimespan& AllowedTime);

		void OnAssetRenamed(const FAssetData& InAsset, const FString& InOldObjectPath);

		void OnPathsAdded(TConstArrayView<FStringView> InPathsAdded);
		FTimespan ProcessAddedPaths(const FTimespan& AllowedTIme);

		void OnPathsRemoved(TConstArrayView<FStringView> InPathsRemoved);

		void OnDatabaseUpdate();

		void UpdateAvgProcessedTime(double& OutAverageTime, uint64 LatestStartTime, uint64 LatestEndTime, int32 LastBatchSize);

		void ProcessTaskQueues(const FTimespan AllowedTime);

		void RecordAssetDataStats();

		bool bIsInitialized = false;
		bool bIsPaused = true;

		TArray<FName> AddedPaths;
		TArray<FAssetData> AddedAssets;
		TArray<FAssetData> UpdatedAssets;

		const FTimespan MaxAllowedTimePerTick = FTimespan::FromMilliseconds(6);
		const FTimespan DefaultMaxTimePerProcessCategory = MaxAllowedTimePerTick / 3;
		const uint32 DefaultNumberOfProcessedItems = 500;

		double AvgTimePerProcessedPathAdd = static_cast<double>((DefaultMaxTimePerProcessCategory / DefaultNumberOfProcessedItems).GetTicks());
		double AvgTimePerProcessedAssetAdd = static_cast<double>((DefaultMaxTimePerProcessCategory / DefaultNumberOfProcessedItems).GetTicks());
		double AvgTimePerProcessedAssetUpdate = static_cast<double>((DefaultMaxTimePerProcessCategory / DefaultNumberOfProcessedItems).GetTicks());

		const int32 MinimumBatchSize = 250;
		int32 LastAddedPathBatchSize = 0;
		int32 LastAddedAssetBatchSize = 0;
		int32 LastUpdatedAssetBatchSize = 0;

		static std::atomic<uint32> ProcessedAddedPathBatchSize;
		static std::atomic<uint32> ProcessedAddedAssetBatchSize;
		static std::atomic<uint32> ProcessedUpdatedAssetBatchSize;

		uint64 ProcessingStartTime = 0;

		static std::atomic<uint64> ProcessPathAddStartTime;
		static std::atomic<uint64> ProcessPathAddEndTime;

		static std::atomic<uint64> ProcessAssetAddStartTime;
		static std::atomic<uint64> ProcessAssetAddEndTime;

		static std::atomic<uint64> ProcessAssetUpdateStartTime;
		static std::atomic<uint64> ProcessAssetUpdateEndTime;

		UE::Editor::DataStorage::ICoreProvider& Database;
		DataStorage::TableHandle PathsTable = DataStorage::InvalidTableHandle;
		DataStorage::TableHandle AssetsDataTable = DataStorage::InvalidTableHandle;

		DataStorage::QueryHandle RemoveUpdatedPathTagQuery = DataStorage::InvalidQueryHandle;
		DataStorage::QueryHandle RemoveUpdatedAssetDataTagQuery = DataStorage::InvalidQueryHandle;
	};
} // namespace UE::Editor::AssetData::Private
