// Copyright Epic Games, Inc. All Rights Reserved.

#include "TedsAssetDataHelper.h"

namespace TedsAssetDataHelper
{
	namespace MetaDataNames
	{
		static const FLazyName ThumbnailStatus("bAllowAssetStatusThumbnailOverlay");
		static const FLazyName ThumbnailFadeIn("bAllowFadeIn");
		static const FLazyName ThumbnailHintText("bAllowHintText");
		static const FLazyName ThumbnailRealTimeOnHovered("bAllowRealTimeOnHovered");
		static const FLazyName ThumbnailSizeOffset("ThumbnailSizeOffset");
		static const FLazyName ThumbnailCanDisplayEditModePrimitiveTools("CanDisplayEditModePrimitiveTools");

		FName GetThumbnailStatusMetaDataName() { return ThumbnailStatus; }
		FName GetThumbnailFadeInMetaDataName() { return ThumbnailFadeIn; }
		FName GetThumbnailHintTextMetaDataName() { return ThumbnailHintText; }
		FName GetThumbnailRealTimeOnHoveredMetaDataName() { return ThumbnailRealTimeOnHovered; }
		FName GetThumbnailSizeOffsetMetaDataName() { return ThumbnailSizeOffset; }
		FName GetThumbnailCanDisplayEditModePrimitiveTools() { return ThumbnailCanDisplayEditModePrimitiveTools; }
	}

	namespace TableView
	{
		static FLazyName TableViewerWidgetTableName("Editor_TableViewerWidgetTable");

		FName GetWidgetTableName() { return TableViewerWidgetTableName; }
	}

	FString RemoveSlashFromStart(const FString& InStringToModify)
	{
		FString StringToReturn = InStringToModify;
		int32 StartingSlashToRemoveIndex = INDEX_NONE;

		for (int32 Index = 0; Index < StringToReturn.Len(); ++Index)
		{
			if (StringToReturn[Index] == TEXT('/'))
			{
				// Add 1 since we later use RightChop which use the count and not the index
				StartingSlashToRemoveIndex = Index + 1;
			}
			else
			{
				break;
			}
		}

		if (StartingSlashToRemoveIndex != INDEX_NONE)
		{
			StringToReturn = StringToReturn.RightChop(StartingSlashToRemoveIndex);
		}

		return StringToReturn;
	}

	FString RemoveAllFromLastSlash(const FString& InStringToModify)
	{
		int32 LastSlashToRemoveIndex = INDEX_NONE;
		InStringToModify.FindLastChar(TEXT('/'), LastSlashToRemoveIndex);

		if (LastSlashToRemoveIndex != INDEX_NONE)
		{
			return InStringToModify.Left(LastSlashToRemoveIndex);
		}

		return InStringToModify;
	}
}
