// Copyright Epic Games, Inc. All Rights Reserved.

#include "TedsAssetDataModule.h"

#include "CB/TedsAssetDataCBDataSource.h"
#include "DataStorage/Features.h"
#include "Elements/Interfaces/TypedElementDataStorageInterface.h"
#include "HAL/IConsoleManager.h"
#include "Modules/ModuleManager.h"
#include "TedsAssetData.h"

IMPLEMENT_MODULE(UE::Editor::AssetData::FTedsAssetDataModule, TedsAssetData);

namespace UE::Editor::AssetData
{

namespace Private
{
TAutoConsoleVariable<bool> CVarTEDSAssetDataStorage(TEXT("TEDS.AssetDataStorage"), true, TEXT("When true we will activate a wrapper that store the a copy of the asset data including the in memory change from the asset registry into TEDS.")
	, FConsoleVariableDelegate::CreateLambda([](IConsoleVariable* Variable)
	{
		const bool bIsEnabled = Variable->GetBool();
		FTedsAssetDataModule& Module = FTedsAssetDataModule::GetChecked();

		if (bIsEnabled)
		{
			Module.EnableTedsAssetRegistryStorage();
		}
		else
		{
			Module.DisableTedsAssetRegistryStorage();
		}
	}));
} // namespace Private

void FTedsAssetDataModule::StartupModule()
{
	if (Private::CVarTEDSAssetDataStorage.GetValueOnGameThread())
	{
		EnableTedsAssetRegistryStorage();
	}
}

void FTedsAssetDataModule::ShutdownModule()
{
	UE::Editor::DataStorage::OnEditorDataStorageFeaturesEnabled().RemoveAll(this);
}

FTedsAssetDataModule* FTedsAssetDataModule::Get()
{
	return FModuleManager::Get().LoadModulePtr<FTedsAssetDataModule>(TEXT("TedsAssetData"));
}

FTedsAssetDataModule& FTedsAssetDataModule::GetChecked()
{
	return FModuleManager::Get().LoadModuleChecked<FTedsAssetDataModule>(TEXT("TedsAssetData"));
}

void FTedsAssetDataModule::EnableTedsAssetRegistryStorage()
{
	using namespace UE::Editor::DataStorage;
	if (!AssetRegistryStorage)
	{
		FModuleManager::Get().LoadModuleChecked(TEXT("TypedElementFramework"));
		if (AreEditorDataStorageFeaturesEnabled())
		{
			InitAssetRegistryStorage();
		}
		else
		{
			OnEditorDataStorageFeaturesEnabled().AddRaw(this, &FTedsAssetDataModule::InitAssetRegistryStorage);
		}

		if (!Private::CVarTEDSAssetDataStorage.GetValueOnGameThread())
		{
			Private::CVarTEDSAssetDataStorage.AsVariable()->Set(true);
		}
	}
}

void FTedsAssetDataModule::DisableTedsAssetRegistryStorage()
{
	if (AssetRegistryStorage)
	{
		AssetRegistryStorage.Reset();

		if (Private::CVarTEDSAssetDataStorage.GetValueOnGameThread())
		{
			Private::CVarTEDSAssetDataStorage.AsVariable()->Set(false);
		}
	}
}

bool FTedsAssetDataModule::IsTedsAssetRegistryStorageEnabled() const
{
	return AssetRegistryStorage.IsValid();
}

void FTedsAssetDataModule::ProcessDependentEvents()
{
	if (Private::FTedsAssetData* Storage = AssetRegistryStorage.Get())
	{
		Storage->ProcessAllEvents();
	}
}

void FTedsAssetDataModule::InitAssetRegistryStorage()
{
	using namespace UE::Editor::DataStorage;
	ICoreProvider& MutableDataStorage = *GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName);

	AssetDataCBDataSource = MakeUnique<Private::FTedsAssetDataCBDataSource>(MutableDataStorage);
	AssetRegistryStorage = MakeUnique<Private::FTedsAssetData>(MutableDataStorage);
	
	OnAssetRegistryStorageInitDelegate.Broadcast();
}

void FTedsAssetDataModule::EnableAssetDataMetadataStorage()
{
	if (AssetDataCBDataSource)
	{
		AssetDataCBDataSource->EnableMetadataStorage(true);
	}
}

void FTedsAssetDataModule::DisableAssetDataMetadataStorage()
{
	if (AssetDataCBDataSource)
	{
		AssetDataCBDataSource->EnableMetadataStorage(false);
	}
}

FSimpleMulticastDelegate& FTedsAssetDataModule::OnAssetRegistryStorageInit()
{
	return OnAssetRegistryStorageInitDelegate;
}
}// namespace UE::Editor::AssetData
