// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"
#include "Modules/ModuleInterface.h"
#include "Templates/UniquePtr.h"

namespace UE::Editor::AssetData
{ 

namespace Private
{
	class FTedsAssetData;
	class FTedsAssetDataCBDataSource;
} // namespace Private

class FTedsAssetDataModule : public IModuleInterface
{
public:
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

	static FTedsAssetDataModule* Get();
	static FTedsAssetDataModule& GetChecked();
	void EnableTedsAssetRegistryStorage();
	void DisableTedsAssetRegistryStorage();
	bool IsTedsAssetRegistryStorageEnabled() const;
	void EnableAssetDataMetadataStorage();
	void DisableAssetDataMetadataStorage();
	FSimpleMulticastDelegate& OnAssetRegistryStorageInit();

	/**
	 * Process now any pending event that might make the Teds database out of sync with the asset registry.
	 * Note: This isn't needed when using the editor so it should be only called by the automation scripts that need it to avoid creating some unneeded stalls.
	 */
	void ProcessDependentEvents();

private:
	void InitAssetRegistryStorage();

	TUniquePtr<Private::FTedsAssetDataCBDataSource> AssetDataCBDataSource;
	TUniquePtr<Private::FTedsAssetData> AssetRegistryStorage;
	FSimpleMulticastDelegate OnAssetRegistryStorageInitDelegate;
};

} // namespace UE::Editor::AssetData
