// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/AutomationTest.h"

#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/IAssetRegistry.h"
#include "Containers/ChunkedArray.h"
#include "Containers/Set.h"
#include "Containers/UnrealString.h"
#include "DataStorage/Features.h"
#include "DataStorage/Handles.h"
#include "Elements/Framework/TypedElementRegistry.h"
#include "Elements/Interfaces/TypedElementDataStorageInterface.h"
#include "Misc/ScopeExit.h"
#include "TedsAssetDataColumns.h"
#include "TedsAssetDataModule.h"
#include "UObject/NameTypes.h"

namespace UE::Editor::AssetData::Tests
{
#if 0 // TODO: Test must be updated to handle verse correctly
	using namespace DataStorage;
IMPLEMENT_SIMPLE_AUTOMATION_TEST(FTedsAssetDataTest, "Editor.DataStorage.AssetRegistry.ValidateState", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter | EAutomationTestFlags::HighPriority)

bool FTedsAssetDataTest::RunTest(const FString& Parameters)
{
	FTedsAssetDataModule& TedsAssetRegistry = FTedsAssetDataModule::GetChecked();

	const bool bIsEnabled = TedsAssetRegistry.IsTedsAssetRegistryStorageEnabled();
	ON_SCOPE_EXIT
	{
		if (!bIsEnabled)
		{
			TedsAssetRegistry.DisableTedsAssetRegistryStorage();
		}
	};

	TedsAssetRegistry.EnableTedsAssetRegistryStorage();

	TMap<FName, int32> AssetRegistryPathsAndAssetCount;

	IAssetRegistry& AssetRegistry = IAssetRegistry::GetChecked();
	AssetRegistry.WaitForCompletion();
	 
	AssetRegistry.EnumerateAllCachedPaths([&AssetRegistryPathsAndAssetCount](FName InPath)
		{
			AssetRegistryPathsAndAssetCount.Add(InPath, 0);
			return true;
		});

	TChunkedArray<FAssetData> AssetRegistryAssetsData;
	AssetRegistry.EnumerateAllAssets([&AssetRegistryAssetsData, &AssetRegistryPathsAndAssetCount](const FAssetData& InAssetData)
		{
			AssetRegistryAssetsData.AddElement(InAssetData);

			if (int32* Count = AssetRegistryPathsAndAssetCount.Find(InAssetData.PackagePath))
			{
				//ensure(!InAssetData.OptionalOuterPath.IsNull());
				++(*Count);
			}

			return true;
		});

	TedsAssetRegistry.ProcessDependentEvents();


	// Do a sanity check that the data from the asset registry exist in Teds
	using namespace UE::Editor::DataStorage;
	const ICoreProvider* Database = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName);
	
	for (const TPair<FName, int32>& PathAndAssetCount : AssetRegistryPathsAndAssetCount)
	{
		RowHandle Row = Database->LookupMappedRow(AssetData::MappingDomain, FMapKeyView(PathAndAssetCount.Key));
		bool bHadError = false;
		bHadError |= !TestTrue(TEXT("Asset registry folder/path is indexed in TEDS"), Row != InvalidRowHandle);

		FName NameStoredInTeds;
		if (const FAssetPathColumn_Experimental* AssetPathColumn = Database->GetColumn<FAssetPathColumn_Experimental>(Row))
		{
			NameStoredInTeds = AssetPathColumn->Path;
		}

		bHadError |= !TestEqual(TEXT("Path Stored In TEDS Match Path From Asset Registry"), NameStoredInTeds, PathAndAssetCount.Key);

		if (bHadError)
		{
			TArray<FStringFormatArg> FormatArgs;
			FormatArgs.Emplace(PathAndAssetCount.Key.ToString());
			FString Message = FString::Format(TEXT("Error was found at the following path:\"{0}\""), FormatArgs);
			AddError(Message);

			// Stop at the first error so that test don't take top long if something broke
			break;
		}
	}

	FNameBuilder Builder;
	for (const FAssetData& AssetData : AssetRegistryAssetsData)
	{
		AssetData.PackagePath.ToString(Builder);
		Builder.AppendChar(TEXT('/'));

		// Verse package don't emit asset added event so we will ignore those for the test to avoid false errors
		if (!FPackageName::IsVersePackage(Builder))
		{ 
			RowHandle Row = Database->LookupMappedRow(AssetData::MappingDomain, FMapKey(AssetData.GetSoftObjectPath()));

			bool bHadError = false;

			bHadError |= !TestTrue(TEXT("Asset registry asset path is indexed in TEDS"), Row != InvalidRowHandle);

			bHadError |= !TestNotNull(TEXT("Teds doesn't have a asset column for an asset of the asset registry"), Database->GetColumn<FAssetDataColumn_Experimental>(Row));

			if (bHadError)
			{
				TArray<FStringFormatArg> FormatArgs;
				FormatArgs.Emplace(AssetData.GetSoftObjectPath().ToString());
				FString Message = FString::Format(TEXT("Error was found for this asset:\"{0}\""), FormatArgs);
				AddError(Message);

				AssetData.PrintAssetData();

				// Stop at the first error so that test don't take top long if something broke
				break;
			}
		}
	}

	return true;
}
#endif
} // namespace UE::Editor::AssetData::Tests
