// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/AssetDataDiskSizeWidget.h"

#include "Elements/Framework/TypedElementAttributeBinding.h"
#include "Internationalization/Text.h"
#include "TedsAssetDataColumns.h"
#include "UObject/ObjectMacros.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Text/STextBlock.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AssetDataDiskSizeWidget)

void UDiskSizeWidgetFactory::RegisterWidgetConstructors(UE::Editor::DataStorage::ICoreProvider& DataStorage,
	UE::Editor::DataStorage::IUiProvider& DataStorageUi) const
{
	using namespace UE::Editor::DataStorage::Queries;
	DataStorageUi.RegisterWidgetFactory<FDiskSizeWidgetConstructor>(DataStorageUi.FindPurpose(DataStorageUi.GetGeneralWidgetPurposeID()), TColumn<FDiskSizeColumn>());
}

FDiskSizeWidgetConstructor::FDiskSizeWidgetConstructor()
	: FSimpleWidgetConstructor(StaticStruct())
{
}

TSharedPtr<SWidget> FDiskSizeWidgetConstructor::CreateWidget(UE::Editor::DataStorage::ICoreProvider* DataStorage,
	UE::Editor::DataStorage::IUiProvider* DataStorageUi, UE::Editor::DataStorage::RowHandle TargetRow, UE::Editor::DataStorage::RowHandle WidgetRow,
	const UE::Editor::DataStorage::FMetaDataView& Arguments)
{
	UE::Editor::DataStorage::FAttributeBinder Binder(TargetRow, DataStorage);

	static const FMargin ColumnItemPadding(5, 0, 5, 0);

	return SNew(SBox)
			.Padding(ColumnItemPadding)
			[
				SNew(STextBlock)
				.Text(Binder.BindData(&FDiskSizeColumn::DiskSize, [](int64 DiskSize)
					{
						return FText::AsMemory(DiskSize);
					}))
			];
}
