// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/Interfaces/TypedElementDataStorageFactory.h"
#include "Elements/Interfaces/TypedElementDataStorageUiInterface.h"

#include "AssetDataDiskSizeWidget.generated.h"

class UScriptStruct;

UCLASS()
class UDiskSizeWidgetFactory : public UEditorDataStorageFactory
{
	GENERATED_BODY()

public:
	~UDiskSizeWidgetFactory() override = default;

	TEDSASSETDATA_API void RegisterWidgetConstructors(UE::Editor::DataStorage::ICoreProvider& DataStorage,
		UE::Editor::DataStorage::IUiProvider& DataStorageUi) const override;
};

// Widget to show disk size in bytes
USTRUCT()
struct FDiskSizeWidgetConstructor : public FSimpleWidgetConstructor
{
	GENERATED_BODY()

public:
	TEDSASSETDATA_API FDiskSizeWidgetConstructor();
	~FDiskSizeWidgetConstructor() override = default;

	TEDSASSETDATA_API virtual TSharedPtr<SWidget> CreateWidget(
		UE::Editor::DataStorage::ICoreProvider* DataStorage,
		UE::Editor::DataStorage::IUiProvider* DataStorageUi,
		UE::Editor::DataStorage::RowHandle TargetRow,
		UE::Editor::DataStorage::RowHandle WidgetRow,
		const UE::Editor::DataStorage::FMetaDataView& Arguments) override;
};