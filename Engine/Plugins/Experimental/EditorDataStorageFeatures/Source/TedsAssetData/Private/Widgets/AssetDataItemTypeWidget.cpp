// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/AssetDataItemTypeWidget.h"

#include "AssetDefinition.h"
#include "AssetDefinitionRegistry.h"
#include "Elements/Framework/TypedElementAttributeBinding.h"
#include "Internationalization/Text.h"
#include "TedsAssetDataColumns.h"
#include "TedsAssetDataWidgetColumns.h"
#include "Columns/SlateDelegateColumns.h"
#include "Elements/Columns/TypedElementFolderColumns.h"
#include "UObject/ObjectMacros.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Text/STextBlock.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AssetDataItemTypeWidget)

#define LOCTEXT_NAMESPACE "FAssetDataItemTypeWidgetConstructor"

void UAssetDataItemTypeWidgetFactory::RegisterWidgetConstructors(UE::Editor::DataStorage::ICoreProvider& DataStorage,
	UE::Editor::DataStorage::IUiProvider& DataStorageUi) const
{
	using namespace UE::Editor::DataStorage::Queries;
	DataStorageUi.RegisterWidgetFactory<FAssetDataItemTypeWidgetConstructor>(DataStorageUi.FindPurpose(DataStorageUi.GetGeneralWidgetPurposeID()),
		TColumn<FAssetClassColumn>() || TColumn<FFolderTag>());
}

FAssetDataItemTypeWidgetConstructor::FAssetDataItemTypeWidgetConstructor()
	: FSimpleWidgetConstructor(StaticStruct())
{
}

TSharedPtr<SWidget> FAssetDataItemTypeWidgetConstructor::CreateWidget(UE::Editor::DataStorage::ICoreProvider* DataStorage,
	UE::Editor::DataStorage::IUiProvider* DataStorageUi,
	UE::Editor::DataStorage::RowHandle TargetRow, UE::Editor::DataStorage::RowHandle WidgetRow,
	const UE::Editor::DataStorage::FMetaDataView& Arguments)
{
	UE::Editor::DataStorage::FAttributeBinder Binder(TargetRow, DataStorage);
	UE::Editor::DataStorage::FAttributeBinder WidgetBinder(WidgetRow, DataStorage);

	const bool bIsFolder = DataStorage->HasColumns<FFolderTag>(TargetRow);

	return SNew(STextBlock)
			.Font(WidgetBinder.BindData(&FFontStyleColumn_Experimental::FontInfo, FAppStyle::GetFontStyle("NormalFont")))
			.Visibility(WidgetBinder.BindData(&FWidgetVisibilityColumn_Experimental::Visibility, EVisibility::Visible))
			.OverflowPolicy(WidgetBinder.BindData(&FTextOverflowPolicyColumn_Experimental::OverflowPolicy, ETextOverflowPolicy::Ellipsis).Get())
			.ColorAndOpacity(WidgetBinder.BindEventAttribute(&FOnGetWidgetColorAndOpacityColumn_Experimental::OnGetWidgetColorAndOpacity, FSlateColor::UseForeground()))
			.Text(Binder.BindData(&FAssetClassColumn::ClassPath, [bIsFolder](FTopLevelAssetPath InClassPath)
			{
				if (bIsFolder)
				{
					return LOCTEXT("AssetClassFolder", "Folder");
				}

				if (const UAssetDefinitionRegistry* AssetDefinitionRegistry = UAssetDefinitionRegistry::Get())
				{
					if (const UClass* FoundClass = FindObject<UClass>(InClassPath))
					{
						if (const UAssetDefinition* AssetDefinition = AssetDefinitionRegistry->GetAssetDefinitionForClass(FoundClass))
						{
							return AssetDefinition->GetAssetDisplayName();
						}
					}
				}
				return LOCTEXT("AssetClass_Invalid", "Invalid or not found");
			}));
}

#undef LOCTEXT_NAMESPACE
