// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/Interfaces/TypedElementDataStorageFactory.h"
#include "Elements/Interfaces/TypedElementDataStorageUiInterface.h"

#include "AssetDataItemTypeWidget.generated.h"

UCLASS()
class UAssetDataItemTypeWidgetFactory : public UEditorDataStorageFactory
{
	GENERATED_BODY()

public:
	~UAssetDataItemTypeWidgetFactory() override = default;

	TEDSASSETDATA_API void RegisterWidgetConstructors(UE::Editor::DataStorage::ICoreProvider& DataStorage,
		UE::Editor::DataStorage::IUiProvider& DataStorageUi) const override;
};

// Widget to show Item type (folder/material/mesh etc...)
USTRUCT()
struct FAssetDataItemTypeWidgetConstructor : public FSimpleWidgetConstructor
{
	GENERATED_BODY()

public:
	TEDSASSETDATA_API FAssetDataItemTypeWidgetConstructor();
	~FAssetDataItemTypeWidgetConstructor() override = default;

	TEDSASSETDATA_API virtual TSharedPtr<SWidget> CreateWidget(
		UE::Editor::DataStorage::ICoreProvider* DataStorage,
		UE::Editor::DataStorage::IUiProvider* DataStorageUi,
		UE::Editor::DataStorage::RowHandle TargetRow,
		UE::Editor::DataStorage::RowHandle WidgetRow,
		const UE::Editor::DataStorage::FMetaDataView& Arguments) override;
};
