// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/AssetDataLabelWidget.h"

#include "AssetDefinition.h"
#include "AssetDefinitionRegistry.h"
#include "AssetViewTypes.h"
#include "Elements/Framework/TypedElementAttributeBinding.h"
#include "TedsAssetDataColumns.h"
#include "TedsAssetDataHelper.h"
#include "Elements/Columns/TypedElementSlateWidgetColumns.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/SBoxPanel.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AssetDataLabelWidget)

#define LOCTEXT_NAMESPACE "FAssetDataLabelWidgetConstructor"

void UAssetDataLabelWidgetFactory::RegisterWidgetConstructors(UE::Editor::DataStorage::ICoreProvider& DataStorage,
	UE::Editor::DataStorage::IUiProvider& DataStorageUi) const
{
	using namespace UE::Editor::DataStorage::Queries;
	DataStorageUi.RegisterWidgetFactory<FAssetDataLabelWidgetConstructor>(
		DataStorageUi.FindPurpose(IUiProvider::FPurposeInfo("General", "RowLabel", NAME_None).GeneratePurposeID()),
		TColumn<FAssetNameColumn>() && (TColumn<FAssetTag>() || TColumn<FAssetPathColumn_Experimental>()));
}

FAssetDataLabelWidgetConstructor::FAssetDataLabelWidgetConstructor()
	: FSimpleWidgetConstructor(StaticStruct())
{
}

FAssetDataLabelWidgetConstructor::FAssetDataLabelWidgetConstructor(const UScriptStruct* TypeInfo)
	: FSimpleWidgetConstructor(TypeInfo)
{
}

TSharedPtr<SWidget> FAssetDataLabelWidgetConstructor::CreateWidget(UE::Editor::DataStorage::ICoreProvider* DataStorage,
	UE::Editor::DataStorage::IUiProvider* DataStorageUi,
	UE::Editor::DataStorage::RowHandle TargetRow, UE::Editor::DataStorage::RowHandle WidgetRow,
	const UE::Editor::DataStorage::FMetaDataView& Arguments)
{
	using namespace UE::Editor::DataStorage;
	if (DataStorage->IsRowAvailable(TargetRow))
	{
		FAttributeBinder Binder(TargetRow, DataStorage);

		bool bIsAsset = DataStorage->HasColumns<FAssetDataColumn_Experimental>(TargetRow);

		TAttribute<FSlateColor> ColorAndOpacityAttribute;
		TAttribute<const FSlateBrush*> FolderImage;

		// Create the LabelWidget through its General Widget
		UE::Editor::DataStorage::RowHandle LabelPurposeName = DataStorageUi->FindPurpose(DataStorageUi->GetGeneralWidgetPurposeID());

		IUiProvider::FWidgetConstructorPtr OutLabelWidgetConstructorPtr;

		auto AssignLabelWidgetToColumn = [&OutLabelWidgetConstructorPtr] (IUiProvider::FWidgetConstructorPtr WidgetConstructor, TConstArrayView<TWeakObjectPtr<const UScriptStruct>>)
		{
			OutLabelWidgetConstructorPtr = MoveTemp(WidgetConstructor);
			return false;
		};

		TArray<TWeakObjectPtr<const UScriptStruct>> LabelColumns = GetLabelColumns();
		DataStorageUi->CreateWidgetConstructors(LabelPurposeName, IUiProvider::EMatchApproach::ExactMatch, LabelColumns, Arguments, AssignLabelWidgetToColumn);

		TSharedPtr<SWidget> LabelWidget = SNullWidget::NullWidget;
		if (OutLabelWidgetConstructorPtr)
		{
			LabelWidget = DataStorageUi->ConstructWidget(WidgetRow, *OutLabelWidgetConstructorPtr, Arguments);
		}

		if (!LabelWidget.IsValid())
		{
			LabelWidget = SNullWidget::NullWidget;
		}

		// For assets, grab the color from the asset definition
		if(bIsAsset)
		{
			ColorAndOpacityAttribute = Binder.BindData(&FAssetDataColumn_Experimental::AssetData, [](const FAssetData& AssetData)
			{
				if(const UAssetDefinition* AssetDefinition = UAssetDefinitionRegistry::Get()->GetAssetDefinitionForAsset(AssetData))
				{
					return FSlateColor(AssetDefinition->GetAssetColor());
				}

				return FSlateColor::UseForeground();
			});
		}
		// For folders, use the color and folder type column directly
		else
		{
			FolderImage = Binder.BindData(&FFolderTypeColumn_Experimental::FolderType, [] (EFolderType InFolderType)
			{
				// Default values
				switch (InFolderType)
				{
					case EFolderType::Developer:
						return FAppStyle::GetBrush(FName(TEXT("ContentBrowser.AssetTreeFolderClosedDeveloper")));

					case EFolderType::PluginRoot:
						return FAppStyle::GetBrush(FName(TEXT("ContentBrowser.AssetTreeFolderClosedPluginRoot")));

					// TODO: Cpp and Virtual are not currently checked, see TedsAssetData PopulatePathDataTableRow for more info
					case EFolderType::Code:
						return FAppStyle::GetBrush(FName(TEXT("ContentBrowser.AssetTreeFolderClosedCode")));

					case EFolderType::CustomVirtual:
						return FAppStyle::GetBrush(FName(TEXT("ContentBrowser.AssetTreeFolderClosedVirtual")));

					case EFolderType::Normal:
					default:
						return FAppStyle::GetBrush(FName(TEXT("ContentBrowser.ColumnViewFolderIcon")));
				}
			});

			ColorAndOpacityAttribute = Binder.BindData(&FSlateColorColumn::Color, FSlateColor::UseForeground());
		}

		static const FMargin ColumnItemPadding(5, 0, 5, 0);

		return SNew(SBox)
				.Padding(ColumnItemPadding)
				[
					SNew(SHorizontalBox)
					+SHorizontalBox::Slot()
					.AutoWidth()
					[
						SNew(SImage)
							.Image(bIsAsset ? FAppStyle::GetBrush(FName(TEXT("ContentBrowser.ColumnViewAssetIcon"))) : FolderImage)
							.ColorAndOpacity(ColorAndOpacityAttribute)
					]
					+SHorizontalBox::Slot()
					.AutoWidth()
					[
						SNew(SSpacer)
							.Size(FVector2D(5.0f, 0.0f))
					]
					+SHorizontalBox::Slot()
					.FillWidth(1.0f)
					[
						LabelWidget.ToSharedRef()
					]
				];
	}
	else
	{
		return SNullWidget::NullWidget;
	}
}

TArray<TWeakObjectPtr<const UScriptStruct>> FAssetDataLabelWidgetConstructor::GetLabelColumns()
{
	static TArray<TWeakObjectPtr<const UScriptStruct>> LabelColumns({ TWeakObjectPtr(FAssetNameColumn::StaticStruct()) });
	return LabelColumns;
}

#undef LOCTEXT_NAMESPACE
