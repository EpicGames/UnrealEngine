// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/AssetDataThumbnailWidget.h"

#include "AssetThumbnail.h"
#include "AssetViewTypes.h"
#include "Elements/Framework/TypedElementAttributeBinding.h"
#include "TedsAssetDataColumns.h"
#include "TedsAssetDataHelper.h"
#include "TedsAssetDataWidgetColumns.h"
#include "Columns/SlateDelegateColumns.h"
#include "Elements/Columns/TypedElementFolderColumns.h"
#include "Elements/Columns/TypedElementHiearchyColumns.h"
#include "Elements/Columns/TypedElementSlateWidgetColumns.h"
#include "Interfaces/IPluginManager.h"
#include "Styling/StyleColors.h"
#include "ThumbnailRendering/ThumbnailManager.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SToolTip.h"
#include "Widgets/Text/STextBlock.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AssetDataThumbnailWidget)

#define LOCTEXT_NAMESPACE "FAssetThumbnailWidgetConstructor"

namespace ThumbnailHelper
{
	template <typename InColumnType>
	UE::Editor::DataStorage::FAttributeBinder GetBinderForColumn(UE::Editor::DataStorage::ICoreProvider* DataStorage,
		UE::Editor::DataStorage::RowHandle WidgetRow,
		UE::Editor::DataStorage::RowHandle ParentWidgetRow)
	{
		
		return DataStorage->IsRowAvailable(ParentWidgetRow) && !DataStorage->GetColumn<InColumnType>(WidgetRow) ?
			UE::Editor::DataStorage::FAttributeBinder(ParentWidgetRow, DataStorage) :
			UE::Editor::DataStorage::FAttributeBinder(WidgetRow, DataStorage);
	}
}

namespace Private
{
	bool IsTopLevelFolder(const FStringView InFolderPath)
	{
		int32 SlashCount = 0;
		for (const TCHAR PathChar : InFolderPath)
		{
			if (PathChar == TEXT('/'))
			{
				if (++SlashCount > 1)
				{
					break;
				}
			}
		}

		return SlashCount == 1;
	}

	void AddToToolTipInfoBox(const TSharedRef<SVerticalBox>& InfoBox, const FText& Key, const FText& Value)
	{
		InfoBox->AddSlot()
        	.Padding(0.f, 0.f, 0.f, 6.f)
        	.AutoHeight()
        	[
        		SNew(SHorizontalBox)
        		+SHorizontalBox::Slot()
        		.AutoWidth()
        		.Padding(0, 0, 4, 0)
        		[
        			SNew(STextBlock)
        			.Font(FAppStyle::GetFontStyle("ContentBrowser.Tooltip.EntryFont"))
        			.Text(FText::Format(NSLOCTEXT("AssetThumbnailToolTip", "AssetViewTooltipFormat", "{0}:"), Key))
        		]
        
        		+SHorizontalBox::Slot()
        		.AutoWidth()
        		[
        			SNew(STextBlock)
        			.Font(FAppStyle::GetFontStyle("ContentBrowser.Tooltip.EntryFont"))
        			.ColorAndOpacity(FStyleColors::White)
        			.Text(Value)
        		]
        	];
	}

	TSharedRef<SWidget> GetFolderTooltip(UE::Editor::DataStorage::ICoreProvider* InDataStorage, UE::Editor::DataStorage::RowHandle InTargetRow, TAttribute<const FSlateBrush*> InFolderImage)
	{
		UE::Editor::DataStorage::FAttributeBinder Binder(InTargetRow, InDataStorage);

		// Create a box to hold every line of info in the body of the tooltip
		TSharedRef<SVerticalBox> InfoBox = SNew(SVerticalBox);

		if (const FAssetPathColumn_Experimental* AssetPathColumn = InDataStorage->GetColumn<FAssetPathColumn_Experimental>(InTargetRow))
		{
			const FText FolderPath = FText::FromName(AssetPathColumn->Path);
			AddToToolTipInfoBox(InfoBox, LOCTEXT("TileViewTooltipPath", "Path"), FolderPath);
		}

		if (const FAssetPathColumn_Experimental* InternalPathColumn = InDataStorage->GetColumn<FAssetPathColumn_Experimental>(InTargetRow))
		{
			const FName InternalPath = InternalPathColumn->Path;
			if (!InternalPath.IsNone())
			{
				FNameBuilder FolderPathBuilder(InternalPath);
				if (IsTopLevelFolder(FStringView(FolderPathBuilder)))
				{
					FStringView PluginName(FolderPathBuilder);
					PluginName.RightChopInline(1);

					if (TSharedPtr<IPlugin> Plugin = IPluginManager::Get().FindPlugin(PluginName))
					{
						if (Plugin->GetDescriptor().Description.Len() > 0)
						{
							AddToToolTipInfoBox(InfoBox, LOCTEXT("TileViewTooltipPluginDescription", "Plugin Description"), FText::FromString(Plugin->GetDescriptor().Description));
						}
					}
				}
			}
		}

		return SNew(SVerticalBox)

				+ SVerticalBox::Slot()
				.AutoHeight()
				[
					SNew(SVerticalBox)

					+ SVerticalBox::Slot()
					.AutoHeight()
					.Padding(0.f, 0.f, 0.f, 6.f)
					[
						SNew(SHorizontalBox)

						+ SHorizontalBox::Slot()
						.AutoWidth()
						.VAlign(VAlign_Center)
						[
							SNew(STextBlock)
							.Text(Binder.BindData(&FAssetNameColumn::Name, [] (FName InName)
							{
								return FText::FromString(TedsAssetDataHelper::RemoveSlashFromStart(InName.ToString()));
							}))
							.ColorAndOpacity(FStyleColors::White)
							.Font(FAppStyle::GetFontStyle("ContentBrowser.Tooltip.EntryFont"))
						]
					]

					+ SVerticalBox::Slot()
					.AutoHeight()
					[
						SNew(SHorizontalBox)

						+ SHorizontalBox::Slot()
						.AutoWidth()
						.VAlign(VAlign_Center)
						.Padding(0.f, 0.f, 4.f, 0.f)
						[
							SNew(SBox)
							.WidthOverride(16.f)
							.HeightOverride(16.f)
							[
								SNew(SImage)
								.ColorAndOpacity(Binder.BindData(&FSlateColorColumn::Color))
								.Image(InFolderImage)
							]
						]

						+ SHorizontalBox::Slot()
						.AutoWidth()
						.VAlign(VAlign_Center)
						[
							SNew(STextBlock)
							.Font(FAppStyle::GetFontStyle("ContentBrowser.Tooltip.EntryFont"))
							.Text(LOCTEXT("FolderNameBracketedLabel", "Folder"))
						]
					]
				]

				+ SVerticalBox::Slot()
				.Padding(FMargin(0.f,6.f))
				.AutoHeight()
				[
					SNew(SSeparator)
					.Orientation(Orient_Horizontal)
					.Thickness(1.f)
					.ColorAndOpacity(COLOR("#484848FF"))
					.SeparatorImage(FAppStyle::Get().GetBrush("WhiteBrush"))
				]

				+ SVerticalBox::Slot()
				.AutoHeight()
				[
					InfoBox
				];
	}

	class SFolderItemToolTip : public SToolTip
	{
	public:
		SLATE_BEGIN_ARGS(SFolderItemToolTip)
		{ }

			SLATE_ARGUMENT(UE::Editor::DataStorage::RowHandle, TargetRow)
			SLATE_ARGUMENT(UE::Editor::DataStorage::RowHandle, WidgetRow)
			SLATE_ARGUMENT(UE::Editor::DataStorage::ICoreProvider*, DataStorage)
			SLATE_ATTRIBUTE(const FSlateBrush*, FolderImage)

		SLATE_END_ARGS()

		void Construct(const FArguments& InArgs)
		{
			DataStorage = InArgs._DataStorage;
			TargetRow = InArgs._TargetRow;
			WidgetRow = InArgs._WidgetRow;
			FolderImage = InArgs._FolderImage;

			SToolTip::Construct(
				SToolTip::FArguments()
				.TextMargin(FMargin(12.f, 8.f, 12.f, 2.f))
				.BorderImage(FAppStyle::GetBrush("AssetThumbnail.Tooltip.Border")));
		}

		// IToolTip interface
		virtual bool IsEmpty() const override
		{
			return	TargetRow == UE::Editor::DataStorage::InvalidRowHandle ||
					WidgetRow == UE::Editor::DataStorage::InvalidRowHandle ||
					DataStorage == nullptr ||
					!FolderImage.IsSet();
		}

		virtual void OnOpening() override
		{
			SetContentWidget(Private::GetFolderTooltip(DataStorage, TargetRow, FolderImage));

			// When opening the tooltip update it on the Row as well
			if (FLocalWidgetTooltipColumn_Experimental* WidgetTooltipColumn = DataStorage->GetColumn<FLocalWidgetTooltipColumn_Experimental>(WidgetRow))
			{
				WidgetTooltipColumn->Tooltip = SharedThis(this);
			}
		}

		virtual void OnClosed() override
		{
			ResetContentWidget();
		}

	private:
		UE::Editor::DataStorage::RowHandle TargetRow = UE::Editor::DataStorage::InvalidRowHandle;
		UE::Editor::DataStorage::RowHandle WidgetRow = UE::Editor::DataStorage::InvalidRowHandle;
		UE::Editor::DataStorage::ICoreProvider* DataStorage = nullptr;
		TAttribute<const FSlateBrush*> FolderImage;
	};
}

void UAssetThumbnailWidgetFactory::RegisterWidgetPurposes(UE::Editor::DataStorage::IUiProvider& DataStorageUi) const
{
	DataStorageUi.RegisterWidgetPurpose(
	UE::Editor::DataStorage::IUiProvider::FPurposeInfo("ContentBrowser", "Thumbnail", NAME_None,
		UE::Editor::DataStorage::IUiProvider::EPurposeType::UniqueByNameAndColumn,
		LOCTEXT("CBThumbnailPurpose", "Specific purpose display thumbnails in the CB.")));
}


void UAssetThumbnailWidgetFactory::RegisterWidgetConstructors(UE::Editor::DataStorage::ICoreProvider& DataStorage,
                                                              UE::Editor::DataStorage::IUiProvider& DataStorageUi) const
{
	using namespace UE::Editor::DataStorage::Queries;
	DataStorageUi.RegisterWidgetFactory<FAssetThumbnailWidgetConstructor>(
		DataStorageUi.FindPurpose(IUiProvider::FPurposeInfo("ContentBrowser", "Thumbnail", NAME_None).GeneratePurposeID()),
		TColumn<FAssetTag>() || TColumn<FFolderTag>());
}

FAssetThumbnailWidgetConstructor::FAssetThumbnailWidgetConstructor()
	: FSimpleWidgetConstructor(StaticStruct())
{
}

FAssetThumbnailWidgetConstructor::FAssetThumbnailWidgetConstructor(const UScriptStruct* TypeInfo)
	: FSimpleWidgetConstructor(TypeInfo)
{
}

TSharedPtr<SWidget> FAssetThumbnailWidgetConstructor::CreateWidget(UE::Editor::DataStorage::ICoreProvider* DataStorage,
	UE::Editor::DataStorage::IUiProvider* DataStorageUi,
	UE::Editor::DataStorage::RowHandle TargetRow, UE::Editor::DataStorage::RowHandle WidgetRow,
	const UE::Editor::DataStorage::FMetaDataView& Arguments)
{
	UE::Editor::DataStorage::FAttributeBinder Binder(TargetRow, DataStorage);
	UE::Editor::DataStorage::FAttributeBinder WidgetRowBinder(WidgetRow, DataStorage);

	bool bIsAsset = DataStorage->HasColumns<FAssetTag>(TargetRow);
	UE::Editor::DataStorage::RowHandle ParentWidgetRowHandle = UE::Editor::DataStorage::InvalidRowHandle;
	if (FTableRowParentColumn* ParentWidgetRow = DataStorage->GetColumn<FTableRowParentColumn>(WidgetRow))
	{
		ParentWidgetRowHandle = ParentWidgetRow->Parent;
	}

	float ThumbnailSizeOffset = 0.f;
	UE::Editor::DataStorage::FMetaDataEntryView ThumbnailSizeOffsetMetaData = Arguments.FindGeneric(TedsAssetDataHelper::MetaDataNames::GetThumbnailSizeOffsetMetaDataName());
	if (ThumbnailSizeOffsetMetaData.IsSet() && ThumbnailSizeOffsetMetaData.IsType<double>())
	{
		ThumbnailSizeOffset = (float)(*ThumbnailSizeOffsetMetaData.TryGetExact<double>());
	}
	constexpr int32 DefaultThumbnailSize = 64;


	UE::Editor::DataStorage::FAttributeBinder PaddingBinder = ThumbnailHelper::GetBinderForColumn<FWidgetPaddingColumn_Experimental>(DataStorage, WidgetRow, ParentWidgetRowHandle);
	UE::Editor::DataStorage::FAttributeBinder SizeBinder = ThumbnailHelper::GetBinderForColumn<FSizeValueColumn_Experimental>(DataStorage, WidgetRow, ParentWidgetRowHandle);

	// Thumbnail Box container
	const TSharedRef<SBox> ThumbnailBox = SNew(SBox)
		.Padding(PaddingBinder.BindData(&FWidgetPaddingColumn_Experimental::Padding, FMargin(0.f)))
		.WidthOverride(SizeBinder.BindData(&FSizeValueColumn_Experimental::SizeValue, [DefaultThumbnailSize, ThumbnailSizeOffset] (float InSizeValue)
		{
			const float Size = FMath::IsNearlyZero(InSizeValue) ? DefaultThumbnailSize : InSizeValue;
			return FOptionalSize(Size + ThumbnailSizeOffset);
		}))
		.HeightOverride(SizeBinder.BindData(&FSizeValueColumn_Experimental::SizeValue, [DefaultThumbnailSize, ThumbnailSizeOffset] (float InSizeValue)
		{
			const float Size = FMath::IsNearlyZero(InSizeValue) ? DefaultThumbnailSize : InSizeValue;
			return FOptionalSize(Size + ThumbnailSizeOffset);
		}));

	// For assets, grab the color from the asset definition
	if (bIsAsset)
	{
		// Retrieve the AssetData, used to create the thumbnail
		FAssetData AssetDataToUse = FAssetData();
		if (const FAssetDataColumn_Experimental* AssetDataColumn = DataStorage->GetColumn<FAssetDataColumn_Experimental>(TargetRow))
		{
			AssetDataToUse = AssetDataColumn->AssetData;
		}

		// Thumbnail Configuration Args
		FAssetThumbnailConfig ThumbnailConfig;
		UE::Editor::DataStorage::FMetaDataEntryView bAllowFadeInMeta = Arguments.FindGeneric(TedsAssetDataHelper::MetaDataNames::GetThumbnailFadeInMetaDataName());
		if (bAllowFadeInMeta.IsSet() && bAllowFadeInMeta.IsType<bool>())
		{
			ThumbnailConfig.bAllowFadeIn = *bAllowFadeInMeta.TryGetExact<bool>();
		}

		UE::Editor::DataStorage::FMetaDataEntryView bAllowHintTextMeta = Arguments.FindGeneric(TedsAssetDataHelper::MetaDataNames::GetThumbnailHintTextMetaDataName());
		if (bAllowHintTextMeta.IsSet() && bAllowHintTextMeta.IsType<bool>())
		{
			ThumbnailConfig.bAllowHintText = *bAllowHintTextMeta.TryGetExact<bool>();
		}

		UE::Editor::DataStorage::FMetaDataEntryView bAllowRealTimeOnHoveredMeta = Arguments.FindGeneric(TedsAssetDataHelper::MetaDataNames::GetThumbnailRealTimeOnHoveredMetaDataName());
		if (bAllowRealTimeOnHoveredMeta.IsSet() && bAllowRealTimeOnHoveredMeta.IsType<bool>())
		{
			ThumbnailConfig.bAllowRealTimeOnHovered = *bAllowRealTimeOnHoveredMeta.TryGetExact<bool>();
		}

		if (DataStorage->HasColumns<FOnGetWidgetSlateBrushColumn_Experimental>(WidgetRow))
		{
			ThumbnailConfig.AssetBorderImageOverride = WidgetRowBinder.BindEventAttribute(&FOnGetWidgetSlateBrushColumn_Experimental::OnGetWidgetSlateBrush, FAppStyle::GetNoBrush());
		}

		// TODO: AssetItem->GetItem().GetItemTemporaryReason() == EContentBrowserItemFlags::Temporary_Creation;
		// This was the previous check to assign bForceGenericThumbnail, for now it will use the Generic if the AssetData is not valid
		ThumbnailConfig.bForceGenericThumbnail = !AssetDataToUse.IsValid();
		ThumbnailConfig.AllowAssetSpecificThumbnailOverlay = !ThumbnailConfig.bForceGenericThumbnail;
		ThumbnailConfig.ThumbnailLabel = EThumbnailLabel::ClassName;

		ThumbnailConfig.GenericThumbnailSize = SizeBinder.BindData(&FSizeValueColumn_Experimental::SizeValue, [DefaultThumbnailSize] (float InThumbnailSize)
		{
			return FMath::IsNearlyZero(InThumbnailSize) ? DefaultThumbnailSize : static_cast<int32>(InThumbnailSize);
		});

		// TODO: This data will need to be integrated in TEDS and later on remove this/update it to use TEDS instead to retrieve the data needed
		// ThumbnailConfig.AssetSystemInfoProvider = nullptr;
		UE::Editor::DataStorage::FMetaDataEntryView bAllowAssetStatusThumbnailOverlayMeta = Arguments.FindGeneric(TedsAssetDataHelper::MetaDataNames::GetThumbnailStatusMetaDataName());
		if (bAllowAssetStatusThumbnailOverlayMeta.IsSet() && bAllowAssetStatusThumbnailOverlayMeta.IsType<bool>())
		{
			ThumbnailConfig.AllowAssetStatusThumbnailOverlay = *bAllowAssetStatusThumbnailOverlayMeta.TryGetExact<bool>();
		}
		ThumbnailConfig.ShowAssetColor = true;
		// TODO: If we are able to change the thumbnail in Teds view we would need to bind this to make it work

		UE::Editor::DataStorage::FMetaDataEntryView bCanDisplayEditModePrimitiveTools = Arguments.FindGeneric(TedsAssetDataHelper::MetaDataNames::GetThumbnailCanDisplayEditModePrimitiveTools());
		if (bCanDisplayEditModePrimitiveTools.IsSet() && bCanDisplayEditModePrimitiveTools.IsType<bool>())
		{
			ThumbnailConfig.bCanDisplayEditModePrimitiveTools = *bCanDisplayEditModePrimitiveTools.TryGetExact<bool>();
		}

		UE::Editor::DataStorage::FAttributeBinder ThumbnailEditModeBinder = ThumbnailHelper::GetBinderForColumn<FThumbnailEditModeColumn_Experimental>(DataStorage, WidgetRow, ParentWidgetRowHandle);
		ThumbnailConfig.IsEditModeVisible = ThumbnailEditModeBinder.BindData(&FThumbnailEditModeColumn_Experimental::IsEditModeToggled, [] (bool IsEditModeToggled)
		{
			return IsEditModeToggled ? EVisibility::Visible : EVisibility::Collapsed;
		});

		// TODO: Consider caching them instead, to avoid creating them everytime
		constexpr uint32 ThumbnailResolution = 256;
		const TSharedRef<FAssetThumbnail> AssetThumbnail = MakeShared<FAssetThumbnail>(AssetDataToUse, ThumbnailResolution, ThumbnailResolution, UThumbnailManager::Get().GetSharedThumbnailPool());

		TSharedRef<SWidget> ThumbnailWidget = AssetThumbnail->MakeThumbnailWidget(ThumbnailConfig);
		ThumbnailBox->SetContent(ThumbnailWidget);

		// If the widget creating this is interested in the ThumbnailTooltip it has to add the column itself to avoid adding them if not used
		if (FLocalWidgetTooltipColumn_Experimental* WidgetTooltipColumn = DataStorage->GetColumn<FLocalWidgetTooltipColumn_Experimental>(WidgetRow))
		{
			WidgetTooltipColumn->Tooltip = ThumbnailWidget->GetToolTip();
		}
	}
	// For folders, use the color and folder type column directly
	else
	{
		TAttribute<const FSlateBrush*> FolderImage;

		FolderImage = Binder.BindData(&FFolderTypeColumn_Experimental::FolderType, [] (EFolderType InFolderType)
		{
			// Default values
			switch (InFolderType)
			{
				case EFolderType::Developer:
					return FAppStyle::GetBrush(FName(TEXT("ContentBrowser.ListViewDeveloperFolderIcon")));

				case EFolderType::PluginRoot:
					return FAppStyle::GetBrush(FName(TEXT("ContentBrowser.ListViewPluginFolderIcon")));

				// TODO: Cpp and Virtual are not currently checked, see TedsAssetData PopulatePathDataTableRow for more info
				case EFolderType::Code:
					return FAppStyle::GetBrush(FName(TEXT("ContentBrowser.ListViewCodeFolderIcon")));

				case EFolderType::CustomVirtual:
					return FAppStyle::GetBrush(FName(TEXT("ContentBrowser.ListViewVirtualFolderIcon")));

				case EFolderType::Normal:
				default:
					return FAppStyle::GetBrush(FName(TEXT("ContentBrowser.ListViewFolderIcon")));
			}
		});

		TAttribute<const FSlateBrush*> ShadowFolderImage;

		ShadowFolderImage = Binder.BindData(&FFolderTypeColumn_Experimental::FolderType, [] (EFolderType InFolderType)
		{
			// Default values
			switch (InFolderType)
			{
				// TODO: Cpp and Virtual are not currently checked, see TedsAssetData PopulatePathDataTableRow for more info
				case EFolderType::CustomVirtual:
					return FAppStyle::GetBrush(FName(TEXT("ContentBrowser.ListViewVirtualFolderShadow")));

				case EFolderType::Developer:
				case EFolderType::PluginRoot:
				case EFolderType::Code:
				case EFolderType::Normal:
				default:
					return FAppStyle::GetBrush(FName(TEXT("ContentBrowser.FolderItem.DropShadow")));
			}
		});

		ThumbnailBox->SetContent(
			SNew(SBorder)
			.BorderImage(ShadowFolderImage)
			.Padding(FMargin(0,0,2.0f,2.0f))
			[
				SNew(SImage)
				.Image(FolderImage)
				.ColorAndOpacity(Binder.BindData(&FSlateColorColumn::Color, FSlateColor::UseForeground()))
			]);

		ThumbnailBox->SetToolTip(SNew(Private::SFolderItemToolTip)
				.TargetRow(TargetRow)
				.WidgetRow(WidgetRow)
				.DataStorage(DataStorage)
				.FolderImage(FolderImage));
	}

	return ThumbnailBox;
}

#undef LOCTEXT_NAMESPACE
