// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/Interfaces/TypedElementDataStorageFactory.h"
#include "Elements/Interfaces/TypedElementDataStorageUiInterface.h"
#include "Internationalization/Text.h"
#include "UObject/ObjectMacros.h"

#include "AssetDataThumbnailWidget.generated.h"

class UScriptStruct;

UCLASS()
class UAssetThumbnailWidgetFactory : public UEditorDataStorageFactory
{
	GENERATED_BODY()

public:
	virtual ~UAssetThumbnailWidgetFactory() override = default;

	TEDSASSETDATA_API virtual void RegisterWidgetPurposes(UE::Editor::DataStorage::IUiProvider& DataStorageUi) const override;
	TEDSASSETDATA_API virtual void RegisterWidgetConstructors(UE::Editor::DataStorage::ICoreProvider& DataStorage,
		UE::Editor::DataStorage::IUiProvider& DataStorageUi) const override;
};

// Thumbnail widget for assets in TEDS
USTRUCT()
struct FAssetThumbnailWidgetConstructor : public FSimpleWidgetConstructor
{
	GENERATED_BODY()

public:
	TEDSASSETDATA_API FAssetThumbnailWidgetConstructor();
	TEDSASSETDATA_API explicit FAssetThumbnailWidgetConstructor(const UScriptStruct* TypeInfo);
	virtual ~FAssetThumbnailWidgetConstructor() override = default;

	TEDSASSETDATA_API virtual TSharedPtr<SWidget> CreateWidget(
		UE::Editor::DataStorage::ICoreProvider* DataStorage,
		UE::Editor::DataStorage::IUiProvider* DataStorageUi,
		UE::Editor::DataStorage::RowHandle TargetRow,
		UE::Editor::DataStorage::RowHandle WidgetRow, 
		const UE::Editor::DataStorage::FMetaDataView& Arguments) override;
};
