// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/AssetDataVirtualPathWidget.h"

#include "Elements/Framework/TypedElementAttributeBinding.h"
#include "Internationalization/Text.h"
#include "TedsAssetDataColumns.h"
#include "TedsAssetDataHelper.h"
#include "UObject/ObjectMacros.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Text/STextBlock.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AssetDataVirtualPathWidget)


namespace UE::Editor::DataStorage
{
	int32 FAssetDataVirtualPathWidgetSorter::Compare(const ICoreProvider& Storage, RowHandle Left, RowHandle Right) const
	{
		FString LeftPath;
		bool bLeftPathFound = ExtractString(LeftPath, Storage, Left);
		FString RightPath;
		bool bRightPathFound = ExtractString(RightPath, Storage, Right);

		return (bLeftPathFound && bRightPathFound) 
			? LeftPath.Compare(RightPath, ESearchCase::IgnoreCase)
			: (static_cast<int32>(bLeftPathFound) - static_cast<int32>(bRightPathFound));
	}

	FPrefixInfo FAssetDataVirtualPathWidgetSorter::CalculatePrefix(
		const ICoreProvider& Storage, RowHandle Row, uint32 ByteIndex) const
	{
		FString Path;
		return ExtractString(Path, Storage, Row) 
			? CreateSortPrefix(ByteIndex, TSortStringView(FSortCaseInsensitive{}, Path))
			: FPrefixInfo{};
	}

	bool FAssetDataVirtualPathWidgetSorter::ExtractString(FString& Result, const ICoreProvider& Storage, RowHandle Row) const
	{
		bool bIsAsset = Storage.HasColumns<FAssetTag>(Row);
		if (bIsAsset)
		{
			if (const FVirtualPathColumn_Experimental* VirtualPathColumn = Storage.GetColumn<FVirtualPathColumn_Experimental>(Row))
			{
				if (!VirtualPathColumn->VirtualPath.IsNone())
				{
					Result = VirtualPathColumn->VirtualPath.ToString();
					Result = TedsAssetDataHelper::RemoveSlashFromStart(Result);
				}
				return true;
			}
		}
		else
		{
			if (const FAssetPathColumn_Experimental* AssetPathColumn = Storage.GetColumn<FAssetPathColumn_Experimental>(Row))
			{
				if (!AssetPathColumn->Path.IsNone())
				{
					Result = AssetPathColumn->Path.ToString();
					Result = TedsAssetDataHelper::RemoveSlashFromStart(Result);
					Result = TedsAssetDataHelper::RemoveAllFromLastSlash(Result);
				}
				return true;
			}
		}
		return false;
	}

	FColumnSorterInterface::ESortType FAssetDataVirtualPathWidgetSorter::GetSortType() const
	{
		return ESortType::HybridSort;
	}

	FText FAssetDataVirtualPathWidgetSorter::GetShortName() const
	{
		return FText::GetEmpty();
	}
}

void UAssetDataVirtualPathWidgetFactory::RegisterWidgetConstructors(UE::Editor::DataStorage::ICoreProvider& DataStorage,
	UE::Editor::DataStorage::IUiProvider& DataStorageUi) const
{
	using namespace UE::Editor::DataStorage::Queries;
	DataStorageUi.RegisterWidgetFactory<FAssetDataVirtualPathWidgetConstructor>(
		DataStorageUi.FindPurpose(DataStorageUi.GetGeneralWidgetPurposeID()),
		TColumn<FVirtualPathColumn_Experimental>() || TColumn<FAssetPathColumn_Experimental>());
}

FAssetDataVirtualPathWidgetConstructor::FAssetDataVirtualPathWidgetConstructor()
	: FSimpleWidgetConstructor(StaticStruct())
{
}

TSharedPtr<SWidget> FAssetDataVirtualPathWidgetConstructor::CreateWidget(UE::Editor::DataStorage::ICoreProvider* DataStorage,
	UE::Editor::DataStorage::IUiProvider* DataStorageUi, UE::Editor::DataStorage::RowHandle TargetRow, UE::Editor::DataStorage::RowHandle WidgetRow,
	const UE::Editor::DataStorage::FMetaDataView& Arguments)
{
	static const FMargin ColumnItemPadding(5, 0, 5, 0);

	bool bIsAsset = DataStorage->HasColumns<FAssetTag>(TargetRow);
	TAttribute<FText> PathText = FText::GetEmpty();

	if (bIsAsset)
	{
		if (const FVirtualPathColumn_Experimental* VirtualPathColumn = DataStorage->GetColumn<FVirtualPathColumn_Experimental>(TargetRow))
		{
			FString VirtualPathString = VirtualPathColumn->VirtualPath.ToString();
			VirtualPathString = TedsAssetDataHelper::RemoveSlashFromStart(VirtualPathString);
			PathText = FText::FromString(VirtualPathString);
		}
	}
	else
	{
		if (const FAssetPathColumn_Experimental* AssetPathColumn = DataStorage->GetColumn<FAssetPathColumn_Experimental>(TargetRow))
		{
			FString AssetPathString = AssetPathColumn->Path.ToString();
			AssetPathString = TedsAssetDataHelper::RemoveSlashFromStart(AssetPathString);
			AssetPathString = TedsAssetDataHelper::RemoveAllFromLastSlash(AssetPathString);
			PathText = FText::FromString(AssetPathString);
		}
	}

	return SNew(SBox)
			.Padding(ColumnItemPadding)
			[
				SNew(STextBlock)
				.Text(PathText)
			];
}

TArray<TSharedPtr<const UE::Editor::DataStorage::FColumnSorterInterface>> FAssetDataVirtualPathWidgetConstructor::ConstructColumnSorters(
	UE::Editor::DataStorage::ICoreProvider* DataStorage,
	UE::Editor::DataStorage::IUiProvider* DataStorageUi,
	const UE::Editor::DataStorage::FMetaDataView& Arguments)
{
	using namespace UE::Editor::DataStorage;
	return TArray<TSharedPtr<const FColumnSorterInterface>>(
		{
			MakeShared<FAssetDataVirtualPathWidgetSorter>()
		});
}
