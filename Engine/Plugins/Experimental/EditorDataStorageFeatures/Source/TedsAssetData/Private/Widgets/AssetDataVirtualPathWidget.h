// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/Interfaces/TypedElementDataStorageFactory.h"
#include "Elements/Interfaces/TypedElementDataStorageUiInterface.h"

#include "AssetDataVirtualPathWidget.generated.h"

namespace UE::Editor::DataStorage
{
	class FAssetDataVirtualPathWidgetSorter final : public FColumnSorterInterface
	{
	public:
		virtual int32 Compare(const ICoreProvider& Storage, RowHandle Left, RowHandle Right) const override;
		virtual FPrefixInfo CalculatePrefix(const ICoreProvider& Storage, RowHandle Row, uint32 ByteIndex) const override;
		virtual ESortType GetSortType() const override;
		virtual FText GetShortName() const override;

	private:
		bool ExtractString(FString& Result, const ICoreProvider& Storage, RowHandle Row) const;
	};
}

UCLASS()
class UAssetDataVirtualPathWidgetFactory : public UEditorDataStorageFactory
{
	GENERATED_BODY()

public:
	~UAssetDataVirtualPathWidgetFactory() override = default;

	void RegisterWidgetConstructors(UE::Editor::DataStorage::ICoreProvider& DataStorage,
		UE::Editor::DataStorage::IUiProvider& DataStorageUi) const override;
};

// Widget to show disk size in bytes
USTRUCT()
struct FAssetDataVirtualPathWidgetConstructor : public FSimpleWidgetConstructor
{
	GENERATED_BODY()

public:
	FAssetDataVirtualPathWidgetConstructor();
	~FAssetDataVirtualPathWidgetConstructor() override = default;

	virtual TSharedPtr<SWidget> CreateWidget(
		UE::Editor::DataStorage::ICoreProvider* DataStorage,
		UE::Editor::DataStorage::IUiProvider* DataStorageUi,
		UE::Editor::DataStorage::RowHandle TargetRow,
		UE::Editor::DataStorage::RowHandle WidgetRow,
		const UE::Editor::DataStorage::FMetaDataView& Arguments) override;

	virtual TArray<TSharedPtr<const UE::Editor::DataStorage::FColumnSorterInterface>> ConstructColumnSorters(
		UE::Editor::DataStorage::ICoreProvider* DataStorage,
		UE::Editor::DataStorage::IUiProvider* DataStorageUi,
		const UE::Editor::DataStorage::FMetaDataView& Arguments) override;
};
