// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/AssetLabelWidget.h"

#include "Elements/Framework/TypedElementAttributeBinding.h"
#include "TedsAssetDataColumns.h"
#include "TedsAssetDataHelper.h"
#include "Widgets/Text/STextBlock.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AssetLabelWidget)

#define LOCTEXT_NAMESPACE "FAssetLabelWidgetConstructor"

void UAssetLabelWidgetFactory::RegisterWidgetConstructors(UE::Editor::DataStorage::ICoreProvider& DataStorage,
	UE::Editor::DataStorage::IUiProvider& DataStorageUi) const
{
	using namespace UE::Editor::DataStorage::Queries;
	DataStorageUi.RegisterWidgetFactory<FAssetLabelWidgetConstructor>(
		DataStorageUi.FindPurpose(IUiProvider::FPurposeInfo("ContentBrowser", "RowLabel", NAME_None).GeneratePurposeID()),
		TColumn<FAssetNameColumn>());
}

FAssetLabelWidgetConstructor::FAssetLabelWidgetConstructor()
	: FSimpleWidgetConstructor(StaticStruct())
{
}

FAssetLabelWidgetConstructor::FAssetLabelWidgetConstructor(const UScriptStruct* TypeInfo)
	: FSimpleWidgetConstructor(TypeInfo)
{
}

TSharedPtr<SWidget> FAssetLabelWidgetConstructor::CreateWidget(UE::Editor::DataStorage::ICoreProvider* DataStorage,
	UE::Editor::DataStorage::IUiProvider* DataStorageUi,
	UE::Editor::DataStorage::RowHandle TargetRow, UE::Editor::DataStorage::RowHandle WidgetRow,
	const UE::Editor::DataStorage::FMetaDataView& Arguments)
{
	UE::Editor::DataStorage::FAttributeBinder Binder(TargetRow, DataStorage);

	TAttribute<FText> AssetName;

	if (const FAssetNameColumn* AssetNameColumn = DataStorage->GetColumn<FAssetNameColumn>(TargetRow))
	{
		const FString AssetNameString = TedsAssetDataHelper::RemoveSlashFromStart(AssetNameColumn->Name.ToString());
		AssetName = FText::FromString(AssetNameString);
	}

	return SNew(STextBlock)
				.Text(AssetName)
				.OverflowPolicy(ETextOverflowPolicy::MiddleEllipsis)
				.ToolTipText(Binder.BindTextFormat(
								LOCTEXT("AssetLabelTooltip", 
									"{Name}\n\nVirtual path: {VirtualPath}\n  Asset path: {AssetPath}\n  Verse path: {VersePath}"))
								.Arg(TEXT("Name"), &FAssetNameColumn::Name)
								.Arg(TEXT("VirtualPath"), &FVirtualPathColumn_Experimental::VirtualPath, LOCTEXT("PathNotSet", "<not set>"))
								.Arg(TEXT("AssetPath"), &FAssetPathColumn_Experimental::Path, LOCTEXT("PathNotSet", "<not set>"))
								.Arg(TEXT("VersePath"), &FVersePathColumn::VersePath, 
									[](const UE::Core::FVersePath& Path) 
									{
										return FText::FromStringView(Path.AsStringView());
									}, 
									LOCTEXT("PathNotSet", "<not set>")));
}

#undef LOCTEXT_NAMESPACE
