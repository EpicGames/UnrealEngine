// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/Interfaces/TypedElementDataStorageFactory.h"
#include "Elements/Interfaces/TypedElementDataStorageUiInterface.h"
#include "Internationalization/Text.h"
#include "UObject/ObjectMacros.h"

#include "AssetLabelWidget.generated.h"

class UScriptStruct;

UCLASS()
class UAssetLabelWidgetFactory : public UEditorDataStorageFactory
{
	GENERATED_BODY()

public:
	virtual ~UAssetLabelWidgetFactory() override = default;

	TEDSASSETDATA_API virtual void RegisterWidgetConstructors(UE::Editor::DataStorage::ICoreProvider& DataStorage,
		UE::Editor::DataStorage::IUiProvider& DataStorageUi) const override;
};

// Label for assets in TEDS
USTRUCT()
struct FAssetLabelWidgetConstructor : public FSimpleWidgetConstructor
{
	GENERATED_BODY()

public:
	TEDSASSETDATA_API FAssetLabelWidgetConstructor();
	TEDSASSETDATA_API explicit FAssetLabelWidgetConstructor(const UScriptStruct* TypeInfo);
	virtual ~FAssetLabelWidgetConstructor() override = default;

	TEDSASSETDATA_API virtual TSharedPtr<SWidget> CreateWidget(
		UE::Editor::DataStorage::ICoreProvider* DataStorage,
		UE::Editor::DataStorage::IUiProvider* DataStorageUi,
		UE::Editor::DataStorage::RowHandle TargetRow,
		UE::Editor::DataStorage::RowHandle WidgetRow, 
		const UE::Editor::DataStorage::FMetaDataView& Arguments) override;
};
