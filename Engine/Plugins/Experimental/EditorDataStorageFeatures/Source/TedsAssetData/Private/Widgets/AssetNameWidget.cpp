// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/AssetNameWidget.h"

#include "TedsAssetDataColumns.h"
#include "Elements/Framework/TypedElementAttributeBinding.h"
#include "Internationalization/Internationalization.h"
#include "Widgets/Text/STextBlock.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AssetNameWidget)

#define LOCTEXT_NAMESPACE "AssetNameWidget"

namespace UE::Editor::DataStorage
{
	//
	// No slash
	//

	FText FAssetNameWidgetSorter_NoSlash::GetShortName() const
	{
		return LOCTEXT("FAssetNameWidgetSorter_NoSlashName", "No slash");
	}
	
	int32 FAssetNameWidgetSorter_NoSlash::Compare(const FAssetNameColumn& Left, const FAssetNameColumn& Right) const
	{
		return TSortNameView(TSortByName<ESortByNameFlags::RemoveLeadingSlash>{}, Left.Name).Compare(Right.Name);
	}

	FPrefixInfo FAssetNameWidgetSorter_NoSlash::CalculatePrefix(const FAssetNameColumn& Column, uint32 ByteIndex) const
	{
		return CreateSortPrefix(ByteIndex, TSortNameView(TSortByName<ESortByNameFlags::RemoveLeadingSlash>{}, Column.Name));
	}

	//
	// With slash
	//

	FText FAssetNameWidgetSorter_WithSlash::GetShortName() const
	{
		return LOCTEXT("FAssetNameWidgetSorter_WithSlashName", "With slash");
	}
	
	int32 FAssetNameWidgetSorter_WithSlash::Compare(const FAssetNameColumn& Left, const FAssetNameColumn& Right) const
	{
		return TSortNameView(TSortByName<>{}, Left.Name).Compare(Right.Name);
	}

	FPrefixInfo FAssetNameWidgetSorter_WithSlash::CalculatePrefix(const FAssetNameColumn& Column, uint32 ByteIndex) const
	{
		return CreateSortPrefix(ByteIndex, TSortNameView(TSortByName<>{}, Column.Name));
	}
} // namespace UE::Editor::DataStorage

void UAssetNameWidgetFactory::RegisterWidgetConstructors(UE::Editor::DataStorage::ICoreProvider& DataStorage,
	UE::Editor::DataStorage::IUiProvider& DataStorageUi) const
{
	using namespace UE::Editor::DataStorage::Queries;
	DataStorageUi.RegisterWidgetFactory<FAssetNameWidgetConstructor>(
		DataStorageUi.FindPurpose(DataStorageUi.GetGeneralWidgetPurposeID()),
		TColumn<FAssetNameColumn>());
}

FAssetNameWidgetConstructor::FAssetNameWidgetConstructor()
	: FSimpleWidgetConstructor(StaticStruct())
{
}

TSharedPtr<SWidget> FAssetNameWidgetConstructor::CreateWidget(UE::Editor::DataStorage::ICoreProvider* DataStorage,
	UE::Editor::DataStorage::IUiProvider* DataStorageUi, UE::Editor::DataStorage::RowHandle TargetRow, UE::Editor::DataStorage::RowHandle WidgetRow,
	const UE::Editor::DataStorage::FMetaDataView& Arguments)
{
	UE::Editor::DataStorage::FAttributeBinder Binder(TargetRow, DataStorage);

	return SNew(STextBlock)
				.Text(Binder.BindText(&FAssetNameColumn::Name))
				.ToolTipText(Binder.BindText(&FAssetNameColumn::Name));
}

TArray<TSharedPtr<const UE::Editor::DataStorage::FColumnSorterInterface>> FAssetNameWidgetConstructor::ConstructColumnSorters(
	UE::Editor::DataStorage::ICoreProvider* DataStorage,
	UE::Editor::DataStorage::IUiProvider* DataStorageUi,
	const UE::Editor::DataStorage::FMetaDataView& Arguments)
{
	using namespace UE::Editor::DataStorage;
	return TArray<TSharedPtr<const FColumnSorterInterface>>(
		{
			MakeShared<FAssetNameWidgetSorter_NoSlash>(),
			MakeShared<FAssetNameWidgetSorter_WithSlash>()
		});
}

#undef LOCTEXT_NAMESPACE
