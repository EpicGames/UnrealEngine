// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetPreviewGeneralWidgetRegistrationFactory.h"

#include "Elements/Interfaces/TypedElementDataStorageUiInterface.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AssetPreviewGeneralWidgetRegistrationFactory)

#define LOCTEXT_NAMESPACE "AssetPreviewGeneralWidgetRegistrationFactory"

void UAssetPreviewGeneralWidgetRegistrationFactory::RegisterWidgetPurposes(UE::Editor::DataStorage::IUiProvider& DataStorageUi) const
{
	using namespace UE::Editor::DataStorage;

	const IUiProvider::FPurposeInfo AssetPreviewPurpose = IUiProvider::FPurposeInfo(
		"AssetPreview", "Default", NAME_None,
		IUiProvider::EPurposeType::UniqueByNameAndColumn,
		LOCTEXT("GeneralAssetPreviewDefaultPurpose", "The default widget purpose for the AssetPreview."));

	DataStorageUi.RegisterWidgetPurpose(AssetPreviewPurpose);
}

#undef LOCTEXT_NAMESPACE
