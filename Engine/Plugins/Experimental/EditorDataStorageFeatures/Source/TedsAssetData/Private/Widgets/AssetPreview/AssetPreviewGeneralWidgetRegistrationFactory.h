// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/Interfaces/TypedElementDataStorageFactory.h"
#include "UObject/ObjectMacros.h"

#include "AssetPreviewGeneralWidgetRegistrationFactory.generated.h"

UCLASS()
class UAssetPreviewGeneralWidgetRegistrationFactory : public UEditorDataStorageFactory
{
	GENERATED_BODY()

public:
	~UAssetPreviewGeneralWidgetRegistrationFactory() override = default;

	void RegisterWidgetPurposes(UE::Editor::DataStorage::IUiProvider& DataStorageUi) const override;
};
