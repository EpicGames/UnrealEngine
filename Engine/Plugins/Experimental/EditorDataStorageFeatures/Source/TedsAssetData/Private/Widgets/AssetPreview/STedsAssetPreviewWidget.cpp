// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/AssetPreview/STedsAssetPreviewWidget.h"
#include "Columns/SlateDelegateColumns.h"
#include "DataStorage/Features.h"
#include "Elements/Columns/TypedElementFolderColumns.h"
#include "Elements/Columns/TypedElementMiscColumns.h"
#include "Elements/Columns/TypedElementUIColumns.h"
#include "Elements/Framework/TypedElementDataStorageWidget.h"
#include "Elements/Interfaces/TypedElementDataStorageInterface.h"
#include "Elements/Interfaces/TypedElementDataStorageUiInterface.h"
#include "TedsAssetDataColumns.h"
#include "TedsAssetDataHelper.h"
#include "TedsAssetDataWidgetColumns.h"
#include "Widgets/AssetPreview/TedsAssetPreviewWidgetColumns.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/SBoxPanel.h"

void STedsAssetPreviewWidget::Construct(const FArguments& InArgs)
{
	WidgetPurpose = InArgs._WidgetPurpose;
	WidgetHeaderPurpose = InArgs._HeaderWidgetPurpose;
	ThumbnailWidgetPurpose = InArgs._ThumbnailWidgetPurpose;

	// Create the internal Teds Widget
	CreateTedsWidget();

	// Default Border
	PreviewPanel = SNew(SBorder);

	TedsWidget->SetContent(PreviewPanel.ToSharedRef());

	ChildSlot
		[
			TedsWidget->AsWidget()
		];
}

UE::Editor::DataStorage::RowHandle STedsAssetPreviewWidget::GetWidgetRowHandle() const
{
	return TedsWidget->GetRowHandle();
}

void STedsAssetPreviewWidget::ReconstructTedsWidget()
{
	PreviewPanel->ClearContent();
	ConstructWidget();
}

void STedsAssetPreviewWidget::SetTargetRow(UE::Editor::DataStorage::RowHandle InTargetRow)
{
	using namespace UE::Editor::DataStorage;

	if (TargetRow != InTargetRow)
	{
		TargetRow = InTargetRow;
		ReconstructTedsWidget();
	}
}

void STedsAssetPreviewWidget::CreateTedsWidget()
{
	using namespace UE::Editor::DataStorage;

	IUiProvider* DataStorageUI = GetMutableDataStorageFeature<IUiProvider>(UiFeatureName);
	TedsWidget = DataStorageUI->CreateContainerTedsWidget(InvalidRowHandle);

	const RowHandle WidgetRowHandle = TedsWidget->GetRowHandle();
	ICoreProvider* DataStorage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName);

	if (DataStorage->IsRowAvailable(WidgetRowHandle))
	{
		// External bound callback to get the ContextMenu of the Previewed Asset
		DataStorage->AddColumn(WidgetRowHandle, FWidgetContextMenuColumn::StaticStruct());
		DataStorage->AddColumn(WidgetRowHandle, FThumbnailEditModeColumn_Experimental::StaticStruct());
	}
}

void STedsAssetPreviewWidget::ConstructWidget()
{
	using namespace UE::Editor::DataStorage;

	ICoreProvider* DataStorage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName);

	if (!DataStorage->IsRowAvailable(TargetRow))
	{
		return;
	}

	RowHandle WidgetRowPreview = TedsWidget->GetRowHandle();

	IUiProvider* DataStorageUi = GetMutableDataStorageFeature<IUiProvider>(UiFeatureName);

	TSharedRef<SVerticalBox> PreviewPanelVerticalBox = SNew(SVerticalBox);
	TSharedRef<SHorizontalBox> ThumbnailHorizontalBox = SNew(SHorizontalBox);

	FLazyName NamePurposeTableName = TEXT("Editor_WidgetTable");

	// 1) Header
	{
		IUiProvider::FWidgetConstructorPtr OutHeaderButtonsWidgetConstructorPtr;
		auto AssignHeaderButtonsWidgetToColumn = [&OutHeaderButtonsWidgetConstructorPtr] (IUiProvider::FWidgetConstructorPtr WidgetConstructor, TConstArrayView<TWeakObjectPtr<const UScriptStruct>>)
		{
			OutHeaderButtonsWidgetConstructorPtr = MoveTemp(WidgetConstructor);;
			return false;
		};

		TArray<TWeakObjectPtr<const UScriptStruct>> HeaderColumns = GetHeaderColumns();
		DataStorageUi->CreateWidgetConstructors(DataStorageUi->FindPurpose(WidgetHeaderPurpose), IUiProvider::EMatchApproach::LongestMatch, HeaderColumns, FMetaDataView(), AssignHeaderButtonsWidgetToColumn);

		TSharedPtr<SWidget> HeaderWidget = SNullWidget::NullWidget;
		if (OutHeaderButtonsWidgetConstructorPtr)
		{
			if (HeaderWidgetRowHandle = DataStorage->AddRow(DataStorage->FindTable(NamePurposeTableName));
				DataStorage->IsRowAvailable(HeaderWidgetRowHandle))
			{
				// Set the current TargetRow
				DataStorage->AddColumn(HeaderWidgetRowHandle, FTypedElementRowReferenceColumn { .Row = TargetRow });

				// External bound callback to get the ContextMenu of the Previewed Asset
				if (FWidgetContextMenuColumn* ContextMenuWidgetRowColumn = DataStorage->GetColumn<FWidgetContextMenuColumn>(WidgetRowPreview))
				{
					DataStorage->AddColumn(HeaderWidgetRowHandle, FWidgetContextMenuColumn { .OnContextMenuOpening = ContextMenuWidgetRowColumn->OnContextMenuOpening });
				}

				// Set the current EditMode
				DataStorage->AddColumn(HeaderWidgetRowHandle, FThumbnailEditModeColumn_Experimental { .IsEditModeToggled = IsEditModeEnabled });

				// Set the external OnClicked event
				DataStorage->AddColumn(HeaderWidgetRowHandle, FExternalWidgetOnClickedColumn_Experimental
					{
						.OnClicked = MakeShared<TDelegate<void()>>(TDelegate<void()>::CreateLambda([this] ()
						{
							IsEditModeEnabled = !IsEditModeEnabled;
							ICoreProvider* DataStorage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName);

							// Update ThumbnailEditMode column
							if (FThumbnailEditModeColumn_Experimental* ThumbnailEditModeColumn = DataStorage->GetColumn<FThumbnailEditModeColumn_Experimental>(ThumbnailWidgetRowHandle))
							{
								ThumbnailEditModeColumn->IsEditModeToggled = IsEditModeEnabled;
							}
						}))
					});

				HeaderWidget = DataStorageUi->ConstructWidget(HeaderWidgetRowHandle, *OutHeaderButtonsWidgetConstructorPtr, FMetaDataView());
			}
		}

		if (!HeaderWidget.IsValid())
		{
			HeaderWidget = SNullWidget::NullWidget;
		}

		PreviewPanelVerticalBox->AddSlot()
			.AutoHeight()
			[
				HeaderWidget.ToSharedRef()
			];
	}

	// 2) Thumbnail
	{
		IUiProvider::FWidgetConstructorPtr OutThumbnailWidgetConstructorPtr;
		auto AssignThumbnailWidgetToColumn = [&OutThumbnailWidgetConstructorPtr] (IUiProvider::FWidgetConstructorPtr WidgetConstructor, TConstArrayView<TWeakObjectPtr<const UScriptStruct>>)
		{
			OutThumbnailWidgetConstructorPtr = MoveTemp(WidgetConstructor);;
			return false;
		};

		TArray<TWeakObjectPtr<const UScriptStruct>> ThumbnailColumns = GetThumbnailColumns();
		DataStorageUi->CreateWidgetConstructors(DataStorageUi->FindPurpose(ThumbnailWidgetPurpose), IUiProvider::EMatchApproach::LongestMatch, ThumbnailColumns, FMetaDataView(), AssignThumbnailWidgetToColumn);

		TSharedPtr<SWidget> ThumbnailWidget = SNullWidget::NullWidget;
		if (OutThumbnailWidgetConstructorPtr)
		{
			if (ThumbnailWidgetRowHandle = DataStorage->AddRow(DataStorage->FindTable(NamePurposeTableName));
				DataStorage->IsRowAvailable(ThumbnailWidgetRowHandle))
			{
				// Set the current TargetRow
				DataStorage->AddColumn(ThumbnailWidgetRowHandle, FTypedElementRowReferenceColumn{ .Row = TargetRow });

				// Set the current EditMode
				DataStorage->AddColumn(ThumbnailWidgetRowHandle, FThumbnailEditModeColumn_Experimental{ .IsEditModeToggled = IsEditModeEnabled });

				// Sizing is handled by the AssetPreview thumbnail widget

				ThumbnailWidget = DataStorageUi->ConstructWidget(ThumbnailWidgetRowHandle, *OutThumbnailWidgetConstructorPtr, FMetaDataView());
			}
		}

		if (!ThumbnailWidget.IsValid())
		{
			ThumbnailWidget = SNullWidget::NullWidget;
		}

		ThumbnailHorizontalBox->AddSlot()
			.AutoWidth()
			[
				ThumbnailWidget.ToSharedRef()
			];
	}

	// 3) Asset Name and Basic info
	{
		IUiProvider::FWidgetConstructorPtr OutBasicInfoWidgetConstructorPtr;
		auto AssignBasicInfoWidgetToColumn = [&OutBasicInfoWidgetConstructorPtr] (IUiProvider::FWidgetConstructorPtr WidgetConstructor, TConstArrayView<TWeakObjectPtr<const UScriptStruct>>)
		{
			OutBasicInfoWidgetConstructorPtr = MoveTemp(WidgetConstructor);;
			return false;
		};

		TArray<TWeakObjectPtr<const UScriptStruct>> BasicInfoColumns = GetBasicInfoColumns();
		DataStorageUi->CreateWidgetConstructors(DataStorageUi->FindPurpose(WidgetPurpose), IUiProvider::EMatchApproach::LongestMatch, BasicInfoColumns, FMetaDataView(), AssignBasicInfoWidgetToColumn);

		TSharedPtr<SWidget> BasicInfoWidget = SNullWidget::NullWidget;
		if (OutBasicInfoWidgetConstructorPtr)
		{
			if (BasicInfoWidgetRowHandle = DataStorage->AddRow(DataStorage->FindTable(NamePurposeTableName));
				DataStorage->IsRowAvailable(BasicInfoWidgetRowHandle))
			{
				// Set the current TargetRow
				DataStorage->AddColumn(BasicInfoWidgetRowHandle, FTypedElementRowReferenceColumn{ .Row = TargetRow });

				BasicInfoWidget = DataStorageUi->ConstructWidget(BasicInfoWidgetRowHandle, *OutBasicInfoWidgetConstructorPtr, FMetaDataView());
			}
		}

		if (!BasicInfoWidget.IsValid())
		{
			BasicInfoWidget = SNullWidget::NullWidget;
		}

		ThumbnailHorizontalBox->AddSlot()
			[
				BasicInfoWidget.ToSharedRef()
			];

		PreviewPanelVerticalBox->AddSlot()
			.AutoHeight()
			[
				ThumbnailHorizontalBox
			];
	}

	// 4) Full Details
	{
		IUiProvider::FWidgetConstructorPtr OutFullInfoWidgetConstructorPtr;
		auto AssignFullInfoWidgetToColumn = [&OutFullInfoWidgetConstructorPtr] (IUiProvider::FWidgetConstructorPtr WidgetConstructor, TConstArrayView<TWeakObjectPtr<const UScriptStruct>>)
		{
			OutFullInfoWidgetConstructorPtr = MoveTemp(WidgetConstructor);;
			return false;
		};

		TArray<TWeakObjectPtr<const UScriptStruct>> FullInfoColumns = GetAdvancedInfoColumns();
		DataStorageUi->CreateWidgetConstructors(DataStorageUi->FindPurpose(WidgetPurpose), IUiProvider::EMatchApproach::LongestMatch, FullInfoColumns, FMetaDataView(), AssignFullInfoWidgetToColumn);

		TSharedPtr<SWidget> FullInfoWidget = SNullWidget::NullWidget;
		if (OutFullInfoWidgetConstructorPtr)
		{
			if (AdvancedInfoWidgetRowHandle = DataStorage->AddRow(DataStorage->FindTable(NamePurposeTableName));
				DataStorage->IsRowAvailable(AdvancedInfoWidgetRowHandle))
			{
				// Set the current TargetRow
				DataStorage->AddColumn(AdvancedInfoWidgetRowHandle, FTypedElementRowReferenceColumn{ .Row = TargetRow });

				FullInfoWidget = DataStorageUi->ConstructWidget(AdvancedInfoWidgetRowHandle, *OutFullInfoWidgetConstructorPtr, FMetaDataView());
			}
		}

		if (!FullInfoWidget.IsValid())
		{
			FullInfoWidget = SNullWidget::NullWidget;
		}

		PreviewPanelVerticalBox->AddSlot()
			.FillHeight(1.f)
			[
				FullInfoWidget.ToSharedRef()
			];
	}

	// Assign to the Border the VerticalBox containing the TEDS Widgets
	PreviewPanel->SetContent(PreviewPanelVerticalBox);
}

TArray<TWeakObjectPtr<const UScriptStruct>> STedsAssetPreviewWidget::GetHeaderColumns()
{
	static TArray<TWeakObjectPtr<const UScriptStruct>> HeaderColumns(
	{
		TWeakObjectPtr(FAssetTag::StaticStruct()),
		TWeakObjectPtr(FFolderTag::StaticStruct())
	});
	return HeaderColumns;
}

TArray<TWeakObjectPtr<const UScriptStruct>> STedsAssetPreviewWidget::GetThumbnailColumns()
{
	static TArray<TWeakObjectPtr<const UScriptStruct>> ThumbnailColumns(
		{
			TWeakObjectPtr(FAssetTag::StaticStruct()),
			TWeakObjectPtr(FFolderTag::StaticStruct())
		});
	return ThumbnailColumns;
}

TArray<TWeakObjectPtr<const UScriptStruct>> STedsAssetPreviewWidget::GetBasicInfoColumns()
{
	static TArray<TWeakObjectPtr<const UScriptStruct>> BasicInfoColumns(
	{
		TWeakObjectPtr(FAssetTag::StaticStruct()),
		TWeakObjectPtr(FFolderTag::StaticStruct()),
		TWeakObjectPtr(FAssetNameColumn::StaticStruct())
	});
	return BasicInfoColumns;
}

TArray<TWeakObjectPtr<const UScriptStruct>> STedsAssetPreviewWidget::GetAdvancedInfoColumns()
{
	static TArray<TWeakObjectPtr<const UScriptStruct>> AdvancedInfoColumns(
		{
			TWeakObjectPtr(FAssetTag::StaticStruct()),
			TWeakObjectPtr(FFolderTag::StaticStruct())
		});
	return AdvancedInfoColumns;
}

UE::Editor::DataStorage::RowHandle STedsAssetPreviewWidget::GetReferencedRowHandle() const
{
	return TargetRow;
}
