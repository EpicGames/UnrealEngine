// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetPreviewAdvancedInfoWidget.h"

#include "Elements/Framework/TypedElementAttributeBinding.h"
#include "Internationalization/Text.h"
#include "TedsAssetDataColumns.h"
#include "Elements/Columns/TypedElementFolderColumns.h"
#include "UObject/ObjectMacros.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SRowDetails.h"
#include "Widgets/Text/STextBlock.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AssetPreviewAdvancedInfoWidget)

#define LOCTEXT_NAMESPACE "AssetPreviewAdvancedInfoWidget"

void UAssetPreviewAdvancedInfoWidgetFactory::RegisterWidgetConstructors(UE::Editor::DataStorage::ICoreProvider& DataStorage,
                                                                    UE::Editor::DataStorage::IUiProvider& DataStorageUi) const
{
	using namespace UE::Editor::DataStorage::Queries;
	DataStorageUi.RegisterWidgetFactory<FAssetPreviewAdvancedInfoWidgetConstructor>(
		DataStorageUi.FindPurpose(IUiProvider::FPurposeInfo("AssetPreview", "Default", NAME_None).GeneratePurposeID()),
		TColumn<FAssetTag>() || TColumn<FFolderTag>());
}

FAssetPreviewAdvancedInfoWidgetConstructor::FAssetPreviewAdvancedInfoWidgetConstructor()
	: FSimpleWidgetConstructor(StaticStruct())
{
}

TSharedPtr<SWidget> FAssetPreviewAdvancedInfoWidgetConstructor::CreateWidget(UE::Editor::DataStorage::ICoreProvider* DataStorage,
	UE::Editor::DataStorage::IUiProvider* DataStorageUi, UE::Editor::DataStorage::RowHandle TargetRow, UE::Editor::DataStorage::RowHandle WidgetRow,
	const UE::Editor::DataStorage::FMetaDataView& Arguments)
{
	using namespace UE::Editor::DataStorage;

	// For now Re-Use the RowDetails, but in the future we would probably need some customization or a specific new widget
	TSharedRef<SRowDetails> RowDetails = SNew(SRowDetails).ShowAllDetails(true);
	RowDetails->SetRow(TargetRow);

	return SNew(SScrollBox)
			+ SScrollBox::Slot()
			[
				RowDetails
			];
}

#undef LOCTEXT_NAMESPACE
