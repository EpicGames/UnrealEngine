// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/Interfaces/TypedElementDataStorageFactory.h"
#include "Elements/Interfaces/TypedElementDataStorageUiInterface.h"

#include "AssetPreviewAdvancedInfoWidget.generated.h"

class IEditorDataStorageProvider;
class UScriptStruct;
class SWidget;

UCLASS()
class UAssetPreviewAdvancedInfoWidgetFactory : public UEditorDataStorageFactory
{
	GENERATED_BODY()

public:
	virtual ~UAssetPreviewAdvancedInfoWidgetFactory() override = default;

	virtual void RegisterWidgetConstructors(UE::Editor::DataStorage::ICoreProvider& DataStorage,
	                                        UE::Editor::DataStorage::IUiProvider& DataStorageUi) const override;
};

// Widget for the AssetPreview Advanced Info
USTRUCT()
struct FAssetPreviewAdvancedInfoWidgetConstructor : public FSimpleWidgetConstructor
{
	GENERATED_BODY()

public:
	FAssetPreviewAdvancedInfoWidgetConstructor();
	virtual ~FAssetPreviewAdvancedInfoWidgetConstructor() override = default;

	virtual TSharedPtr<SWidget> CreateWidget(
		UE::Editor::DataStorage::ICoreProvider* DataStorage,
		UE::Editor::DataStorage::IUiProvider* DataStorageUi,
		UE::Editor::DataStorage::RowHandle TargetRow,
		UE::Editor::DataStorage::RowHandle WidgetRow,
		const UE::Editor::DataStorage::FMetaDataView& Arguments) override;
};
