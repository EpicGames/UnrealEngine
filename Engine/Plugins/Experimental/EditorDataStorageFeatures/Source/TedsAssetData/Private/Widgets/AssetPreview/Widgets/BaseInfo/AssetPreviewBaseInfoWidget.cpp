// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetPreviewBaseInfoWidget.h"

#include "AssetDefinition.h"
#include "AssetDefinitionRegistry.h"
#include "ClassIconFinder.h"
#include "Elements/Framework/TypedElementAttributeBinding.h"
#include "Internationalization/Text.h"
#include "TedsAssetDataColumns.h"
#include "Elements/Columns/TypedElementFolderColumns.h"
#include "Elements/Columns/TypedElementMiscColumns.h"
#include "Elements/Columns/TypedElementSlateWidgetColumns.h"
#include "UObject/ObjectMacros.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AssetPreviewBaseInfoWidget)

#define LOCTEXT_NAMESPACE "AssetPreviewBaseInfoWidget"

void UAssetPreviewBaseInfoWidgetFactory::RegisterWidgetConstructors(UE::Editor::DataStorage::ICoreProvider& DataStorage,
                                                                    UE::Editor::DataStorage::IUiProvider& DataStorageUi) const
{
	using namespace UE::Editor::DataStorage::Queries;

	DataStorageUi.RegisterWidgetFactory<FAssetPreviewBaseInfoWidgetConstructor>(
		DataStorageUi.FindPurpose(IUiProvider::FPurposeInfo("AssetPreview", "Default", NAME_None).GeneratePurposeID()),
		(TColumn<FAssetTag>() || TColumn<FFolderTag>()) && TColumn<FAssetNameColumn>());
}

FAssetPreviewBaseInfoWidgetConstructor::FAssetPreviewBaseInfoWidgetConstructor()
	: FSimpleWidgetConstructor(StaticStruct())
{
}

TSharedPtr<SWidget> FAssetPreviewBaseInfoWidgetConstructor::CreateWidget(UE::Editor::DataStorage::ICoreProvider* DataStorage,
	UE::Editor::DataStorage::IUiProvider* DataStorageUi, UE::Editor::DataStorage::RowHandle TargetRow, UE::Editor::DataStorage::RowHandle WidgetRow,
	const UE::Editor::DataStorage::FMetaDataView& Arguments)
{
	using namespace UE::Editor::DataStorage;

	TSharedPtr<SWidget> NameWidget = SNullWidget::NullWidget;

	// Asset Name
	TSharedRef<SVerticalBox> BaseInfoBox = SNew(SVerticalBox);
	if (DataStorage->HasColumns<FAssetNameColumn>(TargetRow))
	{
		const FName NamePurposeTableName = TEXT("Editor_WidgetTable");

		IUiProvider::FWidgetConstructorPtr OutNameWidgetConstructorPtr;
		auto AssignNameWidgetToColumn = [&OutNameWidgetConstructorPtr] (IUiProvider::FWidgetConstructorPtr WidgetConstructor, TConstArrayView<TWeakObjectPtr<const UScriptStruct>>)
		{
			OutNameWidgetConstructorPtr = MoveTemp(WidgetConstructor);;
			return false;
		};

		// MetaData
		UE::Editor::DataStorage::FMetaData NameMeta;
		NameMeta.AddOrSetMutableData(TEXT("OverflowPolicy"), 1);
		NameMeta.AddOrSetMutableData(TEXT("Font_TypeFace_Name"), TEXT("Bold"));
		NameMeta.AddOrSetMutableData(TEXT("Font_Size"), 15);

		TArray<TWeakObjectPtr<const UScriptStruct>> BasicInfoColumns = GetLabelColumns();
		DataStorageUi->CreateWidgetConstructors(DataStorageUi->FindPurpose(DataStorageUi->GetGeneralWidgetPurposeID()), IUiProvider::EMatchApproach::ExactMatch, BasicInfoColumns, FGenericMetaDataView(NameMeta), AssignNameWidgetToColumn);

		RowHandle NameWidgetRowHandle = InvalidRowHandle;
		if (OutNameWidgetConstructorPtr)
		{
			if (NameWidgetRowHandle = DataStorage->AddRow(DataStorage->FindTable(NamePurposeTableName));
				DataStorage->IsRowAvailable(NameWidgetRowHandle))
			{
				// Get the current TargetRow based on the provider
				DataStorage->AddColumn(NameWidgetRowHandle, FTypedElementRowReferenceColumn{ .Row = TargetRow });

				// Add Columns before calling this if needed
				NameWidget = DataStorageUi->ConstructWidget(NameWidgetRowHandle, *OutNameWidgetConstructorPtr, FGenericMetaDataView(NameMeta));
			}
		}

		if (!NameWidget.IsValid())
		{
			NameWidget = SNullWidget::NullWidget;
		}
	}

	FSlateIcon Icon = FSlateIcon();
	FSlateColor IconColor = FSlateColor::UseForeground();
	FText AssetClassName = FText::GetEmpty();
	if (DataStorage->HasColumns<FAssetTag>(TargetRow))
	{
		if (FAssetDataColumn_Experimental* AssetDataColumn = DataStorage->GetColumn<FAssetDataColumn_Experimental>(TargetRow))
		{
			FAssetData AssetData = AssetDataColumn->AssetData;
			if (const UClass* AssetClass = AssetData.GetClass())
			{
				AssetClassName = AssetClass->GetDisplayNameText();
				Icon = FSlateIconFinder::FindIconForClass(AssetClass);

				if (DataStorage->HasColumns<FSlateColorColumn>(TargetRow))
				{
					IconColor = DataStorage->GetColumn<FSlateColorColumn>(TargetRow)->Color;
				}
				// If no Color is set use the Class specific one
				else if (const UAssetDefinition* AssetDefinition = UAssetDefinitionRegistry::Get()->GetAssetDefinitionForClass(AssetClass))
				{
					IconColor = AssetDefinition->GetAssetColor();
				}
			}
		}
	}

	TSharedPtr<SWidget> IconWidget = SNullWidget::NullWidget;
	if (Icon.IsSet())
	{
		IconWidget = SNew(SImage).Image(Icon.GetIcon()).ColorAndOpacity(IconColor);
	}

	// Add Icon and AssetName
	BaseInfoBox->AddSlot()
		.AutoHeight()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				IconWidget.ToSharedRef()
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				NameWidget.ToSharedRef()
			]
		];

	// Class Name
	BaseInfoBox->AddSlot()
		.AutoHeight()
		[
			SNew(STextBlock)
			.Text(AssetClassName)
		];

	return BaseInfoBox;
}

TArray<TWeakObjectPtr<const UScriptStruct>> FAssetPreviewBaseInfoWidgetConstructor::GetLabelColumns()
{
	static TArray<TWeakObjectPtr<const UScriptStruct>> LabelColumns({ TWeakObjectPtr(FAssetNameColumn::StaticStruct()) });
	return LabelColumns;
}

#undef LOCTEXT_NAMESPACE
