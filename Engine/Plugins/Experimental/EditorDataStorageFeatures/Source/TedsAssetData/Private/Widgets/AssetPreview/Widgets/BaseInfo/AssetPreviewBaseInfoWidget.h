// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/Interfaces/TypedElementDataStorageFactory.h"
#include "Elements/Interfaces/TypedElementDataStorageUiInterface.h"

#include "AssetPreviewBaseInfoWidget.generated.h"

class UScriptStruct;

UCLASS()
class UAssetPreviewBaseInfoWidgetFactory : public UEditorDataStorageFactory
{
	GENERATED_BODY()

public:
	virtual ~UAssetPreviewBaseInfoWidgetFactory() override = default;

	virtual void RegisterWidgetConstructors(UE::Editor::DataStorage::ICoreProvider& DataStorage,
	                                        UE::Editor::DataStorage::IUiProvider& DataStorageUi) const override;
};

// Widget for the AssetPreview Basic Info
USTRUCT()
struct FAssetPreviewBaseInfoWidgetConstructor : public FSimpleWidgetConstructor
{
	GENERATED_BODY()

public:
	FAssetPreviewBaseInfoWidgetConstructor();
	virtual ~FAssetPreviewBaseInfoWidgetConstructor() override = default;

	virtual TSharedPtr<SWidget> CreateWidget(
		UE::Editor::DataStorage::ICoreProvider* DataStorage,
		UE::Editor::DataStorage::IUiProvider* DataStorageUi,
		UE::Editor::DataStorage::RowHandle TargetRow,
		UE::Editor::DataStorage::RowHandle WidgetRow,
		const UE::Editor::DataStorage::FMetaDataView& Arguments) override;

private:
	TArray<TWeakObjectPtr<const UScriptStruct>> GetLabelColumns();
};
