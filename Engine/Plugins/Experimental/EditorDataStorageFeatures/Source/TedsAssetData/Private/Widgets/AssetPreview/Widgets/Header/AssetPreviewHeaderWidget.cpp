// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetPreviewHeaderWidget.h"

#include "AssetThumbnail.h"
#include "AssetDefinition.h"
#include "Elements/Framework/TypedElementAttributeBinding.h"
#include "TedsAssetDataColumns.h"
#include "TedsAssetDataWidgetColumns.h"
#include "Columns/SlateDelegateColumns.h"
#include "Elements/Columns/TypedElementFolderColumns.h"
#include "Elements/Columns/TypedElementMiscColumns.h"
#include "Widgets/AssetPreview/TedsAssetPreviewWidgetColumns.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/SBoxPanel.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AssetPreviewHeaderWidget)

#define LOCTEXT_NAMESPACE "FAssetPreviewHeaderWidgetConstructor"

void UAssetPreviewHeaderWidgetFactory::RegisterWidgetPurposes(UE::Editor::DataStorage::IUiProvider& DataStorageUi) const
{
	using namespace UE::Editor::DataStorage::Queries;
	
	DataStorageUi.RegisterWidgetPurpose(
		IUiProvider::FPurposeInfo("AssetPreview", "Header", "Default",
		IUiProvider::EPurposeType::UniqueByNameAndColumn,
		LOCTEXT("GeneralAssetPreviewDefaultHeaderPurpose", "The default widget purpose for the AssetPreview header.")));
}

void UAssetPreviewHeaderWidgetFactory::RegisterWidgetConstructors(UE::Editor::DataStorage::ICoreProvider& DataStorage,
	UE::Editor::DataStorage::IUiProvider& DataStorageUi) const
{
	using namespace UE::Editor::DataStorage::Queries;

	DataStorageUi.RegisterWidgetFactory<FAssetPreviewHeaderWidgetConstructor>(
		DataStorageUi.FindPurpose(IUiProvider::FPurposeInfo("AssetPreview", "Header", "Default").GeneratePurposeID()),
		TColumn<FAssetTag>() || TColumn<FFolderTag>());
}

FAssetPreviewHeaderWidgetConstructor::FAssetPreviewHeaderWidgetConstructor()
	: FSimpleWidgetConstructor(StaticStruct())
{
}

TSharedPtr<SWidget> FAssetPreviewHeaderWidgetConstructor::CreateWidget(UE::Editor::DataStorage::ICoreProvider* DataStorage,
	UE::Editor::DataStorage::IUiProvider* DataStorageUi, UE::Editor::DataStorage::RowHandle TargetRow, UE::Editor::DataStorage::RowHandle WidgetRow,
	const UE::Editor::DataStorage::FMetaDataView& Arguments)
{
	using namespace UE::Editor::DataStorage;

	const FName NamePurposeTableName = TEXT("Editor_WidgetTable");

	IUiProvider::FWidgetConstructorPtr OutEditModeButtonWidgetConstructorPtr;
	IUiProvider::FWidgetConstructorPtr OutContextMenuWidgetConstructorPtr;

	auto AssignEditModeButtonWidgetToColumn = [&OutEditModeButtonWidgetConstructorPtr] (IUiProvider::FWidgetConstructorPtr WidgetConstructor, TConstArrayView<TWeakObjectPtr<const UScriptStruct>>)
	{
		OutEditModeButtonWidgetConstructorPtr = MoveTemp(WidgetConstructor);;
		return false;
	};

	auto AssignContextMenuWidgetToColumn = [&OutContextMenuWidgetConstructorPtr] (IUiProvider::FWidgetConstructorPtr WidgetConstructor, TConstArrayView<TWeakObjectPtr<const UScriptStruct>>)
	{
		OutContextMenuWidgetConstructorPtr = MoveTemp(WidgetConstructor);;
		return false;
	};

	if (DataStorage->HasColumns<FAssetTag>(TargetRow))
	{
		TArray<TWeakObjectPtr<const UScriptStruct>> EditModeButtonColumns = GetEditModeColumns();
		DataStorageUi->CreateWidgetConstructors(DataStorageUi->FindPurpose(
			IUiProvider::FPurposeInfo("AssetPreview", "Header", "EditMode").GeneratePurposeID()),
			IUiProvider::EMatchApproach::ExactMatch,
			EditModeButtonColumns,
			Arguments,
			AssignEditModeButtonWidgetToColumn);
	}

	TArray<TWeakObjectPtr<const UScriptStruct>> ContextMenuButtonColumns = GetItemContextMenuColumns();
	DataStorageUi->CreateWidgetConstructors(DataStorageUi->FindPurpose(
		IUiProvider::FPurposeInfo("AssetPreview", "Header", "ContextMenu").GeneratePurposeID()),
		IUiProvider::EMatchApproach::ExactMatch,
		ContextMenuButtonColumns,
		Arguments,
		AssignContextMenuWidgetToColumn);

	const FMargin ColumnItemPadding(8, 0);
	constexpr float ThumbnailNameHorizontalPadding = 8.f;

	TSharedPtr<SWidget> EditModeWidget = SNullWidget::NullWidget;
	TSharedPtr<SWidget> ContextMenuWidget = SNullWidget::NullWidget;

	RowHandle EditModeWidgetRowHandle = InvalidRowHandle;
	if (OutEditModeButtonWidgetConstructorPtr)
	{
		if (EditModeWidgetRowHandle = DataStorage->AddRow(DataStorage->FindTable(NamePurposeTableName));
			DataStorage->IsRowAvailable(EditModeWidgetRowHandle))
		{
			// Get the current TargetRow based on the provider
			DataStorage->AddColumn(EditModeWidgetRowHandle, FTypedElementRowReferenceColumn{ .Row = TargetRow });

			// Assign the same FThumbnailEditModeColumn_Experimental initial value that was assigned to this widget row
			if (const FThumbnailEditModeColumn_Experimental* IsEditModeToggledColumn = DataStorage->GetColumn<FThumbnailEditModeColumn_Experimental>(WidgetRow))
			{
				DataStorage->AddColumn(EditModeWidgetRowHandle, FThumbnailEditModeColumn_Experimental{ .IsEditModeToggled = IsEditModeToggledColumn->IsEditModeToggled });
			}

			// Assign the same OnClicked that was assigned to this widget row
			if (const FExternalWidgetOnClickedColumn_Experimental* OnWidgetClickedColumn = DataStorage->GetColumn<FExternalWidgetOnClickedColumn_Experimental>(WidgetRow))
			{
				DataStorage->AddColumn(EditModeWidgetRowHandle, FExternalWidgetOnClickedColumn_Experimental{ .OnClicked = OnWidgetClickedColumn->OnClicked });
			}

			// Add Columns before calling this if needed
			EditModeWidget = DataStorageUi->ConstructWidget(EditModeWidgetRowHandle, *OutEditModeButtonWidgetConstructorPtr, Arguments);
		}
	}

	RowHandle ContextMenuWidgetRowHandle = InvalidRowHandle;
	if (OutContextMenuWidgetConstructorPtr)
	{
		if (ContextMenuWidgetRowHandle = DataStorage->AddRow(DataStorage->FindTable(NamePurposeTableName));
			DataStorage->IsRowAvailable(ContextMenuWidgetRowHandle))
		{
			// Get the current TargetRow based on the provider
			DataStorage->AddColumn(ContextMenuWidgetRowHandle, FTypedElementRowReferenceColumn{ .Row = TargetRow });

			// Assign the same ContextMenuOpening that was assigned to this widget row
			if (const FWidgetContextMenuColumn* WidgetContextMenuColumn = DataStorage->GetColumn<FWidgetContextMenuColumn>(WidgetRow))
			{
				DataStorage->AddColumn(ContextMenuWidgetRowHandle, FWidgetContextMenuColumn{ .OnContextMenuOpening = WidgetContextMenuColumn->OnContextMenuOpening });
			}

			// Add Columns before calling this if needed
			ContextMenuWidget = DataStorageUi->ConstructWidget(ContextMenuWidgetRowHandle, *OutContextMenuWidgetConstructorPtr, Arguments);
		}
	}

	if (!EditModeWidget.IsValid())
	{
		EditModeWidget = SNullWidget::NullWidget;
	}

	if (!ContextMenuWidget.IsValid())
	{
		ContextMenuWidget = SNullWidget::NullWidget;
	}

	return SNew(SBox)
			.Padding(ColumnItemPadding)
			[
				SNew(SHorizontalBox)

				+SHorizontalBox::Slot()
				.FillWidth(1.f)
				.VAlign(VAlign_Center)
				[
					EditModeWidget.ToSharedRef()
				]

				+SHorizontalBox::Slot()
				.HAlign(HAlign_Right)
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(ThumbnailNameHorizontalPadding, 0.f, 0.f, 0.f)
				[
					ContextMenuWidget.ToSharedRef()
				]
			];
}

TConstArrayView<const UScriptStruct*> FAssetPreviewHeaderWidgetConstructor::GetAdditionalColumnsList() const
{
	using namespace UE::Editor::DataStorage;
	static TTypedElementColumnTypeList<FThumbnailEditModeColumn_Experimental, FExternalWidgetOnClickedColumn_Experimental, FWidgetContextMenuColumn> Columns;
	return Columns;
}

TArray<TWeakObjectPtr<const UScriptStruct>> FAssetPreviewHeaderWidgetConstructor::GetEditModeColumns()
{
	static TArray<TWeakObjectPtr<const UScriptStruct>> EditModeColumns(
		{
			TWeakObjectPtr(FAssetTag::StaticStruct())
		});
	return EditModeColumns;
}

TArray<TWeakObjectPtr<const UScriptStruct>> FAssetPreviewHeaderWidgetConstructor::GetItemContextMenuColumns()
{
	static TArray<TWeakObjectPtr<const UScriptStruct>> MenuColumns(
		{
			TWeakObjectPtr(FAssetTag::StaticStruct()),
			TWeakObjectPtr(FFolderTag::StaticStruct())
		});
	return MenuColumns;
}

#undef LOCTEXT_NAMESPACE
