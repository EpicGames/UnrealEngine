// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/Interfaces/TypedElementDataStorageFactory.h"
#include "Elements/Interfaces/TypedElementDataStorageUiInterface.h"

#include "AssetPreviewHeaderWidget.generated.h"

class IEditorDataStorageProvider;
class UScriptStruct;

UCLASS()
class UAssetPreviewHeaderWidgetFactory : public UEditorDataStorageFactory
{
	GENERATED_BODY()

public:
	virtual ~UAssetPreviewHeaderWidgetFactory() override = default;

	virtual void RegisterWidgetPurposes(UE::Editor::DataStorage::IUiProvider& DataStorageUi) const override;
	virtual void RegisterWidgetConstructors(UE::Editor::DataStorage::ICoreProvider& DataStorage,
	                                        UE::Editor::DataStorage::IUiProvider& DataStorageUi) const override;
};

// Widget for the AssetPreview Header
USTRUCT()
struct FAssetPreviewHeaderWidgetConstructor : public FSimpleWidgetConstructor
{
	GENERATED_BODY()

public:
	FAssetPreviewHeaderWidgetConstructor();
	virtual ~FAssetPreviewHeaderWidgetConstructor() override = default;

	virtual TSharedPtr<SWidget> CreateWidget(
		UE::Editor::DataStorage::ICoreProvider* DataStorage,
		UE::Editor::DataStorage::IUiProvider* DataStorageUi,
		UE::Editor::DataStorage::RowHandle TargetRow,
		UE::Editor::DataStorage::RowHandle WidgetRow,
		const UE::Editor::DataStorage::FMetaDataView& Arguments) override;

	virtual TConstArrayView<const UScriptStruct*> GetAdditionalColumnsList() const override;

private:
	TArray<TWeakObjectPtr<const UScriptStruct>> GetEditModeColumns();
	TArray<TWeakObjectPtr<const UScriptStruct>> GetItemContextMenuColumns();
};
