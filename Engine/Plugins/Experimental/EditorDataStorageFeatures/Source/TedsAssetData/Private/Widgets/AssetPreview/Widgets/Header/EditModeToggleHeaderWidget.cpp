// Copyright Epic Games, Inc. All Rights Reserved.

#include "EditModeToggleHeaderWidget.h"

#include "Elements/Framework/TypedElementAttributeBinding.h"
#include "Internationalization/Text.h"
#include "TedsAssetDataColumns.h"
#include "TedsAssetDataWidgetColumns.h"
#include "UObject/ObjectMacros.h"
#include "Widgets/AssetPreview/TedsAssetPreviewWidgetColumns.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Text/STextBlock.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EditModeToggleHeaderWidget)

#define LOCTEXT_NAMESPACE "EditModeToggleHeaderWidget"

void UEditModeToggleHeaderWidgetFactory::RegisterWidgetPurposes(UE::Editor::DataStorage::IUiProvider& DataStorageUi) const
{
	using namespace UE::Editor::DataStorage::Queries;
	
	DataStorageUi.RegisterWidgetPurpose(
		IUiProvider::FPurposeInfo("AssetPreview", "Header", "EditMode",
		IUiProvider::EPurposeType::Generic,
		LOCTEXT("AssetPreview_Header", "Widget that display the Header of the Preview Panel")));
}

void UEditModeToggleHeaderWidgetFactory::RegisterWidgetConstructors(UE::Editor::DataStorage::ICoreProvider& DataStorage,
                                                                    UE::Editor::DataStorage::IUiProvider& DataStorageUi) const
{
	using namespace UE::Editor::DataStorage::Queries;
	DataStorageUi.RegisterWidgetFactory<FEditModeToggleHeaderWidgetConstructor>(
		DataStorageUi.FindPurpose(IUiProvider::FPurposeInfo("AssetPreview", "Header", "EditMode").GeneratePurposeID()),
		TColumn<FAssetTag>());
}

FEditModeToggleHeaderWidgetConstructor::FEditModeToggleHeaderWidgetConstructor()
	: FSimpleWidgetConstructor(StaticStruct())
{
}

TSharedPtr<SWidget> FEditModeToggleHeaderWidgetConstructor::CreateWidget(UE::Editor::DataStorage::ICoreProvider* DataStorage,
	UE::Editor::DataStorage::IUiProvider* DataStorageUi, UE::Editor::DataStorage::RowHandle TargetRow, UE::Editor::DataStorage::RowHandle WidgetRow,
	const UE::Editor::DataStorage::FMetaDataView& Arguments)
{
	using namespace UE::Editor::DataStorage;

	FAttributeBinder WidgetRowBinder(WidgetRow, DataStorage);

	return SNew(SButton)
		.Text(LOCTEXT("EditModeButton_Toggle", "Edit..."))
		.ButtonColorAndOpacity(WidgetRowBinder.BindData(&FThumbnailEditModeColumn_Experimental::IsEditModeToggled, [] (bool IsEditMode)
		{
			if (IsEditMode)
			{
				return FSlateColor(FLinearColor::Blue);
			}
			return FSlateColor::UseForeground();
		}))
		.OnClicked_Lambda([WidgetRow, DataStorage] ()
		{
			FAttributeBinder WidgetRowBinder(WidgetRow, DataStorage);
			const TDelegate<void()> Event = WidgetRowBinder.BindEvent(&FExternalWidgetOnClickedColumn_Experimental::OnClicked);
			if (Event.IsBound())
			{
				Event.Execute();
				if (FThumbnailEditModeColumn_Experimental* IsEditModeToggledColumn = DataStorage->GetColumn<FThumbnailEditModeColumn_Experimental>(WidgetRow))
				{
					IsEditModeToggledColumn->IsEditModeToggled = !IsEditModeToggledColumn->IsEditModeToggled;
				}
			}
			return FReply::Handled();
		});
}

TConstArrayView<const UScriptStruct*> FEditModeToggleHeaderWidgetConstructor::GetAdditionalColumnsList() const
{
	using namespace UE::Editor::DataStorage;
	static TTypedElementColumnTypeList<FThumbnailEditModeColumn_Experimental, FExternalWidgetOnClickedColumn_Experimental> Columns;
	return Columns;
}

#undef LOCTEXT_NAMESPACE
