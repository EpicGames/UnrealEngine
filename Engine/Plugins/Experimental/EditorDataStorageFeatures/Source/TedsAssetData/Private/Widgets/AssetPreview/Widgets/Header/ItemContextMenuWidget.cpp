// Copyright Epic Games, Inc. All Rights Reserved.

#include "ItemContextMenuWidget.h"

#include "Columns/SlateDelegateColumns.h"
#include "Elements/Columns/TypedElementFolderColumns.h"
#include "Elements/Framework/TypedElementAttributeBinding.h"
#include "Framework/Application/SlateApplication.h"
#include "Internationalization/Text.h"
#include "TedsAssetDataColumns.h"
#include "UObject/ObjectMacros.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ItemContextMenuWidget)

#define LOCTEXT_NAMESPACE "ItemContextMenuWidget"

void UItemContextMenuWidgetFactory::RegisterWidgetPurposes(UE::Editor::DataStorage::IUiProvider& DataStorageUi) const
{
	using namespace UE::Editor::DataStorage::Queries;

	DataStorageUi.RegisterWidgetPurpose(
		IUiProvider::FPurposeInfo("AssetPreview", "Header", "ContextMenu",
		IUiProvider::EPurposeType::UniqueByName,
		LOCTEXT("AssetPreview_Header", "Widget that display the ContextMenu of the selected Asset/Folder")));
}

void UItemContextMenuWidgetFactory::RegisterWidgetConstructors(UE::Editor::DataStorage::ICoreProvider& DataStorage,
                                                               UE::Editor::DataStorage::IUiProvider& DataStorageUi) const
{
	using namespace UE::Editor::DataStorage::Queries;

	DataStorageUi.RegisterWidgetFactory<FItemContextMenuWidgetConstructor>(
		DataStorageUi.FindPurpose(IUiProvider::FPurposeInfo("AssetPreview", "Header", "ContextMenu").GeneratePurposeID()),
		TColumn<FAssetTag>() || TColumn<FFolderTag>());
}

FItemContextMenuWidgetConstructor::FItemContextMenuWidgetConstructor()
	: FSimpleWidgetConstructor(StaticStruct())
{
}

TSharedPtr<SWidget> FItemContextMenuWidgetConstructor::CreateWidget(UE::Editor::DataStorage::ICoreProvider* DataStorage,
	UE::Editor::DataStorage::IUiProvider* DataStorageUi, UE::Editor::DataStorage::RowHandle TargetRow, UE::Editor::DataStorage::RowHandle WidgetRow,
	const UE::Editor::DataStorage::FMetaDataView& Arguments)
{
	TSharedRef<SButton> Button = SNew(SButton)
		[
			SNew(SImage)
			.Image(FAppStyle::GetBrush("Icons.EllipsisVerticalNarrow"))
		];

	Button->SetOnClicked(FOnClicked::CreateLambda([Button, DataStorage, WidgetRow] ()
	{
		if (FWidgetContextMenuColumn* ContextMenuColumn = DataStorage->GetColumn<FWidgetContextMenuColumn>(WidgetRow))
		{
			if (ContextMenuColumn->OnContextMenuOpening && ContextMenuColumn->OnContextMenuOpening->IsBound())
			{
				const FVector2f SummonLocation = FSlateApplication::Get().GetCursorPos();
				FWidgetPath ButtonWidgetPath = FWidgetPath();
				FSlateApplication::Get().FindPathToWidget(Button, ButtonWidgetPath);
				FSlateApplication::Get().PushMenu(Button->AsShared(), ButtonWidgetPath, ContextMenuColumn->OnContextMenuOpening->Execute().ToSharedRef(), SummonLocation, FPopupTransitionEffect(FPopupTransitionEffect::ContextMenu));
			}
		}
		return FReply::Handled();
	}));

	return Button;
}

TConstArrayView<const UScriptStruct*> FItemContextMenuWidgetConstructor::GetAdditionalColumnsList() const
{
	using namespace UE::Editor::DataStorage;
	static TTypedElementColumnTypeList<FWidgetContextMenuColumn> Columns;
	return Columns;
}

#undef LOCTEXT_NAMESPACE
