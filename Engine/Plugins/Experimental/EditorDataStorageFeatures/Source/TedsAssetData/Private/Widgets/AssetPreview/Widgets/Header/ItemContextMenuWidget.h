// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/Interfaces/TypedElementDataStorageFactory.h"
#include "Elements/Interfaces/TypedElementDataStorageUiInterface.h"

#include "ItemContextMenuWidget.generated.h"

class IEditorDataStorageProvider;
class UScriptStruct;

UCLASS()
class UItemContextMenuWidgetFactory : public UEditorDataStorageFactory
{
	GENERATED_BODY()

public:
	virtual ~UItemContextMenuWidgetFactory() override = default;

	virtual void RegisterWidgetPurposes(UE::Editor::DataStorage::IUiProvider& DataStorageUi) const override;
	virtual void RegisterWidgetConstructors(UE::Editor::DataStorage::ICoreProvider& DataStorage,
	                                        UE::Editor::DataStorage::IUiProvider& DataStorageUi) const override;
};

// Widget that display the ContextMenu button for the AssetPreview panel
USTRUCT()
struct FItemContextMenuWidgetConstructor : public FSimpleWidgetConstructor
{
	GENERATED_BODY()

public:
	FItemContextMenuWidgetConstructor();
	virtual ~FItemContextMenuWidgetConstructor() override = default;

	virtual TSharedPtr<SWidget> CreateWidget(
		UE::Editor::DataStorage::ICoreProvider* DataStorage,
		UE::Editor::DataStorage::IUiProvider* DataStorageUi,
		UE::Editor::DataStorage::RowHandle TargetRow,
		UE::Editor::DataStorage::RowHandle WidgetRow,
		const UE::Editor::DataStorage::FMetaDataView& Arguments) override;

	virtual TConstArrayView<const UScriptStruct*> GetAdditionalColumnsList() const override;
};
