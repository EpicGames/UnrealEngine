// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetPreviewThumbnailWidget.h"

#include "AssetThumbnail.h"
#include "Elements/Framework/TypedElementAttributeBinding.h"
#include "TedsAssetDataColumns.h"
#include "TedsAssetDataHelper.h"
#include "TedsAssetDataWidgetColumns.h"
#include "Elements/Columns/TypedElementFolderColumns.h"
#include "Elements/Columns/TypedElementHiearchyColumns.h"
#include "Elements/Columns/TypedElementMiscColumns.h"
#include "Styling/StyleColors.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SOverlay.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AssetPreviewThumbnailWidget)


#define LOCTEXT_NAMESPACE "AssetPreviewThumbnailWidget"

// used to make the AssetPreview expand state global
static bool bIsThumbnailExpandedGlobal = false;

void UAssetPreviewThumbnailWidgetFactory::RegisterWidgetPurposes(UE::Editor::DataStorage::IUiProvider& DataStorageUi) const
{
	using namespace UE::Editor::DataStorage;
	
	DataStorageUi.RegisterWidgetPurpose(
		IUiProvider::FPurposeInfo("AssetPreview", "Thumbnail", NAME_None,
		IUiProvider::EPurposeType::UniqueByNameAndColumn,
		LOCTEXT("AssetPreviewThumbnailPurpose", "Specific purpose display thumbnails in the AssetPreview."),
		IUiProvider::FPurposeInfo("ContentBrowser", "Thumbnail", NAME_None).GeneratePurposeID()));
}


void UAssetPreviewThumbnailWidgetFactory::RegisterWidgetConstructors(UE::Editor::DataStorage::ICoreProvider& DataStorage,
                                                              UE::Editor::DataStorage::IUiProvider& DataStorageUi) const
{
	using namespace UE::Editor::DataStorage::Queries;
	DataStorageUi.RegisterWidgetFactory<FAssetPreviewThumbnailWidgetConstructor>(
		DataStorageUi.FindPurpose(IUiProvider::FPurposeInfo("AssetPreview", "Thumbnail", NAME_None).GeneratePurposeID()),
		TColumn<FAssetTag>() || TColumn<FFolderTag>());
}

FAssetPreviewThumbnailWidgetConstructor::FAssetPreviewThumbnailWidgetConstructor()
	: FSimpleWidgetConstructor(StaticStruct())
{
}

TSharedPtr<SWidget> FAssetPreviewThumbnailWidgetConstructor::CreateWidget(UE::Editor::DataStorage::ICoreProvider* DataStorage,
	UE::Editor::DataStorage::IUiProvider* DataStorageUi,
	UE::Editor::DataStorage::RowHandle TargetRow, UE::Editor::DataStorage::RowHandle WidgetRow,
	const UE::Editor::DataStorage::FMetaDataView& Arguments)
{
	using namespace UE::Editor::DataStorage;

	bool bIsAsset = DataStorage->HasColumns<FAssetTag>(TargetRow);
	const FName NamePurposeTableName = TEXT("Editor_WidgetTable");

	if (FSizeValueColumn_Experimental* SizeValueColumn = DataStorage->GetColumn<FSizeValueColumn_Experimental>(WidgetRow))
	{
		SizeValueColumn->SizeValue = bIsThumbnailExpandedGlobal ? 256.f : 128.f;
	}

	IUiProvider::FWidgetConstructorPtr OutThumbnailWidgetConstructorPtr;
    auto AssignThumbnailWidgetToColumn = [&OutThumbnailWidgetConstructorPtr] (IUiProvider::FWidgetConstructorPtr WidgetConstructor, TConstArrayView<TWeakObjectPtr<const UScriptStruct>>)
    {
    	OutThumbnailWidgetConstructorPtr = MoveTemp(WidgetConstructor);;
    	return false;
    };

	// Thumbnail Config MetaData
	UE::Editor::DataStorage::FMetaData ThumbnailMeta;
	ThumbnailMeta.AddOrSetMutableData(TedsAssetDataHelper::MetaDataNames::GetThumbnailStatusMetaDataName(), true);
	ThumbnailMeta.AddOrSetMutableData(TedsAssetDataHelper::MetaDataNames::GetThumbnailFadeInMetaDataName(), true);
	ThumbnailMeta.AddOrSetMutableData(TedsAssetDataHelper::MetaDataNames::GetThumbnailHintTextMetaDataName(), false);
	ThumbnailMeta.AddOrSetMutableData(TedsAssetDataHelper::MetaDataNames::GetThumbnailRealTimeOnHoveredMetaDataName(), false);
	ThumbnailMeta.AddOrSetMutableData(TedsAssetDataHelper::MetaDataNames::GetThumbnailCanDisplayEditModePrimitiveTools(), true);
	const FGenericMetaDataView ThumbnailMetaView = FGenericMetaDataView(ThumbnailMeta);

	TArray<TWeakObjectPtr<const UScriptStruct>> ThumbnailColumns = GetThumbnailColumns();
	DataStorageUi->CreateWidgetConstructors(DataStorageUi->FindPurpose(
		IUiProvider::FPurposeInfo("ContentBrowser", "Thumbnail", NAME_None).GeneratePurposeID()),
		IUiProvider::EMatchApproach::LongestMatch,
		ThumbnailColumns,
		ThumbnailMetaView,
		AssignThumbnailWidgetToColumn);

	TSharedPtr<SWidget> ThumbnailWidget = SNullWidget::NullWidget;

	RowHandle ThumbnailWidgetRowHandle = InvalidRowHandle;
	if (OutThumbnailWidgetConstructorPtr)
	{
		if (ThumbnailWidgetRowHandle = DataStorage->AddRow(DataStorage->FindTable(NamePurposeTableName));
			DataStorage->IsRowAvailable(ThumbnailWidgetRowHandle))
		{
			// Set the current TargetRow
			DataStorage->AddColumn(ThumbnailWidgetRowHandle, FTypedElementRowReferenceColumn{ .Row = TargetRow });

			// Set this as the parent so that it will use the EditMode and Size of this TEDS widget
			DataStorage->AddColumn(ThumbnailWidgetRowHandle, FTableRowParentColumn{ .Parent = WidgetRow });

			ThumbnailWidget = DataStorageUi->ConstructWidget(ThumbnailWidgetRowHandle, *OutThumbnailWidgetConstructorPtr, ThumbnailMetaView);
		}
	}

	if (!ThumbnailWidget.IsValid())
	{
		ThumbnailWidget = SNullWidget::NullWidget;
	}

	// If the widget creating this is interested in the ThumbnailTooltip it has to add the column itself to avoid adding them if not used
	if (FLocalWidgetTooltipColumn_Experimental* WidgetTooltipColumn = DataStorage->GetColumn<FLocalWidgetTooltipColumn_Experimental>(WidgetRow))
	{
		WidgetTooltipColumn->Tooltip = ThumbnailWidget->GetToolTip();
	}

	// For assets, grab the color from the asset definition
	if (bIsAsset)
	{
		return SNew(SBox)
			[
				SNew(SOverlay)

				+ SOverlay::Slot()
				[
					ThumbnailWidget.ToSharedRef()
				]

				// Overlay for incrementing Thumbnail Size
				+ SOverlay::Slot()
				.Padding(2.f)
				.HAlign(HAlign_Right)
				.VAlign(VAlign_Top)
				[
					SNew(SBox)
					.WidthOverride(20.f)
					.HeightOverride(20.f)
					[
						SNew(SButton)
						.ContentPadding(0.f)
						.ButtonStyle(&FAppStyle::Get().GetWidgetStyle<FButtonStyle>("SimpleButton"))
						.OnClicked_Lambda([DataStorage, WidgetRow] ()
						{
							using namespace UE::Editor::DataStorage;

							bIsThumbnailExpandedGlobal = !bIsThumbnailExpandedGlobal;
							if (FSizeValueColumn_Experimental* ThumbnailSizeValueColumn = DataStorage->GetColumn<FSizeValueColumn_Experimental>(WidgetRow))
							{
								if (bIsThumbnailExpandedGlobal)
								{
									ThumbnailSizeValueColumn->SizeValue = 256.f;
								}
								else
								{
									ThumbnailSizeValueColumn->SizeValue = 128.f;
								}
							}
							return FReply::Handled();
						})
						[
							SNew(SImage)
							.Image_Lambda([] ()
							{
								if (bIsThumbnailExpandedGlobal)
								{
									return FAppStyle::GetBrush("Icons.Minus");
								}
								return FAppStyle::GetBrush("Icons.Plus");
							})
						]
					]
				]
			];
	}

	return ThumbnailWidget;
}

TConstArrayView<const UScriptStruct*> FAssetPreviewThumbnailWidgetConstructor::GetAdditionalColumnsList() const
{
	using namespace UE::Editor::DataStorage;

	static const TTypedElementColumnTypeList<
		FSizeValueColumn_Experimental,
		FThumbnailEditModeColumn_Experimental,
		FLocalWidgetTooltipColumn_Experimental> Columns;

	return Columns;
}

TArray<TWeakObjectPtr<const UScriptStruct>> FAssetPreviewThumbnailWidgetConstructor::GetThumbnailColumns()
{
	static TArray<TWeakObjectPtr<const UScriptStruct>> ThumbnailColumns(
	{
		TWeakObjectPtr(FAssetTag::StaticStruct()),
		TWeakObjectPtr(FFolderTag::StaticStruct())
	});
	return ThumbnailColumns;
}

#undef LOCTEXT_NAMESPACE
