// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/Interfaces/TypedElementDataStorageFactory.h"
#include "Elements/Interfaces/TypedElementDataStorageUiInterface.h"
#include "Internationalization/Text.h"
#include "UObject/ObjectMacros.h"

#include "AssetPreviewThumbnailWidget.generated.h"

class UScriptStruct;

UCLASS()
class UAssetPreviewThumbnailWidgetFactory : public UEditorDataStorageFactory
{
	GENERATED_BODY()

public:
	virtual ~UAssetPreviewThumbnailWidgetFactory() override = default;

	virtual void RegisterWidgetPurposes(UE::Editor::DataStorage::IUiProvider& DataStorageUi) const override;
	virtual void RegisterWidgetConstructors(UE::Editor::DataStorage::ICoreProvider& DataStorage,
		UE::Editor::DataStorage::IUiProvider& DataStorageUi) const override;
};

// Thumbnail widget for the asset preview
USTRUCT()
struct FAssetPreviewThumbnailWidgetConstructor : public FSimpleWidgetConstructor
{
	GENERATED_BODY()

public:
	FAssetPreviewThumbnailWidgetConstructor();
	virtual ~FAssetPreviewThumbnailWidgetConstructor() override = default;

	virtual TSharedPtr<SWidget> CreateWidget(
		UE::Editor::DataStorage::ICoreProvider* DataStorage,
		UE::Editor::DataStorage::IUiProvider* DataStorageUi,
		UE::Editor::DataStorage::RowHandle TargetRow,
		UE::Editor::DataStorage::RowHandle WidgetRow, 
		const UE::Editor::DataStorage::FMetaDataView& Arguments) override;

	virtual TConstArrayView<const UScriptStruct*> GetAdditionalColumnsList() const override;

private:
	/** Get the columns used for the Thumbnail TEDS widget */
	TArray<TWeakObjectPtr<const UScriptStruct>> GetThumbnailColumns();
};
