// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/ContentBrowserListViewNameWidget.h"

#include "AssetThumbnail.h"
#include "AssetDefinition.h"
#include "Elements/Framework/TypedElementAttributeBinding.h"
#include "TedsAssetDataColumns.h"
#include "TedsAssetDataHelper.h"
#include "TedsAssetDataWidgetColumns.h"
#include "Columns/SlateDelegateColumns.h"
#include "Elements/Columns/TypedElementFolderColumns.h"
#include "Elements/Columns/TypedElementHiearchyColumns.h"
#include "Elements/Columns/TypedElementMiscColumns.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/SBoxPanel.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ContentBrowserListViewNameWidget)

#define LOCTEXT_NAMESPACE "FContentBrowserLabelAssetWidgetConstructor"

UE::Editor::DataStorage::IUiProvider::FPurposeID UContentBrowserListViewNameWidgetFactory::WidgetPurpose(UE::Editor::DataStorage::IUiProvider::FPurposeInfo(
			"ContentBrowser", "RowLabel", NAME_None).GeneratePurposeID());

void UContentBrowserListViewNameWidgetFactory::RegisterWidgetPurposes(UE::Editor::DataStorage::IUiProvider& DataStorageUi) const
{
	using namespace UE::Editor::DataStorage::Queries;

	DataStorageUi.RegisterWidgetPurpose(
		IUiProvider::FPurposeInfo("ContentBrowser", "RowLabel", NAME_None,
			IUiProvider::EPurposeType::UniqueByNameAndColumn,
			LOCTEXT("ContentBrowserLabelWidget_PurposeDescription", "Widget that display a Label + Thumbnail in the Content Browser."),
			IUiProvider::FPurposeInfo("General", "RowLabel", NAME_None).GeneratePurposeID()));
}

void UContentBrowserListViewNameWidgetFactory::RegisterWidgetConstructors(UE::Editor::DataStorage::ICoreProvider& DataStorage,
	UE::Editor::DataStorage::IUiProvider& DataStorageUi) const
{
	using namespace UE::Editor::DataStorage::Queries;
	DataStorageUi.RegisterWidgetFactory<FContentBrowserListViewNameWidgetConstructor>(
		DataStorageUi.FindPurpose(WidgetPurpose),
		TColumn<FAssetNameColumn>() && (TColumn<FAssetTag>() || TColumn<FFolderTag>()));
}

FContentBrowserListViewNameWidgetConstructor::FContentBrowserListViewNameWidgetConstructor()
	: FSimpleWidgetConstructor(StaticStruct())
{
}

FContentBrowserListViewNameWidgetConstructor::FContentBrowserListViewNameWidgetConstructor(const UScriptStruct* TypeInfo)
	: FSimpleWidgetConstructor(TypeInfo)
{
}

TSharedPtr<SWidget> FContentBrowserListViewNameWidgetConstructor::CreateWidget(UE::Editor::DataStorage::ICoreProvider* DataStorage,
	UE::Editor::DataStorage::IUiProvider* DataStorageUi, UE::Editor::DataStorage::RowHandle TargetRow, UE::Editor::DataStorage::RowHandle WidgetRow,
	const UE::Editor::DataStorage::FMetaDataView& Arguments)
{
	using namespace UE::Editor::DataStorage;

	if (DataStorage->IsRowAvailable(TargetRow))
	{
		UE::Editor::DataStorage::RowHandle ParentWidgetRowHandle = UE::Editor::DataStorage::InvalidRowHandle;
		if (FTableRowParentColumn* ParentWidgetRow = DataStorage->GetColumn<FTableRowParentColumn>(WidgetRow))
		{
			ParentWidgetRowHandle = ParentWidgetRow->Parent;
		}

		IUiProvider::FWidgetConstructorPtr OutThumbnailWidgetConstructorPtr;
		IUiProvider::FWidgetConstructorPtr OutLabelWidgetConstructorPtr;

		auto AssignThumbnailWidgetToColumn = [&OutThumbnailWidgetConstructorPtr] (IUiProvider::FWidgetConstructorPtr WidgetConstructor, TConstArrayView<TWeakObjectPtr<const UScriptStruct>>)
		{
			OutThumbnailWidgetConstructorPtr = MoveTemp(WidgetConstructor);;
			return false;
		};

		auto AssignLabelWidgetToColumn = [&OutLabelWidgetConstructorPtr] (IUiProvider::FWidgetConstructorPtr WidgetConstructor, TConstArrayView<TWeakObjectPtr<const UScriptStruct>>)
		{
			OutLabelWidgetConstructorPtr = MoveTemp(WidgetConstructor);;
			return false;
		};

		// TODO: Arguments need to be developed further to customize the Thumbnail Config if needed
		UE::Editor::DataStorage::FMetaData ThumbnailMeta;
		ThumbnailMeta.AddOrSetMutableData(TedsAssetDataHelper::MetaDataNames::GetThumbnailStatusMetaDataName(), true);
		ThumbnailMeta.AddOrSetMutableData(TedsAssetDataHelper::MetaDataNames::GetThumbnailFadeInMetaDataName(), true);
		ThumbnailMeta.AddOrSetMutableData(TedsAssetDataHelper::MetaDataNames::GetThumbnailHintTextMetaDataName(), false);
		// TODO: use our own OnMouseEnter/Leave for setting the realtime flag, also check CB Setting
		ThumbnailMeta.AddOrSetMutableData(TedsAssetDataHelper::MetaDataNames::GetThumbnailRealTimeOnHoveredMetaDataName(), false);
		
		// List all columns on the row so the thumbnail widget can be override based on longest match on the current row
		TArray<TWeakObjectPtr<const UScriptStruct>> ThumbnailColumns;
		DataStorage->ListColumns(TargetRow, [&ThumbnailColumns](const UScriptStruct& ColumnType)
			{
				ThumbnailColumns.Emplace(&ColumnType);
				return true;
			});

		const IUiProvider::FPurposeID ThumbnailPurposeID = IUiProvider::FPurposeInfo("ContentBrowser", "Thumbnail", NAME_None).GeneratePurposeID();
		DataStorageUi->CreateWidgetConstructors(DataStorageUi->FindPurpose(ThumbnailPurposeID), IUiProvider::EMatchApproach::LongestMatch, ThumbnailColumns, Arguments, AssignThumbnailWidgetToColumn);

		TArray<TWeakObjectPtr<const UScriptStruct>> LabelColumns = GetLabelColumns();

		const UE::Editor::DataStorage::IUiProvider::FPurposeID ContentBrowserNameWidgetPurposeId = IUiProvider::FPurposeInfo("ContentBrowser", "RowLabel", NAME_None).GeneratePurposeID();
		DataStorageUi->CreateWidgetConstructors(DataStorageUi->FindPurpose(ContentBrowserNameWidgetPurposeId),
			IUiProvider::EMatchApproach::ExactMatch, LabelColumns, Arguments, AssignLabelWidgetToColumn);

		constexpr float ThumbnailNameHorizontalPadding = 8.f;

		TSharedPtr<SWidget> ThumbnailWidget = SNullWidget::NullWidget;
		TSharedPtr<SWidget> LabelWidget = SNullWidget::NullWidget;

		UE::Editor::DataStorage::RowHandle ThumbnailWidgetRowHandle = UE::Editor::DataStorage::InvalidRowHandle;
		if (OutThumbnailWidgetConstructorPtr)
		{
			if (ThumbnailWidgetRowHandle = DataStorage->AddRow(DataStorage->FindTable(TedsAssetDataHelper::TableView::GetWidgetTableName())); ThumbnailWidgetRowHandle != UE::Editor::DataStorage::InvalidRowHandle)
			{
				// Referenced Data Row
				DataStorage->AddColumn(ThumbnailWidgetRowHandle, FTypedElementRowReferenceColumn{ .Row = TargetRow });

				// Parent widget row
				DataStorage->AddColumn(ThumbnailWidgetRowHandle, FTableRowParentColumn{ .Parent = ParentWidgetRowHandle });

				// TODO: ask, should the thumbnail tooltip be used for the entire widget or just the Thumbnail?
				// Used to retrieve the Thumbnail tooltip to use on the whole TileItem
				// DataStorage->AddColumn(ThumbnailWidgetRowHandle, FWidgetTooltipColumn::StaticStruct());

				// Used to decide on the actual Thumbnail size
				if (FSizeValueColumn_Experimental* TileItemSizeColumn = DataStorage->GetColumn<FSizeValueColumn_Experimental>(WidgetRow))
				{
					DataStorage->AddColumn(ThumbnailWidgetRowHandle, FSizeValueColumn_Experimental{ .SizeValue = TileItemSizeColumn->SizeValue });
				}

				ThumbnailWidget = DataStorageUi->ConstructWidget(ThumbnailWidgetRowHandle, *OutThumbnailWidgetConstructorPtr, Arguments);
			}
		}

		UE::Editor::DataStorage::RowHandle LabelWidgetRowHandle = UE::Editor::DataStorage::InvalidRowHandle;
		if (OutLabelWidgetConstructorPtr)
		{
			if (LabelWidgetRowHandle = DataStorage->AddRow(DataStorage->FindTable(TedsAssetDataHelper::TableView::GetWidgetTableName())); ThumbnailWidgetRowHandle != UE::Editor::DataStorage::InvalidRowHandle)
			{
				// Referenced Data Row
				DataStorage->AddColumn(LabelWidgetRowHandle, FTypedElementRowReferenceColumn{ .Row = TargetRow });

				LabelWidget = DataStorageUi->ConstructWidget(LabelWidgetRowHandle, *OutLabelWidgetConstructorPtr, Arguments);
			}
		}

		if (!ThumbnailWidget.IsValid())
		{
			ThumbnailWidget = SNullWidget::NullWidget;
		}

		if (!LabelWidget.IsValid())
		{
			LabelWidget = SNullWidget::NullWidget;
		}

		return SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.AutoWidth()
				[
					ThumbnailWidget.ToSharedRef()
				]

				+SHorizontalBox::Slot()
				.FillWidth(1.0f)
				.Padding(ThumbnailNameHorizontalPadding, 0.f, 0.f, 0.f)
				.VAlign(VAlign_Center)
				[
					LabelWidget.ToSharedRef()
				];
	}

	return SNullWidget::NullWidget;
}

TArray<TWeakObjectPtr<const UScriptStruct>> FContentBrowserListViewNameWidgetConstructor::GetLabelColumns()
{
	static TArray<TWeakObjectPtr<const UScriptStruct>> LabelColumns({ TWeakObjectPtr(FAssetNameColumn::StaticStruct()) });
	return LabelColumns;
}

#undef LOCTEXT_NAMESPACE
