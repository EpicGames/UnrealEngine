// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/Interfaces/TypedElementDataStorageFactory.h"
#include "Elements/Interfaces/TypedElementDataStorageUiInterface.h"
#include "Internationalization/Text.h"
#include "UObject/ObjectMacros.h"

#include "ContentBrowserListViewNameWidget.generated.h"

namespace UE::Editor::DataStorage
{
	class ICoreProvider;
} // namespace UE::Editor::DataStorage
class UScriptStruct;

UCLASS()
class UContentBrowserListViewNameWidgetFactory : public UEditorDataStorageFactory
{
	GENERATED_BODY()

public:
	static UE::Editor::DataStorage::IUiProvider::FPurposeID WidgetPurpose;

	~UContentBrowserListViewNameWidgetFactory() override = default;

	TEDSASSETDATA_API virtual void RegisterWidgetPurposes(UE::Editor::DataStorage::IUiProvider& DataStorageUi) const override;

	TEDSASSETDATA_API void RegisterWidgetConstructors(UE::Editor::DataStorage::ICoreProvider& DataStorage,
		UE::Editor::DataStorage::IUiProvider& DataStorageUi) const override;
};

// Content Browser Label + Thumbnail widget
USTRUCT()
struct FContentBrowserListViewNameWidgetConstructor : public FSimpleWidgetConstructor
{
	GENERATED_BODY()

public:
	TEDSASSETDATA_API FContentBrowserListViewNameWidgetConstructor();
	TEDSASSETDATA_API explicit FContentBrowserListViewNameWidgetConstructor(const UScriptStruct* TypeInfo);
	~FContentBrowserListViewNameWidgetConstructor() override = default;

	TEDSASSETDATA_API virtual TSharedPtr<SWidget> CreateWidget(
		UE::Editor::DataStorage::ICoreProvider* DataStorage,
		UE::Editor::DataStorage::IUiProvider* DataStorageUi,
		UE::Editor::DataStorage::RowHandle TargetRow,
		UE::Editor::DataStorage::RowHandle WidgetRow, 
		const UE::Editor::DataStorage::FMetaDataView& Arguments) override;

private:
	TArray<TWeakObjectPtr<const UScriptStruct>> GetLabelColumns();
};
