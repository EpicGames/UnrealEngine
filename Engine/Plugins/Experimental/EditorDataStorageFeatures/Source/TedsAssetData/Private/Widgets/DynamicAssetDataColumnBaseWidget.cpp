// Copyright Epic Games, Inc. All Rights Reserved.

#include "DynamicAssetDataColumnBaseWidget.h"

#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Elements/Framework/TypedElementAttributeBinding.h"
#include "IContentBrowserSingleton.h"
#include "Internationalization/Text.h"
#include "SHyperlinkAssetPreviewWidget.h"
#include "TedsAssetDataColumns.h"
#include "UObject/ObjectMacros.h"
#include "Widgets/SBoxPanel.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DynamicAssetDataColumnBaseWidget)

namespace UE::DynamicColumn::Utilities
{
	FAssetData GetAssetData(const FString& InRawPath)
	{
		const FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
		FSoftObjectPath ObjectPath = FSoftObjectPath(InRawPath);
		FAssetData AssetData = AssetRegistryModule.Get().GetAssetByObjectPath(ObjectPath);
		if (!AssetData.IsValid())
		{
			// If not valid try to get it again if it is a Generated BP
			FString ObjectPathAsString = ObjectPath.ToString();
			if (ObjectPathAsString.RemoveFromEnd(TEXT("_C")))
			{
				AssetData = AssetRegistryModule.Get().GetAssetByObjectPath(FSoftObjectPath(ObjectPathAsString));
			}
		}
		return AssetData;
	}	
}

void UDynamicAssetDataColumnBaseWidgetFactory::RegisterWidgetConstructors(UE::Editor::DataStorage::ICoreProvider& DataStorage,
	UE::Editor::DataStorage::IUiProvider& DataStorageUi) const
{
	using namespace UE::Editor::DataStorage::Queries;
	DataStorageUi.RegisterWidgetFactory<FDynamicAssetDataColumnBaseWidgetConstructor>(DataStorageUi.FindPurpose(DataStorageUi.GetGeneralWidgetPurposeID()), TColumn<FItemStringAttributeColumn_Experimental>("ParentClass"));
	DataStorageUi.RegisterWidgetFactory<FDynamicAssetDataColumnBaseWidgetConstructor>(DataStorageUi.FindPurpose(DataStorageUi.GetGeneralWidgetPurposeID()), TColumn<FItemStringAttributeColumn_Experimental>("Skeleton"));
	DataStorageUi.RegisterWidgetFactory<FDynamicAssetDataColumnBaseWidgetConstructor>(DataStorageUi.FindPurpose(DataStorageUi.GetGeneralWidgetPurposeID()), TColumn<FItemStringAttributeColumn_Experimental>("SourceTexture"));
	DataStorageUi.RegisterWidgetFactory<FDynamicAssetDataColumnBaseWidgetConstructor>(DataStorageUi.FindPurpose(DataStorageUi.GetGeneralWidgetPurposeID()), TColumn<FItemStringAttributeColumn_Experimental>("PhysicsAsset"));
	DataStorageUi.RegisterWidgetFactory<FDynamicAssetDataColumnBaseWidgetConstructor>(DataStorageUi.FindPurpose(DataStorageUi.GetGeneralWidgetPurposeID()), TColumn<FItemStringAttributeColumn_Experimental>("ShadowPhysicsAsset"));

}

FDynamicAssetDataColumnBaseWidgetConstructor::FDynamicAssetDataColumnBaseWidgetConstructor()
	: FSimpleWidgetConstructor(StaticStruct())
{
}

TSharedPtr<SWidget> FDynamicAssetDataColumnBaseWidgetConstructor::CreateWidget(UE::Editor::DataStorage::ICoreProvider* DataStorage,
	UE::Editor::DataStorage::IUiProvider* DataStorageUi,
	UE::Editor::DataStorage::RowHandle TargetRow, UE::Editor::DataStorage::RowHandle WidgetRow,
	const UE::Editor::DataStorage::FMetaDataView& Arguments)
{
	using namespace UE::Editor::DataStorage;
	using namespace UE::Editor::DataStorage::Queries;

	FAttributeBinder Binder(TargetRow, DataStorage);

	FName* DynamicTemplates = MatchedDynamicTemplates.Find(FItemStringAttributeColumn_Experimental::StaticStruct());

	if (!DynamicTemplates)
	{
		return SNullWidget::NullWidget;
	}

	const TAttribute<FAssetData>& AssetDataAttribute = Binder.BindData(*DynamicTemplates, &FItemStringAttributeColumn_Experimental::Value, [DataStorage] (const FString& InValue)
		{
			const FMapKey ReferencedAssetRowKey = FMapKey(FSoftObjectPath(InValue));
			const RowHandle ReferencedRow = DataStorage->LookupMappedRow(UE::Editor::AssetData::MappingDomain, ReferencedAssetRowKey);
			if (ReferencedRow != InvalidRowHandle)
			{
				FAttributeBinder Binder(ReferencedRow, DataStorage);
				return Binder.BindData(&FAssetDataColumn_Experimental::AssetData, FAssetData()).Get();
			}
			// Fallback to try to get the asset data from the Path
			return UE::DynamicColumn::Utilities::GetAssetData(InValue);
		});

	const FOnNavigateAsset& OnNavigate = FOnNavigateAsset::CreateLambda([] (const FAssetData& InAssetData)
		{
			IContentBrowserSingleton::Get().SyncBrowserToAssets({ InAssetData });
		});

	return SNew(SHorizontalBox)
		+SHorizontalBox::Slot()
		.AutoWidth()
		[
			SNew(SHyperlinkAssetPreviewWidget)
			.AssetData(AssetDataAttribute)
			.OnNavigateAsset(OnNavigate)
		];
}
