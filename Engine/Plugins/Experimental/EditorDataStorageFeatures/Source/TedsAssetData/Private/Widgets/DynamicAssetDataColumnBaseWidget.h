// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/Interfaces/TypedElementDataStorageFactory.h"
#include "Elements/Interfaces/TypedElementDataStorageUiInterface.h"

#include "DynamicAssetDataColumnBaseWidget.generated.h"

UCLASS()
class UDynamicAssetDataColumnBaseWidgetFactory : public UEditorDataStorageFactory
{
	GENERATED_BODY()

public:
	~UDynamicAssetDataColumnBaseWidgetFactory() override = default;

	TEDSASSETDATA_API void RegisterWidgetConstructors(UE::Editor::DataStorage::ICoreProvider& DataStorage,
		UE::Editor::DataStorage::IUiProvider& DataStorageUi) const override;
};

USTRUCT()
struct FDynamicAssetDataColumnBaseWidgetConstructor : public FSimpleWidgetConstructor
{
	GENERATED_BODY()

public:
	TEDSASSETDATA_API FDynamicAssetDataColumnBaseWidgetConstructor();
	~FDynamicAssetDataColumnBaseWidgetConstructor() override = default;

	TEDSASSETDATA_API virtual TSharedPtr<SWidget> CreateWidget(
		UE::Editor::DataStorage::ICoreProvider* DataStorage,
		UE::Editor::DataStorage::IUiProvider* DataStorageUi,
		UE::Editor::DataStorage::RowHandle TargetRow,
		UE::Editor::DataStorage::RowHandle WidgetRow,
		const UE::Editor::DataStorage::FMetaDataView& Arguments) override;
};
