// Copyright Epic Games, Inc. All Rights Reserved.

#include "SHyperlinkAssetPreviewWidget.h"

#include "AssetThumbnail.h"
#include "IContentBrowserSingleton.h"
#include "Widgets/Input/SHyperlink.h"

#define LOCTEXT_NAMESPACE "SHyperlinkAssetPreviewWidget"

/** HyperlinkAssetPreview tooltip */
class SHyperlinkAssetToolTip : public SToolTip
{
public:
	SLATE_BEGIN_ARGS(SHyperlinkAssetToolTip)
		: _HyperlinkAssetPreview()
	{ }

		SLATE_ARGUMENT(TSharedPtr<SHyperlinkAssetPreviewWidget>, HyperlinkAssetPreview)

	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs)
	{
		HyperlinkAssetPreview = InArgs._HyperlinkAssetPreview;

		SToolTip::Construct(
			SToolTip::FArguments()
			.TextMargin(0.f)
			.BorderImage(FAppStyle::GetBrush("AssetThumbnail.Tooltip.Border"))
			);
	}

	virtual bool IsEmpty() const override
	{
		return !HyperlinkAssetPreview.IsValid();
	}

	virtual void OnOpening() override
	{
		TSharedPtr<SHyperlinkAssetPreviewWidget> AssetViewItemPin = HyperlinkAssetPreview.Pin();
		if (AssetViewItemPin.IsValid())
		{
			SetContentWidget(AssetViewItemPin->GetThumbnailWidget());
		}
	}

	virtual void OnClosed() override
	{
		ResetContentWidget();
	}

private:
	TWeakPtr<SHyperlinkAssetPreviewWidget> HyperlinkAssetPreview;
};

void SHyperlinkAssetPreviewWidget::Construct(const FArguments& InArgs)
{
	AssetDataAttribute = InArgs._AssetData;
	AssetThumbnailTooltip = MakeShared<FAssetThumbnail>(FAssetData(), 64, 64, UThumbnailManager::Get().GetSharedThumbnailPool());
	OnNavigateAssetDelegate = InArgs._OnNavigateAsset;

	// Set the Thumbnail tooltip
	SetToolTip(SNew(SHyperlinkAssetToolTip).HyperlinkAssetPreview(SharedThis(this)));

	SHyperlink::Construct(
		SHyperlink::FArguments()
		.Visibility(this, &SHyperlinkAssetPreviewWidget::GetHyperlinkVisibility)
		.Text(this, &SHyperlinkAssetPreviewWidget::GetAssetDisplayName)
		.OnNavigate(this, &SHyperlinkAssetPreviewWidget::OnNavigate_Internal)
		.Style(FAppStyle::Get(), "Common.GotoBlueprintHyperlink"));
}

TSharedRef<SWidget> SHyperlinkAssetPreviewWidget::GetThumbnailWidget() const
{
	FAssetData AssetData = FAssetData();
	if (AssetDataAttribute.IsSet())
	{
		AssetData = AssetDataAttribute.Get();
	}

	FAssetThumbnailConfig ThumbnailConfig;
	ThumbnailConfig.bAllowFadeIn = true;
	ThumbnailConfig.bAllowHintText = false;
	ThumbnailConfig.bAllowRealTimeOnHovered = false;
	ThumbnailConfig.bForceGenericThumbnail = !AssetData.IsValid();
	ThumbnailConfig.AllowAssetSpecificThumbnailOverlay = !ThumbnailConfig.bForceGenericThumbnail;
	ThumbnailConfig.ThumbnailLabel = EThumbnailLabel::ClassName;
	ThumbnailConfig.GenericThumbnailSize = 64.f;
	ThumbnailConfig.ShowAssetColor = true;

	if (AssetThumbnailTooltip->GetAssetData() != AssetData)
	{
		AssetThumbnailTooltip->SetAsset(AssetData);
		AssetThumbnailTooltip->RefreshThumbnail();
	}

	return AssetThumbnailTooltip->MakeThumbnailWidget(ThumbnailConfig);
}

void SHyperlinkAssetPreviewWidget::OnNavigate_Internal() const
{
	if (AssetDataAttribute.IsSet())
	{
		FAssetData AssetData = AssetDataAttribute.Get();
		if (AssetData.IsValid())
		{
			OnNavigateAssetDelegate.ExecuteIfBound(AssetData);
		}
	}
}

EVisibility SHyperlinkAssetPreviewWidget::GetHyperlinkVisibility() const
{
	if (AssetDataAttribute.IsSet())
	{
		FAssetData AssetData = AssetDataAttribute.Get();
		return AssetData.IsValid() ? EVisibility::Visible : EVisibility::Collapsed;
	}
	return EVisibility::Collapsed;
}

FText SHyperlinkAssetPreviewWidget::GetAssetDisplayName() const
{
	if (AssetDataAttribute.IsSet())
	{
		const FAssetData& AssetData = AssetDataAttribute.Get();
		if (AssetData.IsValid())
		{
			return FText::FromName(AssetData.AssetName);
		}
	}
	return FText::GetEmpty();
}

#undef LOCTEXT_NAMESPACE
