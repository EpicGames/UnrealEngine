// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetRegistry/AssetData.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Input/SHyperlink.h"

class FAssetThumbnail;

DECLARE_DELEGATE_OneParam(FOnNavigateAsset, const FAssetData&)

/** Widget to display a Hyperlink with a Preview of the asset */
class SHyperlinkAssetPreviewWidget : public SHyperlink
{
public:
	SLATE_BEGIN_ARGS(SHyperlinkAssetPreviewWidget)
	{}

		SLATE_ATTRIBUTE(FAssetData, AssetData)

		/** Will be called only if the AssetData is valid */
		SLATE_EVENT(FOnNavigateAsset, OnNavigateAsset)

	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

	/** Get the preview tooltip for the hyperlink text */
	TSharedRef<SWidget> GetThumbnailWidget() const;

private:
	/** Called when trying to navigate to the asset */
	void OnNavigate_Internal() const;

	/** Visible if the AssetData is valid, Collapsed otherwise */
	EVisibility GetHyperlinkVisibility() const;

	/** Retrieve the asset name to use */
	FText GetAssetDisplayName() const;

private:
	/** AssetData to link */
	TAttribute<FAssetData> AssetDataAttribute;

	/** AssetThumbnail used for the tooltip */
	TSharedPtr<FAssetThumbnail> AssetThumbnailTooltip;

	/** Callback when trying to navigate to an asset */
	FOnNavigateAsset OnNavigateAssetDelegate;
};
