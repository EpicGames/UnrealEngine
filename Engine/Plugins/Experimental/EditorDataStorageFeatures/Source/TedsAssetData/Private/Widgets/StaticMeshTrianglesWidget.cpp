// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/StaticMeshTrianglesWidget.h"

#include "Elements/Framework/TypedElementAttributeBinding.h"
#include "Internationalization/Text.h"
#include "TedsAssetDataColumns.h"
#include "UObject/ObjectMacros.h"
#include "Widgets/Text/STextBlock.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(StaticMeshTrianglesWidget)

void UStaticMeshTrianglesWidgetFactory::RegisterWidgetConstructors(UE::Editor::DataStorage::ICoreProvider& DataStorage,
	UE::Editor::DataStorage::IUiProvider& DataStorageUi) const
{
	using namespace UE::Editor::DataStorage::Queries;
	DataStorageUi.RegisterWidgetFactory<FStaticMeshTrianglesWidgetConstructor>(DataStorageUi.FindPurpose(DataStorageUi.GetGeneralWidgetPurposeID()), TColumn<FItemStringAttributeColumn_Experimental>("Triangles"));
}

FStaticMeshTrianglesWidgetConstructor::FStaticMeshTrianglesWidgetConstructor()
	: FSimpleWidgetConstructor(StaticStruct())
{
}

TSharedPtr<SWidget> FStaticMeshTrianglesWidgetConstructor::CreateWidget(UE::Editor::DataStorage::ICoreProvider* DataStorage,
	UE::Editor::DataStorage::IUiProvider* DataStorageUi,
	UE::Editor::DataStorage::RowHandle TargetRow, UE::Editor::DataStorage::RowHandle WidgetRow,
	const UE::Editor::DataStorage::FMetaDataView& Arguments)
{
	UE::Editor::DataStorage::FAttributeBinder Binder(TargetRow, DataStorage);
	
	return SNew(STextBlock)
			.Text(Binder.BindText(FName("Triangles"), &FItemStringAttributeColumn_Experimental::Value));
			
}
