// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/Interfaces/TypedElementDataStorageFactory.h"
#include "Elements/Interfaces/TypedElementDataStorageUiInterface.h"

#include "StaticMeshTrianglesWidget.generated.h"

UCLASS()
class UStaticMeshTrianglesWidgetFactory : public UEditorDataStorageFactory
{
	GENERATED_BODY()

public:
	~UStaticMeshTrianglesWidgetFactory() override = default;

	TEDSASSETDATA_API void RegisterWidgetConstructors(UE::Editor::DataStorage::ICoreProvider& DataStorage,
		UE::Editor::DataStorage::IUiProvider& DataStorageUi) const override;
};

// Specialized widget to display the "Triangles" metadata on static mesh assets
USTRUCT()
struct FStaticMeshTrianglesWidgetConstructor : public FSimpleWidgetConstructor
{
	GENERATED_BODY()

public:
	TEDSASSETDATA_API FStaticMeshTrianglesWidgetConstructor();
	~FStaticMeshTrianglesWidgetConstructor() override = default;

	TEDSASSETDATA_API virtual TSharedPtr<SWidget> CreateWidget(
		UE::Editor::DataStorage::ICoreProvider* DataStorage,
		UE::Editor::DataStorage::IUiProvider* DataStorageUi,
		UE::Editor::DataStorage::RowHandle TargetRow,
		UE::Editor::DataStorage::RowHandle WidgetRow,
		const UE::Editor::DataStorage::FMetaDataView& Arguments) override;
};