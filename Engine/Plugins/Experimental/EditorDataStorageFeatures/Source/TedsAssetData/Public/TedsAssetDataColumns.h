// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "TedsAssetDataStructs.h"

#include "AssetRegistry/AssetData.h"
#include "AssetViewTypes.h" 
#include "Containers/UnrealString.h"
#include "Containers/VersePath.h"
#include "DataStorage/CommonTypes.h"
#include "DataStorage/MapKey.h"
#include "Elements/Interfaces/TypedElementQueryStorageInterfaces.h"
#include "Internationalization/Text.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"

#include "TedsAssetDataColumns.generated.h"

namespace UE::Editor::AssetData
{
	inline static const FName MappingDomain = "Asset";
}

USTRUCT()
struct FAssetPathColumn_Experimental : public FEditorDataStorageColumn
{
	GENERATED_BODY()
	
	UPROPERTY(meta = (Searchable, Sortable))
	FName Path;
};

USTRUCT()
struct FFolderTypeColumn_Experimental : public FEditorDataStorageColumn
{
	GENERATED_BODY()
	
	EFolderType FolderType;
};

USTRUCT()
struct FAssetDataColumn_Experimental : public FEditorDataStorageColumn
{
	GENERATED_BODY()
	
	UPROPERTY()
	FAssetData AssetData;
};

// Tag to identify assets
USTRUCT(meta = (DisplayName = "Asset"))
struct FAssetTag final : public FEditorDataStorageTag
{
	GENERATED_BODY()
};

// Tag to identify assets with private visibility
USTRUCT(meta = (DisplayName = "Private Asset"))
struct FPrivateAssetTag final : public FEditorDataStorageTag
{
	GENERATED_BODY()
};

// Tag to identify assets with epic internal visibility
USTRUCT(meta = (DisplayName = "Epic Internal Asset"))
struct FEpicInternalAssetTag final : public FEditorDataStorageTag
{
	GENERATED_BODY()
};

// Tag to identify assets with public visibility
USTRUCT(meta = (DisplayName = "Public Asset"))
struct FPublicAssetTag final : public FEditorDataStorageTag
{
	GENERATED_BODY()
};

// Column to store the name of an asset
USTRUCT(meta = (DisplayName = "Name"))
struct FAssetNameColumn final : public FEditorDataStorageColumn
{
	GENERATED_BODY()

	// The name of the asset
	UPROPERTY(meta = (Searchable, Sortable))
	FName Name;
};

// Column to store the type of an asset
USTRUCT(meta = (DisplayName = "Type"))
struct FAssetClassColumn final : public FEditorDataStorageColumn
{
	GENERATED_BODY()

	// The path of the type
	UPROPERTY()
	FTopLevelAssetPath ClassPath;
};

// Column to store the disk size of an asset
USTRUCT(meta = (DisplayName = "Disk Size"))
struct FDiskSizeColumn final : public FEditorDataStorageColumn
{
	GENERATED_BODY()

	// Size on disk (in bytes)
	UPROPERTY(meta = (Sortable))
	int64 DiskSize = 0;
};


// Column to store the verse path of an asset
USTRUCT(meta = (DisplayName = "Verse Path"))
struct FVersePathColumn final : public FEditorDataStorageColumn
{
	GENERATED_BODY()
	
	UE::Core::FVersePath VersePath;
};

// Used to notify the dependent queries of the update to the path of the row
USTRUCT()
struct FUpdatedPathTag : public FEditorDataStorageTag
{
	GENERATED_BODY()
};

// Used to notify the dependent queries of a update to the asset data
USTRUCT()
struct FUpdatedAssetDataTag : public FEditorDataStorageTag
{
	GENERATED_BODY()
};

USTRUCT(meta = (DisplayName = "CB Item Path"))
struct FVirtualPathColumn_Experimental : public FEditorDataStorageColumn
{
	GENERATED_BODY()

	UPROPERTY(meta = (Searchable, Sortable))
	FName VirtualPath;
};

USTRUCT()
struct FItemAttributeBaseColumn_Experimental : public FEditorDataStorageColumn
{
	GENERATED_BODY()

	TSharedPtr<const UE::Editor::AssetData::FItemAttributeMetadata> AttributeMetadata;
};

USTRUCT()
struct FItemTextAttributeColumn_Experimental : public FItemAttributeBaseColumn_Experimental
{
	GENERATED_BODY()

	UPROPERTY(meta = (Searchable, Sortable))
	FText Value;
};

USTRUCT(meta=(EditorDataStorage_DynamicColumnTemplate))
struct FItemStringAttributeColumn_Experimental : public FItemAttributeBaseColumn_Experimental
{
	GENERATED_BODY()

	UPROPERTY(meta = (Searchable, Sortable))
	FString Value;
};