// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "UObject/NameTypes.h"

namespace TedsAssetDataHelper
{
	namespace MetaDataNames
	{
		TEDSASSETDATA_API FName GetThumbnailStatusMetaDataName();
		TEDSASSETDATA_API FName GetThumbnailFadeInMetaDataName();
		TEDSASSETDATA_API FName GetThumbnailHintTextMetaDataName();
		TEDSASSETDATA_API FName GetThumbnailRealTimeOnHoveredMetaDataName();
		TEDSASSETDATA_API FName GetThumbnailSizeOffsetMetaDataName();
		TEDSASSETDATA_API FName GetThumbnailCanDisplayEditModePrimitiveTools();
	}

	namespace TableView
	{
		TEDSASSETDATA_API FName GetWidgetTableName();
	}

	/** Remove all / from the start of InStringToModify */
	TEDSASSETDATA_API FString RemoveSlashFromStart(const FString& InStringToModify);

	/** Remove Last / and successive char from InStringToModify */
	TEDSASSETDATA_API FString RemoveAllFromLastSlash(const FString& InStringToModify);
}
