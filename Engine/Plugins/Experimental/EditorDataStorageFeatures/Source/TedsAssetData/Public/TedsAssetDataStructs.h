// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"
#include "Containers/UnrealString.h"
#include "Internationalization/Text.h"

namespace UE::Editor::AssetData
{ 

struct FItemAttributeMetadata
{
	/** The kind of data represented by this tag value */
	UObject::FAssetRegistryTag::ETagType TagType = UObject::FAssetRegistryTag::TT_Hidden;

	/** Flags giving hints at how to display this tag value in the UI (see ETagDisplay) */
	uint32 DisplayFlags = UObject::FAssetRegistryTag::TD_None;

	/** Resolved display name of the associated tag */
	FText DisplayName;

	/** Optional tooltip of the associated tag */
	FText TooltipText;

	/** Optional suffix to apply to values of the tag attribute in the UI */
	FText Suffix;

	/** Optional value which denotes which values should be considered "important" in the UI */
	FString ImportantValue;
};

}
