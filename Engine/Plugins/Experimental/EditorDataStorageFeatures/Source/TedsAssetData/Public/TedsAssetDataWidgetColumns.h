// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "TedsAssetDataStructs.h"

#include "AssetThumbnail.h"
#include "DataStorage/CommonTypes.h"
#include "Framework/SlateDelegates.h"
#include "Templates/SharedPointer.h"

#include "TedsAssetDataWidgetColumns.generated.h"


// Column to store the thumbnail size enum value of an asset
USTRUCT(meta = (DisplayName = "Thumbnail Size"))
struct FThumbnailSizeColumn_Experimental final : public FEditorDataStorageColumn
{
	GENERATED_BODY()

	// EThumbnailSize
	EThumbnailSize ThumbnailSize;
};

// Column to store the size value of a widget
USTRUCT(meta = (DisplayName = "Size Value"))
struct FSizeValueColumn_Experimental final : public FEditorDataStorageColumn
{
	GENERATED_BODY()

	UPROPERTY(meta = (IgnoreForMemberInitializationTest))
	float SizeValue = 0;
};

// Column to store the padding
USTRUCT(meta = (DisplayName = "Padding"))
struct FWidgetPaddingColumn_Experimental final : public FEditorDataStorageColumn
{
	GENERATED_BODY()

	UPROPERTY()
	FMargin Padding;
};

/**
 * Column added to a widget row to allow systems to provide an external EditMode trigger
 */
USTRUCT(meta = (DisplayName = "Content Browser Settings"))
struct FThumbnailEditModeColumn_Experimental final : public FEditorDataStorageColumn
{
	GENERATED_BODY()

	UPROPERTY(meta = (IgnoreForMemberInitializationTest))
	bool IsEditModeToggled = false;
};

/**
 * Column added to a widget row to allow systems to provide an external Overflow Policy
 */
USTRUCT(meta = (DisplayName = "Widget with externally managed OverflowPolicy"))
struct FTextOverflowPolicyColumn_Experimental : public FEditorDataStorageColumn
{
	GENERATED_BODY()
	ETextOverflowPolicy OverflowPolicy;
};

/**
 * Column added to a widget row to allow systems to provide an external Visibility
 */
USTRUCT(meta = (DisplayName = "Widget with externally managed Visibility"))
struct FWidgetVisibilityColumn_Experimental : public FEditorDataStorageColumn
{
	GENERATED_BODY()
	EVisibility Visibility;
};

/**
 * Column added to a widget row to allow systems to provide an external FontStyle
 */
USTRUCT(meta = (DisplayName = "Widget with externally managed FontStyle"))
struct FFontStyleColumn_Experimental : public FEditorDataStorageColumn
{
	GENERATED_BODY()
	FSlateFontInfo FontInfo;
};

/**
 * Column added to a widget row to allow systems to store the widget tooltip
 */
USTRUCT(meta = (DisplayName = "Widget Tooltip"))
struct FLocalWidgetTooltipColumn_Experimental : public FEditorDataStorageColumn
{
	GENERATED_BODY()
	TSharedPtr<class IToolTip> Tooltip;
};

DECLARE_DELEGATE_RetVal(const FSlateBrush*, FOnGetWidgetSlateBrush)
/**
 * Column added to a widget row to allow systems to provide an external brush callback for the widget
 * TODO: This need to be later on removed and replaced by updating the value only when needed (Hover/Selection etc...)
 */
USTRUCT(meta = (DisplayName = "Widget with externally managed Brush"))
struct FOnGetWidgetSlateBrushColumn_Experimental : public FEditorDataStorageColumn
{
	GENERATED_BODY()
	TSharedPtr<FOnGetWidgetSlateBrush> OnGetWidgetSlateBrush;
};

DECLARE_DELEGATE_RetVal(FSlateColor, FOnGetWidgetColorAndOpacity)
/**
 * Column added to a widget row to allow systems to provide an external ColorAndOpacity callback for the widget
 * TODO: This need to be later on removed and replaced by updating the value only when needed (Hover/Selection etc...)
 */
USTRUCT(meta = (DisplayName = "Widget with externally managed ColorAndOpacity"))
struct FOnGetWidgetColorAndOpacityColumn_Experimental : public FEditorDataStorageColumn
{
	GENERATED_BODY()
	TSharedPtr<FOnGetWidgetColorAndOpacity> OnGetWidgetColorAndOpacity;
};
