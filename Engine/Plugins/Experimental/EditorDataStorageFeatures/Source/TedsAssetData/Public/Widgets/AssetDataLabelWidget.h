// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/Interfaces/TypedElementDataStorageFactory.h"
#include "Elements/Interfaces/TypedElementDataStorageUiInterface.h"
#include "Internationalization/Text.h"
#include "UObject/ObjectMacros.h"

#include "AssetDataLabelWidget.generated.h"

namespace UE::Editor::DataStorage
{
	class ICoreProvider;
} // UE::Editor::DataStorage

class UScriptStruct;

UCLASS()
class UAssetDataLabelWidgetFactory : public UEditorDataStorageFactory
{
	GENERATED_BODY()

public:
	~UAssetDataLabelWidgetFactory() override = default;

	TEDSASSETDATA_API void RegisterWidgetConstructors(UE::Editor::DataStorage::ICoreProvider& DataStorage,
		UE::Editor::DataStorage::IUiProvider& DataStorageUi) const override;
};

// Label widget for assets in TEDS
USTRUCT()
struct FAssetDataLabelWidgetConstructor : public FSimpleWidgetConstructor
{
	GENERATED_BODY()

public:
	TEDSASSETDATA_API FAssetDataLabelWidgetConstructor();
	TEDSASSETDATA_API explicit FAssetDataLabelWidgetConstructor(const UScriptStruct* TypeInfo);
	~FAssetDataLabelWidgetConstructor() override = default;

	TEDSASSETDATA_API virtual TSharedPtr<SWidget> CreateWidget(
		UE::Editor::DataStorage::ICoreProvider* DataStorage,
		UE::Editor::DataStorage::IUiProvider* DataStorageUi,
		UE::Editor::DataStorage::RowHandle TargetRow,
		UE::Editor::DataStorage::RowHandle WidgetRow, 
		const UE::Editor::DataStorage::FMetaDataView& Arguments) override;

private:
	TArray<TWeakObjectPtr<const UScriptStruct>> GetLabelColumns();
};