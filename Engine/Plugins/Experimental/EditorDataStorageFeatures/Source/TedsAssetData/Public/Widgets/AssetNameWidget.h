// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "TedsAssetDataColumns.h"
#include "Elements/Interfaces/TypedElementDataStorageFactory.h"
#include "Elements/Interfaces/TypedElementDataStorageUiInterface.h"
#include "Internationalization/Text.h"
#include "UObject/ObjectMacros.h"

#include "AssetNameWidget.generated.h"

class IEditorDataStorageProvider;
class UScriptStruct;

namespace UE::Editor::DataStorage
{
	class FAssetNameWidgetSorter_NoSlash final : public TColumnSorterInterface<FColumnSorterInterface::ESortType::HybridSort, FAssetNameColumn>
	{
	public:
		virtual FText GetShortName() const override;

	protected:
		virtual int32 Compare(const FAssetNameColumn& Left, const FAssetNameColumn& Right) const override;
		virtual FPrefixInfo CalculatePrefix(const FAssetNameColumn& Column, uint32 ByteIndex) const override;
	};

	class FAssetNameWidgetSorter_WithSlash final : public TColumnSorterInterface<FColumnSorterInterface::ESortType::HybridSort, FAssetNameColumn>
	{
	public:
		virtual FText GetShortName() const override;

	protected:
		virtual int32 Compare(const FAssetNameColumn& Left, const FAssetNameColumn& Right) const override;
		virtual FPrefixInfo CalculatePrefix(const FAssetNameColumn& Column, uint32 ByteIndex) const override;
	};
}

UCLASS()
class UAssetNameWidgetFactory : public UEditorDataStorageFactory
{
	GENERATED_BODY()

public:
	virtual ~UAssetNameWidgetFactory() override = default;

	TEDSASSETDATA_API virtual void RegisterWidgetConstructors(UE::Editor::DataStorage::ICoreProvider& DataStorage,
		UE::Editor::DataStorage::IUiProvider& DataStorageUi) const override;
};

// Label for assets in TEDS
USTRUCT()
struct FAssetNameWidgetConstructor : public FSimpleWidgetConstructor
{
	GENERATED_BODY()

public:
	TEDSASSETDATA_API FAssetNameWidgetConstructor();

	TEDSASSETDATA_API virtual TSharedPtr<SWidget> CreateWidget(
		UE::Editor::DataStorage::ICoreProvider* DataStorage,
		UE::Editor::DataStorage::IUiProvider* DataStorageUi,
		UE::Editor::DataStorage::RowHandle TargetRow,
		UE::Editor::DataStorage::RowHandle WidgetRow, 
		const UE::Editor::DataStorage::FMetaDataView& Arguments) override;

	virtual TArray<TSharedPtr<const UE::Editor::DataStorage::FColumnSorterInterface>> ConstructColumnSorters(
		UE::Editor::DataStorage::ICoreProvider* DataStorage,
		UE::Editor::DataStorage::IUiProvider* DataStorageUi,
		const UE::Editor::DataStorage::FMetaDataView& Arguments) override;
};
