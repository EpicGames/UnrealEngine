// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataStorage/Handles.h"
#include "Elements/Interfaces/TypedElementDataStorageUiInterface.h"
#include "Widgets/SCompoundWidget.h"

namespace UE::Editor::DataStorage
{
	class ITedsWidget;
}

class SBorder;

class STedsAssetPreviewWidget : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(STedsAssetPreviewWidget)
		: _WidgetPurpose(UE::Editor::DataStorage::IUiProvider::FPurposeInfo("AssetPreview", "Default", NAME_None).GeneratePurposeID())
		, _HeaderWidgetPurpose(UE::Editor::DataStorage::IUiProvider::FPurposeInfo("AssetPreview", "Header", "Default").GeneratePurposeID())
		, _ThumbnailWidgetPurpose(UE::Editor::DataStorage::IUiProvider::FPurposeInfo("AssetPreview", "Thumbnail", NAME_None).GeneratePurposeID())
		, _TargetRow(UE::Editor::DataStorage::InvalidRowHandle)
	{}

		// The widget purpose to use to create the tile widget
		SLATE_ARGUMENT(UE::Editor::DataStorage::IUiProvider::FPurposeID, WidgetPurpose)

		// The widget header purpose to use to create the tile widget
		SLATE_ARGUMENT(UE::Editor::DataStorage::IUiProvider::FPurposeID, HeaderWidgetPurpose)

		// The widget thumbnail purpose to use to create the tile widget
		SLATE_ARGUMENT(UE::Editor::DataStorage::IUiProvider::FPurposeID, ThumbnailWidgetPurpose)

		// Target Row of the Item we want to create the preview of
		SLATE_ARGUMENT(UE::Editor::DataStorage::RowHandle, TargetRow)

	SLATE_END_ARGS()

	TEDSASSETDATA_API void Construct(const FArguments& InArgs);

	/** Get the Widget Row Handle of this TedsWidget */
	TEDSASSETDATA_API UE::Editor::DataStorage::RowHandle GetWidgetRowHandle() const;

	/** Reconstructs all the TedsWidget used in this widget */
	TEDSASSETDATA_API void ReconstructTedsWidget();

	/** Set the current Target Row to use, needs ReconstructTedsWidget to change */
	TEDSASSETDATA_API void SetTargetRow(UE::Editor::DataStorage::RowHandle InTargetRow);

private:
	/** Create and initialize the ITedsWidget */
	void CreateTedsWidget();

	/** Create all Teds widgets for this widget based on the current TargetRow */
	void ConstructWidget();

	/** Get the columns used for the Asset Preview Header TEDS widget */
	TArray<TWeakObjectPtr<const UScriptStruct>> GetHeaderColumns();

	/** Get the columns used for the Thumbnail TEDS widget */
    TArray<TWeakObjectPtr<const UScriptStruct>> GetThumbnailColumns();

	/** Get the columns used for the Asset Preview BasicInfo TEDS widget */
	TArray<TWeakObjectPtr<const UScriptStruct>> GetBasicInfoColumns();

	/** Get the columns used for the Asset Preview AdvancedInfo TEDS widget */
	TArray<TWeakObjectPtr<const UScriptStruct>> GetAdvancedInfoColumns();

	/** Get the current Target Row */
	UE::Editor::DataStorage::RowHandle GetReferencedRowHandle() const;

private:
	/** Header Row Handle */
	UE::Editor::DataStorage::RowHandle HeaderWidgetRowHandle = UE::Editor::DataStorage::InvalidRowHandle;

	/** Thumbnail Row Handle */
	UE::Editor::DataStorage::RowHandle ThumbnailWidgetRowHandle = UE::Editor::DataStorage::InvalidRowHandle;

	/** Basic Info Row Handle */
	UE::Editor::DataStorage::RowHandle BasicInfoWidgetRowHandle = UE::Editor::DataStorage::InvalidRowHandle;

	/** Advanced Info Row Handle */
	UE::Editor::DataStorage::RowHandle AdvancedInfoWidgetRowHandle = UE::Editor::DataStorage::InvalidRowHandle;

	/** Target Row Handle */
	UE::Editor::DataStorage::RowHandle TargetRow = UE::Editor::DataStorage::InvalidRowHandle;

	/** Asset preview Purpose */
	UE::Editor::DataStorage::IUiProvider::FPurposeID WidgetPurpose;

	/** Asset preview Header Purpose */
	UE::Editor::DataStorage::IUiProvider::FPurposeID WidgetHeaderPurpose;

	/** Asset preview Thumbnail Purpose */
	UE::Editor::DataStorage::IUiProvider::FPurposeID ThumbnailWidgetPurpose;

	/** Whether the thumbnail is big or small */
	bool bIsThumbnailExpanded = false;

	/** Whether the thumbnail edit mode is enabled or not */
	bool IsEditModeEnabled = false;

	/** Our teds widget */
	TSharedPtr<UE::Editor::DataStorage::ITedsWidget> TedsWidget;

	/** Border containing the widget (child of the TedsWdiget) */
	TSharedPtr<SBorder> PreviewPanel;
};
