// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataStorage/CommonTypes.h"
#include "Framework/SlateDelegates.h"
#include "Templates/SharedPointer.h"

#include "TedsAssetPreviewWidgetColumns.generated.h"


/**
 * Column added to a widget row when an external widget manages On Clicked behavior
 */
USTRUCT(meta = (DisplayName = "Widget with externally managed On Clicked behavior"))
struct FExternalWidgetOnClickedColumn_Experimental : public FEditorDataStorageColumn
{
	GENERATED_BODY()

	/** Delegate to execute during the On Click of the widget */
	// TODO: BindEvent currently do not support custom return value, since FReply do not have a default constructor it will fail
	// FOnClicked OnClicked;
	TSharedPtr<TDelegate<void()>> OnClicked;
};
