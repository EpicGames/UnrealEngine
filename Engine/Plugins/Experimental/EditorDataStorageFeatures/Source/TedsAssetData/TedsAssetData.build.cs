// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

// Experimental test module. please refrain from depending on it until this warning is removed
public class TedsAssetData : ModuleRules
{
	public TedsAssetData(ReadOnlyTargetRules Target) : base(Target)
	{
		// Enable truncation warnings in this module
		CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Warning;

		ShortName = "TEDSAssetD";

		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		if (Target.bBuildEditor)
		{
			PublicIncludePaths.AddRange(new string[] { });
			PrivateIncludePaths.AddRange(new string[] { });

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"AssetTools",
					"AssetDefinition",
					"AssetRegistry",
					"ContentBrowserData",
					"Core",
					"CoreUObject",
					"Engine",
					"Slate",
					"SlateCore",
					"TedsAlerts",
					"TedsTableViewer",
					"ToolWidgets",
					"TypedElementFramework",
					"UnrealEd",
				});

			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Projects",
					"ContentBrowser"
				});

		}
	}
}
