// Copyright Epic Games, Inc. All Rights Reserved.

#include "TedsContentBrowserModule.h"

#include "Framework/Application/SlateApplication.h"
#include "ContentBrowserModule.h"
#include "IContentBrowserSingleton.h"
#include "Columns/SlateDelegateColumns.h"
#include "DataStorage/Features.h"
#include "DataStorage/Queries/Types.h"
#include "Elements/Columns/TypedElementFolderColumns.h"
#include "Elements/Interfaces/TypedElementDataStorageInterface.h"
#include "Experimental/ContentBrowserViewExtender.h"
#include "Modules/ModuleManager.h"
#include "TedsAlertColumns.h"
#include "TedsAssetDataColumns.h"
#include "TedsAssetDataWidgetColumns.h"
#include "TedsRowArrayNode.h"
#include "TedsRowViewNode.h"
#include "Elements/Columns/TypedElementLabelColumns.h"
#include "Elements/Columns/TypedElementTypeInfoColumns.h"
#include "Elements/Columns/TypedElementVisibilityColumns.h"
#include "Elements/Framework/TypedElementQueryBuilder.h"
#include "Widgets/STedsTableViewer.h"
#include "Widgets/AssetPreview/STedsAssetPreviewWidget.h"
#include "Widgets/STedsTileViewer.h"
#include "Widgets/Views/SListView.h"

#define LOCTEXT_NAMESPACE "TedsContentBrowserModule"

namespace UE::Editor::ContentBrowser
{
	using namespace DataStorage;

	static bool bEnableTedsContentBrowser = false;
	static FAutoConsoleVariableRef CVarUseTEDSOutliner(
		TEXT("TEDS.UI.EnableTedsContentBrowser"),
		bEnableTedsContentBrowser,
		TEXT("Add the Teds Content Browser as a custom view (requires re-opening any currently open content browsers)")
		, FConsoleVariableDelegate::CreateLambda([](IConsoleVariable* /*CVar*/)
		{
			FContentBrowserModule& ContentBrowserModule = FModuleManager::Get().GetModuleChecked<FContentBrowserModule>(TEXT("ContentBrowser"));
			FTedsContentBrowserModule& TedsContentBrowserModule = FModuleManager::Get().GetModuleChecked<FTedsContentBrowserModule>(TEXT("TedsContentBrowser"));
			
			if(bEnableTedsContentBrowser)
			{
				ContentBrowserModule.SetContentBrowserViewExtender(
					FContentBrowserModule::FCreateViewExtender::CreateStatic(&FTedsContentBrowserModule::CreateContentBrowserViewExtender));
			}
			else
			{
				ContentBrowserModule.SetContentBrowserViewExtender(nullptr);
			}
		}));

	enum class ETableViewerMode : int32
	{
		List = 0,
		Tile = 1
	};

	static int32 TableViewerMode = static_cast<int>(ETableViewerMode::List);
	static FAutoConsoleVariableRef CvarSetTableViewMode(
		TEXT("TEDS.UI.TedsContentBrowserViewMode"),
		TableViewerMode,
		TEXT("Set the view mode of the TEDS-CB. 0 = List View, 1 = Tile View. (requires re-opening any currently open content browsers)")); 

	TSharedPtr<IContentBrowserViewExtender> FTedsContentBrowserModule::CreateContentBrowserViewExtender()
	{
		return MakeShared<FTedsContentBrowserViewExtender>();
	}
	
	static bool bEnableTestContentSource = false;
	static FAutoConsoleVariableRef CVarEnableTestContentSource(
		TEXT("TEDS.UI.EnableTestContentSource"),
		bEnableTestContentSource,
		TEXT("Add a test content source to the Content Browser")
		, FConsoleVariableDelegate::CreateLambda([](IConsoleVariable* /*CVar*/)
		{
			FTedsContentBrowserModule& TedsContentBrowserModule = FModuleManager::Get().GetModuleChecked<FTedsContentBrowserModule>(TEXT("TedsContentBrowser"));
			
			if(bEnableTestContentSource)
			{
				TedsContentBrowserModule.RegisterTestContentSource();
			}
			else
			{
				TedsContentBrowserModule.UnregisterTestContentSource();
			}
		}));

	static const FName TestContentSourceName("TestContentSource");

	void FTedsContentBrowserModule::RegisterTestContentSource()
	{
		IContentBrowserSingleton::Get().RegisterContentSourceFactory(TestContentSourceName, IContentBrowserSingleton::FContentSourceFactory::CreateLambda(
			[]() -> TSharedRef<IContentSource>
		{
			return MakeShared<FTestContentSource>();
		}));
	}

	void FTedsContentBrowserModule::UnregisterTestContentSource()
	{
		IContentBrowserSingleton::Get().UnregisterContentSourceFactory(TestContentSourceName);
	}

	void FTedsContentBrowserModule::StartupModule()
	{
		IModuleInterface::StartupModule();
	}

	void FTedsContentBrowserModule::ShutdownModule()
	{
		IModuleInterface::ShutdownModule();
	}

	void FTedsContentBrowserViewExtender::RefreshRows(TArray<TSharedPtr<FAssetViewItem>>* InItemsSource)
	{
		if(!InItemsSource)
		{
			return;
		}

		RowQueryStack->GetMutableRows().Empty();
		ContentBrowserItemMap.Empty();

		for(TSharedPtr<FAssetViewItem> Item : *InItemsSource)
		{
			if(!Item)
			{
				continue;
			}

			AddRow(Item);
		}

		RowQueryStack->MarkDirty();
	}

	void FTedsContentBrowserViewExtender::AddRow(const TSharedPtr<FAssetViewItem>& Item)
	{
		RowHandle RowHandle = GetRowFromAssetViewItem(Item);
		
		if(DataStorage->IsRowAssigned(RowHandle))
		{
			ContentBrowserItemMap.Emplace(RowHandle, Item);
			
			FRowHandleArray& Rows = RowQueryStack->GetMutableRows();
			Rows.Add(RowHandle);
		}
	}

	TSharedPtr<FAssetViewItem> FTedsContentBrowserViewExtender::GetAssetViewItemFromRow(RowHandle Row)
	{
		// CB 2.0 TODO: Since FAssetViewItem was private previously, there is no good way to lookup currently aside from storing a map
		if(TWeakPtr<FAssetViewItem>* AssetViewItem = ContentBrowserItemMap.Find(Row))
		{
			if(TSharedPtr<FAssetViewItem> AssetViewItemPin = AssetViewItem->Pin())
			{
				return AssetViewItemPin;
			}
		}

		return nullptr;
	}

	DataStorage::RowHandle FTedsContentBrowserViewExtender::GetRowFromAssetViewItem(const TSharedPtr<FAssetViewItem>& Item)
	{
		FAssetData ItemAssetData;
		FName PackagePath;
		
		RowHandle RowHandle = InvalidRowHandle;
		
		if (Item->GetItem().Legacy_TryGetAssetData(ItemAssetData))
		{
			FMapKey Key = FMapKey(ItemAssetData.GetSoftObjectPath());
			RowHandle = DataStorage->LookupMappedRow(AssetData::MappingDomain, Key);
		}
		else if(Item->GetItem().Legacy_TryGetPackagePath(PackagePath))
		{
			FMapKeyView Key = FMapKeyView(PackagePath);
			RowHandle = DataStorage->LookupMappedRow(AssetData::MappingDomain, Key);
		}

		return RowHandle;
	}

	void FTedsContentBrowserViewExtender::CreateListView()
	{
		// Sample dynamic column to display the "Skeleton" attribute on skeletal meshes
		// We probably want the dynamic columns in the table viewer to be data driven based on the rows in the future
		const UScriptStruct* DynamicSkeletalMeshSkeletonColumn = DataStorage->GenerateDynamicColumn(FDynamicColumnDescription
						{
							.TemplateType = FItemStringAttributeColumn_Experimental::StaticStruct(),
							.Identifier = "Skeleton"
						});

		const UScriptStruct* DynamicBlueprintColumn = DataStorage->GenerateDynamicColumn(FDynamicColumnDescription
						{
							.TemplateType = FItemStringAttributeColumn_Experimental::StaticStruct(),
							.Identifier = "ParentClass"
						});

		const UScriptStruct* DynamicSourceTextureColumn = DataStorage->GenerateDynamicColumn(FDynamicColumnDescription
						{
							.TemplateType = FItemStringAttributeColumn_Experimental::StaticStruct(),
							.Identifier = "SourceTexture"
						});

		const UScriptStruct* DynamicPhysicsAssetColumn = DataStorage->GenerateDynamicColumn(FDynamicColumnDescription
						{
							.TemplateType = FItemStringAttributeColumn_Experimental::StaticStruct(),
							.Identifier = "PhysicsAsset"
						});

		const UScriptStruct* DynamicShadowPhysicsAssetColumn = DataStorage->GenerateDynamicColumn(FDynamicColumnDescription
						{
							.TemplateType = FItemStringAttributeColumn_Experimental::StaticStruct(),
							.Identifier = "ShadowPhysicsAsset"
						});

		CustomViewType = ETableViewMode::List;

		// Create the table viewer widget
		TableViewer = SNew(STedsTableViewer)
					.ItemHeight_Lambda([this] () { return GetListViewItemHeight(); })
					.ItemPadding_Lambda([this] () { return GetListViewItemPadding(); })
					.QueryStack(RowQueryStack)
					.CellWidgetPurpose(IUiProvider::FPurposeInfo("ContentBrowser", "RowLabel", NAME_None).GeneratePurposeID())
					.HeaderWidgetPurpose(IUiProvider::FPurposeInfo("General", "Header", NAME_None).GeneratePurposeID())
					// Default list of columns to display
					.Columns({ FAssetNameColumn::StaticStruct(), FTedsAlertColumn::StaticStruct(),
						FAssetClassColumn::StaticStruct(), FAssetTag::StaticStruct(), FFolderTag::StaticStruct(), FAssetPathColumn_Experimental::StaticStruct(),
						FDiskSizeColumn::StaticStruct(), FVirtualPathColumn_Experimental::StaticStruct(),
						DynamicSkeletalMeshSkeletonColumn, DynamicBlueprintColumn, DynamicSourceTextureColumn,
						DynamicPhysicsAssetColumn, DynamicShadowPhysicsAssetColumn })
					.ListSelectionMode(ESelectionMode::Multi)
					.OnSelectionChanged_Lambda([this](RowHandle Row, ESelectInfo::Type SelectInfo)
					{
						UpdateAssetPreviewTargetRow();

						if(TSharedPtr<FAssetViewItem> AssetViewItem = GetAssetViewItemFromRow(Row))
						{
							// CB 2.0 TODO: Does the CB use ESelectInfo and we need to propagate it from the table viewer?
							OnSelectionChangedDelegate.Execute(AssetViewItem, ESelectInfo::Direct);
						}
					});

		const RowHandle TileViewRowHandle = TableViewer->GetWidgetRowHandle();
		DataStorage->AddColumn(TileViewRowHandle, FSizeValueColumn_Experimental{ .SizeValue = GetThumbnailSizeValue() });

		BindViewColumns();
	}

	void FTedsContentBrowserViewExtender::CreateTileView()
	{
		CustomViewType = ETableViewMode::Tile;

		// Create the table viewer widget
		TableViewer = SNew(STedsTileViewer)
					.ItemAlignment(EListItemAlignment::LeftAligned)
					.TileStyle(&FAppStyle::Get().GetWidgetStyle<FTableRowStyle>("ContentBrowser.AssetListView.TileTableRow"))
					.QueryStack(RowQueryStack)
					.WidgetPurpose(IUiProvider::FPurposeInfo("ContentBrowser", "TileLabel", NAME_None).GeneratePurposeID())
					.SelectionMode(ESelectionMode::Multi)
					.ItemWidth_Lambda([this] () { return GetTileViewItemWidth(); })
					.ItemHeight_Lambda([this] () { return GetTileViewItemHeight(); })
					.OnSelectionChanged_Lambda([this](RowHandle Row)
					{
						UpdateAssetPreviewTargetRow();

						if (TSharedPtr<FAssetViewItem> AssetViewItem = GetAssetViewItemFromRow(Row))
						{
							// CB 2.0 TODO: Does the CB use ESelectInfo and we need to propagate it from the table viewer?
							OnSelectionChangedDelegate.Execute(AssetViewItem, ESelectInfo::Direct);
						}
					});

		const RowHandle TileViewRowHandle = TableViewer->GetWidgetRowHandle();
		DataStorage->AddColumn(TileViewRowHandle, FThumbnailSizeColumn_Experimental{ .ThumbnailSize = GetThumbnailSize() });
		DataStorage->AddColumn(TileViewRowHandle, FSizeValueColumn_Experimental{ .SizeValue = GetThumbnailSizeValue() });
		DataStorage->AddColumn(TileViewRowHandle, FThumbnailEditModeColumn_Experimental{ .IsEditModeToggled = IsThumbnailEditMode });

		BindViewColumns();
	}

	void FTedsContentBrowserViewExtender::BindViewColumns()
	{
		if (!TableViewer)
		{
			return;
		}
		
		// Bind the delegates the CB view extender requires to delegates in TEDS columns on the widget row that are fired
		// when the event occurs
		const RowHandle WidgetRow = TableViewer->GetWidgetRowHandle();

		if (FWidgetContextMenuColumn* ContextMenuColumn = DataStorage->GetColumn<FWidgetContextMenuColumn>(WidgetRow))
		{
			ContextMenuColumn->OnContextMenuOpening = MakeShared<FOnContextMenuOpening>();
			ContextMenuColumn->OnContextMenuOpening->BindLambda([this]()
			{
				return OnContextMenuOpenedDelegate.Execute();
			});
		}
		if (FWidgetRowScrolledIntoView* ScrolledIntoViewColumn = DataStorage->GetColumn<FWidgetRowScrolledIntoView>(WidgetRow))
		{
			ScrolledIntoViewColumn->OnItemScrolledIntoView = MakeShared<FWidgetRowScrolledIntoView::FOnItemScrolledIntoView>();
			ScrolledIntoViewColumn->OnItemScrolledIntoView->BindLambda([this](FTedsRowHandle Row, const TSharedPtr<ITableRow>& TableRow)
			{
				if (TSharedPtr<FAssetViewItem> AssetViewItem = GetAssetViewItemFromRow(Row))
				{
					return OnItemScrolledIntoViewDelegate.Execute(AssetViewItem, TableRow);
				}
			});
		}
		if (FWidgetDoubleClickedColumn* DoubleClickedColumn = DataStorage->GetColumn<FWidgetDoubleClickedColumn>(WidgetRow))
		{
			DoubleClickedColumn->OnMouseButtonDoubleClick = MakeShared<FWidgetDoubleClickedColumn::FOnMouseButtonDoubleClick>();
			DoubleClickedColumn->OnMouseButtonDoubleClick->BindLambda([this](FTedsRowHandle Row)
			{
				if (TSharedPtr<FAssetViewItem> AssetViewItem = GetAssetViewItemFromRow(Row))
				{
					return OnItemDoubleClickedDelegate.Execute(AssetViewItem);
				}
			});
		}
	}

	void FTedsContentBrowserViewExtender::CreateAssetPreview()
	{
		AssetPreviewWidget = SNew(STedsAssetPreviewWidget);

		BindAssetPreviewColumns();

		UpdateAssetPreviewTargetRow();
	}

	void FTedsContentBrowserViewExtender::BindAssetPreviewColumns()
	{
		if (!AssetPreviewWidget.IsValid())
		{
			return;
		}

		UE::Editor::DataStorage::RowHandle AssetPreviewRowHandle = AssetPreviewWidget->GetWidgetRowHandle();
		if (FWidgetContextMenuColumn* ContextMenuColumn = DataStorage->GetColumn<FWidgetContextMenuColumn>(AssetPreviewRowHandle))
		{
			ContextMenuColumn->OnContextMenuOpening = MakeShared<FOnContextMenuOpening>();
			ContextMenuColumn->OnContextMenuOpening->BindLambda([this]()
			{
				return OnContextMenuOpenedDelegate.Execute();
			});
		}
	}

	void FTedsContentBrowserViewExtender::UpdateAssetPreviewTargetRow() const
	{
		if (AssetPreviewWidget.IsValid())
		{
			TArray<FTedsRowHandle> SelectedItems;
			TableViewer->ForEachSelectedRow([this, &SelectedItems](RowHandle InRow)
			{
				SelectedItems.Add(FTedsRowHandle(InRow));
			});

			if (SelectedItems.Num())
			{
				AssetPreviewWidget->SetTargetRow(SelectedItems.Last());
			}
			else
			{
				// Nothing is selected so don't show anything
				AssetPreviewWidget->SetTargetRow(InvalidRowHandle);
			}

			// Reconstruct Teds Widgets for the AssetPreview since Purpose and TargetRow may change
			AssetPreviewWidget->ReconstructTedsWidget();
		}
	}

	float FTedsContentBrowserViewExtender::GetTileViewTypeNameHeight() const
	{
		if (CurrentThumbnailSize == EThumbnailSize::Tiny)
		{
			return 0.f;
		}
		return TedsTileViewTypeNameHeight;
	}

	float FTedsContentBrowserViewExtender::GetThumbnailSizeValue() const
	{
		return ThumbnailSizeValue;
	}

	EThumbnailSize FTedsContentBrowserViewExtender::GetThumbnailSize() const
	{
		return CurrentThumbnailSize;
	}

	float FTedsContentBrowserViewExtender::GetTileViewItemWidth() const
	{
		return GetThumbnailSizeValue() + TedsTileViewWidthPadding;
	}

	float FTedsContentBrowserViewExtender::GetTileViewItemHeight() const
	{
		return GetThumbnailSizeValue() + GetTileViewTypeNameHeight() + TedsTileViewHeightPadding;
	}

	float FTedsContentBrowserViewExtender::GetListViewItemHeight() const
	{
		return GetThumbnailSizeValue();
	}

	FMargin FTedsContentBrowserViewExtender::GetListViewItemPadding() const
	{
		return GetThumbnailSize() >= EThumbnailSize::Large? FMargin(0.f, TedsListViewHeightPadding) : FMargin(0.f);
	}

	void FTedsContentBrowserViewExtender::UpdateThumbnailSize() const
	{
		if (TableViewer.IsValid())
		{
			if (FThumbnailSizeColumn_Experimental* ThumbnailSizeColumn = DataStorage->GetColumn<FThumbnailSizeColumn_Experimental>(TableViewer->GetWidgetRowHandle()))
			{
				ThumbnailSizeColumn->ThumbnailSize = GetThumbnailSize();
			}
		}
	}

	void FTedsContentBrowserViewExtender::UpdateSizeValue() const
	{
		if (TableViewer.IsValid())
		{
			if (FSizeValueColumn_Experimental* SizeValueColumn = DataStorage->GetColumn<FSizeValueColumn_Experimental>(TableViewer->GetWidgetRowHandle()))
			{
				SizeValueColumn->SizeValue = GetThumbnailSizeValue();
			}
		}
	}

	void FTedsContentBrowserViewExtender::UpdateEditMode() const
	{
		if (TableViewer.IsValid())
		{
			if (FThumbnailEditModeColumn_Experimental* ContentBrowserSettingsColumn = DataStorage->GetColumn<FThumbnailEditModeColumn_Experimental>(TableViewer->GetWidgetRowHandle()))
			{
				ContentBrowserSettingsColumn->IsEditModeToggled = IsThumbnailEditMode;
			}
		}
	}

	FName FTestContentSource::GetName()
	{
		return FName("TestContentSource");
	}

	FText FTestContentSource::GetDisplayName()
	{
		return LOCTEXT("TestContentSource", "Outliner");
	}

	FSlateIcon FTestContentSource::GetIcon()
	{
		return FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Outliner");
	}

	void FTestContentSource::GetAssetViewInitParams(FTableViewerInitParams& OutInitParams)
	{
		using namespace UE::Editor::DataStorage::Queries;

		
		OutInitParams.QueryDescription = Select()
			.Where()
				.All<FTypedElementClassTypeInfoColumn>() // The test content source is simply looking at all rows with type info for now
			.Compile();

		// A few columns shown in the Outliner as a sample
		OutInitParams.Columns = { FTypedElementLabelColumn::StaticStruct(), FTypedElementClassTypeInfoColumn::StaticStruct(),
			FTedsAlertColumn::StaticStruct(), FTedsChildAlertColumn::StaticStruct() };

		// Same widget purposes as the Outliner for now
		
		OutInitParams.CellWidgetPurpose = IUiProvider::FPurposeID(IUiProvider::FPurposeInfo(
			"SceneOutliner", "Cell", NAME_None).GeneratePurposeID());

	}

	FTedsContentBrowserViewExtender::FTedsContentBrowserViewExtender()
	{
		using namespace UE::Editor::DataStorage;
		DataStorage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName);
		
		RowQueryStack = MakeShared<QueryStack::FRowArrayNode>();
	}

	TSharedRef<SWidget> FTedsContentBrowserViewExtender::CreateView(TArray<TSharedPtr<FAssetViewItem>>* InItemsSource)
	{
		switch (static_cast<ETableViewerMode>(TableViewerMode))
		{
		case ETableViewerMode::List:
			CreateListView();
			break;

		case ETableViewerMode::Tile:
			CreateTileView();
			break;

		default:
			CreateListView();
		}

		CreateAssetPreview();

		RefreshRows(InItemsSource);

		return SNew(SSplitter)
			.Orientation(Orient_Horizontal)

			+ SSplitter::Slot()
			.Resizable(true)
			[
				TableViewer->AsWidget()
			]

			+ SSplitter::Slot()
			.Resizable(true)
			[
				AssetPreviewWidget.ToSharedRef()
			];
	}
	
	void FTedsContentBrowserViewExtender::OnItemListChanged(TArray<TSharedPtr<FAssetViewItem>>* InItemsSource)
	{
		// CB 2.0 TODO: We might want to track individual addition/removals instead of a full refresh for perf
		RefreshRows(InItemsSource);
	}

	TArray<TSharedPtr<FAssetViewItem>> FTedsContentBrowserViewExtender::GetSelectedItems()
	{
		// CB 2.0 TODO: Figure out selection
		TArray<TSharedPtr<FAssetViewItem>> SelectedItems;

		TableViewer->ForEachSelectedRow([this, &SelectedItems](RowHandle Row)
		{
			if(TSharedPtr<FAssetViewItem> AssetViewItem = GetAssetViewItemFromRow(Row))
			{
				SelectedItems.Add(AssetViewItem);
			}
		});
		
		return SelectedItems;
	}

	IContentBrowserViewExtender::FOnSelectionChanged& FTedsContentBrowserViewExtender::OnSelectionChanged()
	{
		return OnSelectionChangedDelegate;
	}

	FOnContextMenuOpening& FTedsContentBrowserViewExtender::OnContextMenuOpened()
	{
		return OnContextMenuOpenedDelegate;
	}

	IContentBrowserViewExtender::FOnItemScrolledIntoView& FTedsContentBrowserViewExtender::OnItemScrolledIntoView()
	{
		return OnItemScrolledIntoViewDelegate;
	}

	IContentBrowserViewExtender::FOnMouseButtonClick& FTedsContentBrowserViewExtender::OnItemDoubleClicked()
	{
		return OnItemDoubleClickedDelegate;
	}

	FText FTedsContentBrowserViewExtender::GetViewDisplayName()
	{
		return LOCTEXT("TedsCBViewName", "TEDS Table View");
	}

	FText FTedsContentBrowserViewExtender::GetViewTooltipText()
	{
		return LOCTEXT("TedsCBViewTooltip", "A table viewer populated using TEDS UI and the asset registry data in TEDS");
	}

	void FTedsContentBrowserViewExtender::FocusList()
	{
		// CB 2.0 TODO: Do we need to focus the internal list? If so, implement using a Teds column
		FSlateApplication::Get().SetKeyboardFocus(TableViewer->AsWidget(), EFocusCause::SetDirectly);
	}

	void FTedsContentBrowserViewExtender::SetSelection(const TSharedPtr<FAssetViewItem>& Item, bool bSelected, const ESelectInfo::Type SelectInfo)
	{
		RowHandle Row = GetRowFromAssetViewItem(Item);

		if(DataStorage->IsRowAssigned(Row))
		{
			// We have to defer the selection by a tick because this fires on path change which has to refresh the internal list of assets.
			// The table viewer doesn't refresh immediately but rather on tick by checking if the query stack is dirty. If we set the selection
			// before the refresh happens SListView will deselect the item since it isn't visible in the list yet.
			// Long term selection should also be handled through TEDS so it happens at the proper time automatically.
			FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateLambda([this, Row, bSelected, SelectInfo](float)
			{
				TableViewer->SetSelection(Row, bSelected, SelectInfo);
				return false;
			}));
		}
	}

	void FTedsContentBrowserViewExtender::RequestScrollIntoView(const TSharedPtr<FAssetViewItem>& Item)
	{
		RowHandle Row = GetRowFromAssetViewItem(Item);

		if(DataStorage->IsRowAssigned(Row))
		{
			// We have to defer the scroll by a tick because this fires on path change which has to refresh the internal list of assets.
			// The table viewer doesn't refresh immediately but rather on tick by checking if the query stack is dirty. If we request scroll
			// before the refresh happens SListView will ignore the request since the item isn't visible in the list yet.
			FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateLambda([this, Row](float)
			{
				TableViewer->ScrollIntoView(Row);
				return false;
			}));
		}
	}

	void FTedsContentBrowserViewExtender::ClearSelection()
	{
		TableViewer->ClearSelection();
	}

	bool FTedsContentBrowserViewExtender::IsRightClickScrolling()
	{
		// CB 2.0 TODO: Implement using a Teds column
		return false;
	}
} // namespace UE::Editor::ContentBrowser

IMPLEMENT_MODULE(UE::Editor::ContentBrowser::FTedsContentBrowserModule, TedsContentBrowser);

#undef LOCTEXT_NAMESPACE