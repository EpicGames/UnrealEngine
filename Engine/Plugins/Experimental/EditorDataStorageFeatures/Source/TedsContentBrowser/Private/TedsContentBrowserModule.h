// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ContentSources/IContentSource.h"
#include "DataStorage/Handles.h"
#include "Elements/Framework/TypedElementRowHandleArray.h"
#include "Experimental/ContentBrowserViewExtender.h"
#include "Modules/ModuleInterface.h"
#include "Templates/SharedPointer.h"

namespace UE::Editor::DataStorage
{
	class FQueryStackNode_RowView;
	class ITableViewer;
	class ICoreProvider;

	namespace QueryStack
	{
		class FRowArrayNode;
	}
}

class FAssetViewItem;
class STedsAssetPreviewWidget;

namespace UE::Editor::ContentBrowser
{

	// A test content source that currently displays the query used by the Outliner
	class FTestContentSource : public IContentSource
	{
	public:
		virtual ~FTestContentSource() override = default;

		virtual FName GetName() override;
		virtual FText GetDisplayName() override;
		virtual FSlateIcon GetIcon() override;
		virtual void GetAssetViewInitParams(FTableViewerInitParams& OutInitParams) override;
	};
	
	// A custom view for the content browser that uses the TEDS asset registry integration to display rows with widgets using TEDS UI
	class FTedsContentBrowserViewExtender : public IContentBrowserViewExtender
	{
	public:

		FTedsContentBrowserViewExtender();

		// IContentBrowserViewExtender interface
		virtual TSharedRef<SWidget> CreateView(TArray<TSharedPtr<FAssetViewItem>>* InItemsSource) override;
		virtual TArray<TSharedPtr<FAssetViewItem>> GetSelectedItems() override;
		virtual FOnSelectionChanged& OnSelectionChanged() override;
		virtual FOnContextMenuOpening& OnContextMenuOpened() override;
		virtual FOnItemScrolledIntoView& OnItemScrolledIntoView() override;
		virtual FOnMouseButtonClick& OnItemDoubleClicked() override;
		virtual FText GetViewDisplayName() override;
		virtual FText GetViewTooltipText() override;
		virtual void FocusList() override;
		virtual void SetSelection(const TSharedPtr<FAssetViewItem>& Item, bool bSelected, const ESelectInfo::Type SelectInfo)override;
		virtual void RequestScrollIntoView(const TSharedPtr<FAssetViewItem>& Item) override;
		virtual void ClearSelection() override;
		virtual bool IsRightClickScrolling() override;
		virtual void OnItemListChanged(TArray<TSharedPtr<FAssetViewItem>>* InItemsSource) override;
		// ~IContentBrowserViewExtender interface

		// Refresh the rows in the current view by syncing the the items source
		void RefreshRows(TArray<TSharedPtr<FAssetViewItem>>* InItemsSource);

		// Add a single row to the table viewer
		void AddRow(const TSharedPtr<FAssetViewItem>& Item);

		// Get the internal FAssetViewItem from a row handle
		TSharedPtr<FAssetViewItem> GetAssetViewItemFromRow(DataStorage::RowHandle Row);

		DataStorage::RowHandle GetRowFromAssetViewItem(const TSharedPtr<FAssetViewItem>& Item);

	protected:
		// Update the table viewer used by the integration to use a list view
		void CreateListView();

		// Update the table viewer used by the integration to use a tile view
		void CreateTileView();

		// Bind the table viewer's columns to CB delegates
		void BindViewColumns();

		// Create the asset preview widget
		void CreateAssetPreview();

		// Bind the AssetPreview columns to CB delegates
		void BindAssetPreviewColumns();

		// Update the AssetPreview Target Row
		void UpdateAssetPreviewTargetRow() const;

	private:
		/** Get the name area total height */
		float GetTileViewTypeNameHeight() const;

		/** Get the thumbnail size value */
		float GetThumbnailSizeValue() const;

		/** Get the thumbnail size */
		EThumbnailSize GetThumbnailSize() const;

		/** Get the tile item width */
		float GetTileViewItemWidth() const;

		/** Get the tile item height */
		float GetTileViewItemHeight() const;

		/** Get the list item height */
		float GetListViewItemHeight() const;

		/** Get the list item padding */
		FMargin GetListViewItemPadding() const;

		/** Update the thumbnail size of the Tile/Column view and broadcast the event */
		void UpdateThumbnailSize() const;

		/** Update the size value of the Tile/Column view and broadcast the event */
		void UpdateSizeValue() const;

		/** Update the EditMode value of the Tile view and broadcast the event */
		void UpdateEditMode() const;

	private:
		/** AssetPreview widget */
		TSharedPtr<STedsAssetPreviewWidget> AssetPreviewWidget;

		/** Custom view type */
		ETableViewMode::Type CustomViewType = ETableViewMode::List;

		/** The current Thumbnail size */
		EThumbnailSize CurrentThumbnailSize = EThumbnailSize::Medium;

		/** The current Thumbnail size value */
		float ThumbnailSizeValue = 80.f;

		/** Whether the thumbnail edit mode is enabled */
		bool IsThumbnailEditMode = false;

		/** Name area total height */
		static constexpr float TedsTileViewTypeNameHeight = 67.f;

		/** Vertical padding for the TileViewItem */
		static constexpr int32 TedsTileViewHeightPadding = 9;

		/** Horizontal padding for the TileViewItem */
		static constexpr int32 TedsTileViewWidthPadding = 8;

		/** Vertical padding for the ListViewItem */
		static constexpr int32 TedsListViewHeightPadding = 2;

		// Ptr to the data storage interface
		DataStorage::ICoreProvider* DataStorage;
		
		// The actual table viewer widget
		TSharedPtr<DataStorage::ITableViewer> TableViewer;
		
		// Query stack used by the table viewer
		TSharedPtr<DataStorage::QueryStack::FRowArrayNode> RowQueryStack;

		// A map from row handle -> FAssetView item for lookups
		TMap<DataStorage::RowHandle, TWeakPtr<FAssetViewItem>> ContentBrowserItemMap;

		// Delegates fired when specific events happen on the list
		FOnSelectionChanged OnSelectionChangedDelegate;
		FOnContextMenuOpening OnContextMenuOpenedDelegate;
		FOnItemScrolledIntoView OnItemScrolledIntoViewDelegate;
		FOnMouseButtonClick OnItemDoubleClickedDelegate;
	};

	/**
	 * Implements the Teds Content Browser module.
	 */
	class FTedsContentBrowserModule
		: public IModuleInterface
	{
	public:

		FTedsContentBrowserModule() = default;
		
		static TSharedPtr<IContentBrowserViewExtender> CreateContentBrowserViewExtender();

		void RegisterTestContentSource();
		void UnregisterTestContentSource();

		// IModuleInterface interface
		virtual void StartupModule() override;
		virtual void ShutdownModule() override;
	};
} // namespace UE::Editor::ContentBrowser
