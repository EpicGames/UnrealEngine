// Copyright Epic Games, Inc. All Rights Reserved.

#include "ContentBrowserTileViewWidget.h"

#include "AssetThumbnail.h"
#include "AssetDefinition.h"
#include "Elements/Framework/TypedElementAttributeBinding.h"
#include "TedsAssetDataColumns.h"
#include "TedsAssetDataHelper.h"
#include "TedsAssetDataWidgetColumns.h"
#include "TedsTableViewerUtils.h"
#include "TedsTableViewerWidgetColumns.h"
#include "Columns/SlateDelegateColumns.h"
#include "Elements/Columns/TypedElementFolderColumns.h"
#include "Elements/Columns/TypedElementHiearchyColumns.h"
#include "Elements/Columns/TypedElementMiscColumns.h"
#include "Internationalization/BreakIterator.h"
#include "Styling/AppStyle.h"
#include "Styling/SlateBrush.h"
#include "Styling/StyleColors.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ContentBrowserTileViewWidget)

#define LOCTEXT_NAMESPACE "ContentBrowserTileViewWidget"

namespace Purpose
{
	static const FLazyName WidgetPurposeNamespace(TEXT("ContentBrowser"));
	static const FLazyName WidgetPurposeName(TEXT("TileLabel"));

	FName GetPurposeNamespace()
	{
		return WidgetPurposeNamespace;
	}

	FName GetPurposeName()
	{
		return WidgetPurposeName;
	}
}

void UContentBrowserTileViewWidgetFactory::RegisterWidgetPurposes(UE::Editor::DataStorage::IUiProvider& DataStorageUi) const
{
	using namespace UE::Editor::DataStorage::Queries;
	DataStorageUi.RegisterWidgetPurpose(
		IUiProvider::FPurposeInfo(Purpose::GetPurposeNamespace(), Purpose::GetPurposeName(), NAME_None,
		IUiProvider::EPurposeType::UniqueByNameAndColumn,
		LOCTEXT("ContentBrowserTileViewWidget_PurposeDescription", "Widget that display a Tile in the Content Browser")));
}

void UContentBrowserTileViewWidgetFactory::RegisterWidgetConstructors(UE::Editor::DataStorage::ICoreProvider& DataStorage,
	UE::Editor::DataStorage::IUiProvider& DataStorageUi) const
{
	using namespace UE::Editor::DataStorage::Queries;
	DataStorageUi.RegisterWidgetFactory<FContentBrowserTileViewWidgetConstructor>(
		DataStorageUi.FindPurpose(IUiProvider::FPurposeInfo(Purpose::GetPurposeNamespace(), Purpose::GetPurposeName(), NAME_None).GeneratePurposeID()), 
		TColumn<FAssetTag>() || TColumn<FFolderTag>());
}

FContentBrowserTileViewWidgetConstructor::FContentBrowserTileViewWidgetConstructor()
	: FSimpleWidgetConstructor(StaticStruct())
{
}

TSharedPtr<SWidget> FContentBrowserTileViewWidgetConstructor::CreateWidget(UE::Editor::DataStorage::ICoreProvider* DataStorage,
	UE::Editor::DataStorage::IUiProvider* DataStorageUi,
	UE::Editor::DataStorage::RowHandle TargetRow, UE::Editor::DataStorage::RowHandle WidgetRow,
	const UE::Editor::DataStorage::FMetaDataView& Arguments)
{
	using namespace UE::Editor::DataStorage;

	if (DataStorage->IsRowAvailable(TargetRow))
	{
		FAttributeBinder WidgetRowBinder(WidgetRow, DataStorage);
		FAttributeBinder RowBinder(TargetRow, DataStorage);

		const RowHandle ParentWidgetRowHandle = TableViewerUtils::GetTableViewerUiRow(DataStorage, WidgetRow);
		
		FAttributeBinder ParentWidgetRowBinder(ParentWidgetRowHandle, DataStorage);

		constexpr float BorderPadding = 1.f;
		constexpr float ShadowLeftTopPadding = 3.f;
		constexpr float ShadowRightBotPadding = 4.f;
		constexpr float ThumbnailBorderPadding = 0.f;
		constexpr float NameAreaBoxLeftRightBotPadding = 4.f;
		constexpr float NameAreaBoxTopPadding = 6.f;
		constexpr float ClassNameMaxHeight = 14.f;
		const FName ItemShadowBorderName = FName(TEXT("ContentBrowser.AssetTileItem.DropShadow"));
		bool bIsAsset = DataStorage->HasColumns<FAssetTag>(TargetRow);

		// Maybe divide them into Folder and Asset
		TSharedPtr<SBox> TileItem = SNew(SBox)
			.Padding(FMargin(BorderPadding, BorderPadding, 0.f, 1.f));

		FTedsTileViewCommonArguments TedsTileViewCommonArguments
		{
			.DataStorage = DataStorage,
			.DataStorageUi = DataStorageUi,
			.TargetRow = TargetRow,
			.WidgetRow = WidgetRow,
			.ParentWidgetRowHandle = ParentWidgetRowHandle,
			.bIsAsset = bIsAsset,
			.ThumbnailWidget = SNullWidget::NullWidget,
			.TileItem = TileItem
		};

		// Create the thumbnail widget through teds
		RowHandle OutThumbnailWidgetRowHandle;
		TSharedPtr<SWidget> ThumbnailWidget = CreateThumbnailWidget(TedsTileViewCommonArguments, OutThumbnailWidgetRowHandle);
		if (!ThumbnailWidget.IsValid())
		{
			ThumbnailWidget = SNullWidget::NullWidget;
		}
		else
		{
			FAttributeBinder ThumbnailWidgetBinder(OutThumbnailWidgetRowHandle, DataStorage);
			TileItem->SetToolTip(ThumbnailWidgetBinder.BindData(&FLocalWidgetTooltipColumn_Experimental::Tooltip));
		}

		// Create the item type widget through teds
		RowHandle OutItemTypeWidgetRowHandle;
		TSharedPtr<SWidget> ItemTypeWidget = CreateItemTypeWidget(TedsTileViewCommonArguments, OutItemTypeWidgetRowHandle);
		if (!ItemTypeWidget.IsValid())
		{
			ItemTypeWidget = SNullWidget::NullWidget;
		}

		TedsTileViewCommonArguments.ThumbnailWidget = ThumbnailWidget;

		TileItem->SetContent(
				SNew(SBorder)
				.Padding(FMargin(ShadowLeftTopPadding, ShadowLeftTopPadding, ShadowRightBotPadding, ShadowRightBotPadding))
				.BorderImage_Lambda([this, ItemShadowBorderName, TedsTileViewCommonArguments] ()
				{
					if (TedsTileViewCommonArguments.bIsAsset)
					{
						return FAppStyle::GetBrush(ItemShadowBorderName);
					}
					return GetFolderBackgroundShadowImage(TedsTileViewCommonArguments);
				})
				[
					SNew(SBorder)
					.Padding(ThumbnailBorderPadding)
					.BorderImage_Lambda([this, TedsTileViewCommonArguments] ()
						{
							if (TedsTileViewCommonArguments.bIsAsset)
							{
								return GetNameAreaBackgroundImage(TedsTileViewCommonArguments);
							}
							return GetFolderBackgroundImage(TedsTileViewCommonArguments);
						})
					[
						SNew(SVerticalBox)

						+ SVerticalBox::Slot()
						.AutoHeight()
						[
							SNew(SOverlay)

							//Thumbnail
							+ SOverlay::Slot()
							.Padding(bIsAsset ? 0.f : FMargin(2.f, 2.f, 2.f, 2.f))
							[
								SNew(SBorder)
								.Padding(0.f)
								.BorderImage_Lambda([this, TedsTileViewCommonArguments] (){ return GetFolderSlotBorder(TedsTileViewCommonArguments); })
								[
									ThumbnailWidget.ToSharedRef()
								]
							]

							// Overlay name for tiny folders
							+ SOverlay::Slot()
							.HAlign(HAlign_Center)
							.VAlign(VAlign_Center)
							.Padding(10.f, 0.f)
							[
								SNew(SBorder)
								.Padding(4.f, 2.f)
								.BorderImage(FAppStyle::GetBrush("ContentBrowser.AssetTileItem.TinyFolderTextBorder"))
								.Visibility(ParentWidgetRowBinder.BindData(&FThumbnailSizeColumn_Experimental::ThumbnailSize, [bIsAsset] (EThumbnailSize InThumbnailSize)
								{
									return !bIsAsset && InThumbnailSize == EThumbnailSize::Tiny ? EVisibility::Visible : EVisibility::Collapsed;
								}))
								[
									SNew(STextBlock)
									.ColorAndOpacity(FStyleColors::White)
									.OverflowPolicy(ETextOverflowPolicy::Ellipsis)
									.Text(RowBinder.BindData(&FAssetNameColumn::Name, [] (FName InName)
									{
										return FText::FromString(TedsAssetDataHelper::RemoveSlashFromStart(InName.ToString()));
									}))
								]
							]
						]

						+ SVerticalBox::Slot()
						.FillHeight(1.f)
						[
							SNew(SBox)
							.Padding(FMargin(NameAreaBoxLeftRightBotPadding, NameAreaBoxTopPadding, NameAreaBoxLeftRightBotPadding, NameAreaBoxLeftRightBotPadding))
							.Visibility(ParentWidgetRowBinder.BindData(&FThumbnailSizeColumn_Experimental::ThumbnailSize, [] (EThumbnailSize InThumbnailSize)
							{
								return InThumbnailSize == EThumbnailSize::Tiny ? EVisibility::Collapsed : EVisibility::Visible;
							}))
							[
								SNew(SVerticalBox)
								+ SVerticalBox::Slot()
								.HAlign(!bIsAsset ? HAlign_Center : HAlign_Fill)
								[
									SNew(SBox)
									.VAlign(VAlign_Top)
									.HeightOverride(GetNameAreaMaxDesiredHeight())
									[
										SNew(SInlineEditableTextBlock)
											.Font(GetThumbnailFont())
											.Text(RowBinder.BindData(&FAssetNameColumn::Name, [] (FName InName)
											{
												return FText::FromString(TedsAssetDataHelper::RemoveSlashFromStart(InName.ToString()));
											}))
											// TODO: Renaming logic for later, this should be synced between the actual AssetData and TEDS
											// .OnBeginTextEdit()
											// .OnTextCommitted()
											// .OnVerifyTextChanged()
											// .HighlightText(InArgs._HighlightText)
											.IsSelected(WidgetRowBinder.BindEvent(&FExternalWidgetExclusiveSelectionColumn::IsSelectedExclusively))
											// TODO: need to check if in EditMode if valid if temporary and if can be renamed, for now true by default need to be changed when the rename logic is ready
											.IsReadOnly(true)
											// TODO: CB TileView instead of setting this to true is updating this on the tick only when needed, see: SAssetViewItem::Tick
											.AutoWrapNonEditText(true)
											.LineBreakPolicy(FBreakIterator::CreateCamelCaseBreakIterator())
											.OverflowPolicy(ETextOverflowPolicy::MultilineEllipsis)
											.ColorAndOpacity_Lambda([this, TedsTileViewCommonArguments] () { return GetNameAreaTextColor(TedsTileViewCommonArguments); })
									]
								]

								+ SVerticalBox::Slot()
								.AutoHeight()
								[
									SNew(SBox)
									.HeightOverride(ClassNameMaxHeight)
									.VAlign(VAlign_Bottom)
									[
										ItemTypeWidget.ToSharedRef()
									]
								]
							]
						]
					]
				]
			);

		return TileItem;
	}

	return SNullWidget::NullWidget;;
}

TConstArrayView<const UScriptStruct*> FContentBrowserTileViewWidgetConstructor::GetAdditionalColumnsList() const
{
	static UE::Editor::DataStorage::TTypedElementColumnTypeList<FSizeValueColumn_Experimental, FThumbnailSizeColumn_Experimental> Columns;
	return Columns;
}

const FSlateBrush* FContentBrowserTileViewWidgetConstructor::GetFolderBackgroundShadowImage(FTedsTileViewCommonArguments InTedsTileViewCommonArguments) const
{
	if (!InTedsTileViewCommonArguments.DataStorage)
	{
		return FStyleDefaults::GetNoBrush();
	}

	UE::Editor::DataStorage::FAttributeBinder WidgetRowBinder(InTedsTileViewCommonArguments.WidgetRow, InTedsTileViewCommonArguments.DataStorage);

	FIsSelected IsSelected = WidgetRowBinder.BindEvent(&FExternalWidgetSelectionColumn::IsSelected);

	const bool bIsSelected = IsSelected.IsBound() ? IsSelected.Execute() : false;
	const bool bIsHoveredOrDraggedOver = (InTedsTileViewCommonArguments.TileItem.IsValid() && InTedsTileViewCommonArguments.TileItem->IsHovered()) /*|| bDraggedOver*/;

	if (bIsSelected || bIsHoveredOrDraggedOver)
	{
		static const FName DropShadowName("ContentBrowser.AssetTileItem.DropShadow");
		return FAppStyle::Get().GetBrush(DropShadowName);
	}

	return FStyleDefaults::GetNoBrush();
}

const FSlateBrush* FContentBrowserTileViewWidgetConstructor::GetFolderSlotBorder(FTedsTileViewCommonArguments InTedsTileViewCommonArguments) const
{
	if (!InTedsTileViewCommonArguments.DataStorage)
	{
		return FStyleDefaults::GetNoBrush();
	}

	UE::Editor::DataStorage::FAttributeBinder WidgetRowBinder(InTedsTileViewCommonArguments.WidgetRow, InTedsTileViewCommonArguments.DataStorage);

	FIsSelected IsSelected = WidgetRowBinder.BindEvent(&FExternalWidgetSelectionColumn::IsSelected);

	const bool bIsSelected = IsSelected.IsBound() ? IsSelected.Execute() : false;
	const bool bIsHoveredOrDraggedOver = (InTedsTileViewCommonArguments.TileItem.IsValid() && InTedsTileViewCommonArguments.TileItem->IsHovered()) /*|| bDraggedOver*/;

	if (bIsSelected || bIsHoveredOrDraggedOver)
	{
		static const FLazyName SelectedOrHovered("ContentBrowser.AssetTileItem.FolderAreaBackground"); // Panel
		return FAppStyle::Get().GetBrush(SelectedOrHovered);
	}

	return FStyleDefaults::GetNoBrush();
}

const FSlateBrush* FContentBrowserTileViewWidgetConstructor::GetNameAreaBackgroundImage(FTedsTileViewCommonArguments InTedsTileViewCommonArguments) const
{
	if (!InTedsTileViewCommonArguments.DataStorage)
	{
		return FStyleDefaults::GetNoBrush();
	}

	if (const FThumbnailSizeColumn_Experimental* ThumbnailSizeColumn = InTedsTileViewCommonArguments.DataStorage->GetColumn<FThumbnailSizeColumn_Experimental>(InTedsTileViewCommonArguments.ParentWidgetRowHandle))
	{
		if (ThumbnailSizeColumn->ThumbnailSize == EThumbnailSize::Tiny)
		{
			return FStyleDefaults::GetNoBrush();
		}
	}

	UE::Editor::DataStorage::FAttributeBinder WidgetRowBinder(InTedsTileViewCommonArguments.WidgetRow, InTedsTileViewCommonArguments.DataStorage);

	FIsSelected IsSelected = WidgetRowBinder.BindEvent(&FExternalWidgetSelectionColumn::IsSelected);

	const static FLazyName SelectedHover = "ContentBrowser.AssetTileItem.AssetContentSelectedHoverBackground";
	const static FLazyName Selected = "ContentBrowser.AssetTileItem.AssetContentSelectedBackground";
	const static FLazyName Hovered = "ContentBrowser.AssetTileItem.AssetContentHoverBackground";
	const static FLazyName Normal = "ContentBrowser.AssetTileItem.AssetContent";

	const bool bIsSelected = IsSelected.IsBound() ? IsSelected.Execute() : false;
	// TODO: ThumbnailWidget is the TedsWidget and not the actual Thumbnail widget, need to check the hovered state of the actual thumbnail widget, needed for the ThumbnailEditMode to keep it hovered
	const bool bIsHoveredOrDraggedOver = (InTedsTileViewCommonArguments.TileItem.IsValid() && InTedsTileViewCommonArguments.TileItem->IsHovered()) /*|| bDraggedOver || (InThumbnailWidget.IsValid() && InThumbnailWidget->IsHovered())*/;
	if (bIsSelected && bIsHoveredOrDraggedOver)
	{
		return FAppStyle::Get().GetBrush(SelectedHover);
	}
	else if (bIsSelected)
	{
		return FAppStyle::Get().GetBrush(Selected);
	}
	else if (bIsHoveredOrDraggedOver && InTedsTileViewCommonArguments.bIsAsset)
	{
		return FAppStyle::Get().GetBrush(Hovered);
	}
	else if (InTedsTileViewCommonArguments.bIsAsset)
	{
		return FAppStyle::Get().GetBrush(Normal);
	}
	return FStyleDefaults::GetNoBrush();
}

FSlateFontInfo FContentBrowserTileViewWidgetConstructor::GetThumbnailFont() const
{
	const static FLazyName RegularFont("ContentBrowser.AssetTileViewNameFont");
    return FAppStyle::GetFontStyle(RegularFont);
}

FSlateColor FContentBrowserTileViewWidgetConstructor::GetNameAreaTextColor(FTedsTileViewCommonArguments InTedsTileViewCommonArguments) const
{
	if (!InTedsTileViewCommonArguments.DataStorage)
	{
		return FSlateColor::UseForeground();
	}

	UE::Editor::DataStorage::FAttributeBinder WidgetRowBinder(InTedsTileViewCommonArguments.WidgetRow, InTedsTileViewCommonArguments.DataStorage);

	FIsSelected IsSelected = WidgetRowBinder.BindEvent(&FExternalWidgetSelectionColumn::IsSelected);

	const bool bIsSelected = IsSelected.IsBound() ? IsSelected.Execute() : false;
	// TODO: ThumbnailWidget is the TedsWidget and not the actual Thumbnail widget, need to check the hovered state of the actual thumbnail widget, needed for the ThumbnailEditMode to keep it hovered
	const bool bIsHoveredOrDraggedOver = (InTedsTileViewCommonArguments.TileItem.IsValid() && InTedsTileViewCommonArguments.TileItem->IsHovered()) /*|| bDraggedOver || (InThumbnailWidget.IsValid() && InThumbnailWidget->IsHovered())*/;
	if (bIsSelected || bIsHoveredOrDraggedOver)
	{
		return FStyleColors::White;
	}

	return FSlateColor::UseForeground();
}

const FSlateBrush* FContentBrowserTileViewWidgetConstructor::GetAssetThumbnailBorderOverride(FTedsTileViewCommonArguments InTedsTileViewCommonArguments) const
{
	if (!InTedsTileViewCommonArguments.DataStorage)
	{
		return FAppStyle::GetNoBrush();
	}

	UE::Editor::DataStorage::FAttributeBinder WidgetRowBinder(InTedsTileViewCommonArguments.WidgetRow, InTedsTileViewCommonArguments.DataStorage);

	FIsSelected IsSelected = WidgetRowBinder.BindEvent(&FExternalWidgetSelectionColumn::IsSelected);

	const bool bIsSelected = IsSelected.IsBound() ? IsSelected.Execute() : false;
	// TODO: ThumbnailWidget is the TedsWidget and not the actual Thumbnail widget, need to check the hovered state of the actual thumbnail widget, needed for the ThumbnailEditMode to keep it hovered
	const bool bIsHoveredOrDraggedOver = (InTedsTileViewCommonArguments.TileItem.IsValid() && InTedsTileViewCommonArguments.TileItem->IsHovered()) /*|| bDraggedOver || (InThumbnailWidget.IsValid() && InThumbnailWidget->IsHovered())*/;
	if (bIsSelected && bIsHoveredOrDraggedOver)
	{
		static const FLazyName SelectedHover("ContentBrowser.AssetTileItem.AssetBorderSelectedHoverBackground");
		return FAppStyle::Get().GetBrush(SelectedHover);
	}
	else if (bIsSelected)
	{
		static const FLazyName Selected("ContentBrowser.AssetTileItem.AssetBorderSelectedBackground");
		return FAppStyle::Get().GetBrush(Selected);
	}
	else if (bIsHoveredOrDraggedOver && InTedsTileViewCommonArguments.bIsAsset)
	{
		static const FLazyName Hovered("ContentBrowser.AssetTileItem.AssetBorderHoverBackground");
		return FAppStyle::Get().GetBrush(Hovered);
	}
	else if (InTedsTileViewCommonArguments.bIsAsset)
	{
		static const FLazyName Normal("AssetThumbnail.AssetBorder");
		return FAppStyle::Get().GetBrush(Normal);
	}

	return FStyleDefaults::GetNoBrush();
}

FOptionalSize FContentBrowserTileViewWidgetConstructor::GetNameAreaMaxDesiredHeight() const
{
	constexpr int32 MaxHeightNameArea = 42;
	return MaxHeightNameArea;
}

EVisibility FContentBrowserTileViewWidgetConstructor::GetAssetClassLabelVisibility(bool bIsAsset) const
{
	return !bIsAsset ? EVisibility::Collapsed : EVisibility::Visible;
}

FSlateColor FContentBrowserTileViewWidgetConstructor::GetAssetClassLabelTextColor(FTedsTileViewCommonArguments InTedsTileViewCommonArguments) const
{
	if (!InTedsTileViewCommonArguments.DataStorage)
	{
		return FStyleColors::Hover2;
	}

	UE::Editor::DataStorage::FAttributeBinder WidgetRowBinder(InTedsTileViewCommonArguments.WidgetRow, InTedsTileViewCommonArguments.DataStorage);

	FIsSelected IsSelected = WidgetRowBinder.BindEvent(&FExternalWidgetSelectionColumn::IsSelected);

	const bool bIsSelected = IsSelected.IsBound() ? IsSelected.Execute() : false;
	// TODO: ThumbnailWidget is the TedsWidget and not the actual Thumbnail widget, need to check the hovered state of the actual thumbnail widget, needed for the ThumbnailEditMode to keep it hovered
	const bool bIsHoveredOrDraggedOver = (InTedsTileViewCommonArguments.TileItem.IsValid() && InTedsTileViewCommonArguments.TileItem->IsHovered()) /*|| bDraggedOver || (InThumbnailWidget.IsValid() && InThumbnailWidget->IsHovered())*/;
	if (bIsSelected || bIsHoveredOrDraggedOver)
	{
		return FStyleColors::White;
	}

	return FStyleColors::Hover2;
}

const FSlateBrush* FContentBrowserTileViewWidgetConstructor::GetFolderBackgroundImage(FTedsTileViewCommonArguments InTedsTileViewCommonArguments) const
{
	if (!InTedsTileViewCommonArguments.DataStorage)
	{
		return FStyleDefaults::GetNoBrush();
	}

	UE::Editor::DataStorage::FAttributeBinder WidgetRowBinder(InTedsTileViewCommonArguments.WidgetRow, InTedsTileViewCommonArguments.DataStorage);

	FIsSelected IsSelected = WidgetRowBinder.BindEvent(&FExternalWidgetSelectionColumn::IsSelected);

	const bool bIsSelected = IsSelected.IsBound() ? IsSelected.Execute() : false;
	const bool bIsHoveredOrDraggedOver = (InTedsTileViewCommonArguments.TileItem.IsValid() && InTedsTileViewCommonArguments.TileItem->IsHovered()) /*|| bDraggedOver*/;

	if (bIsSelected && bIsHoveredOrDraggedOver)
	{
		static const FName SelectedHoverBackground("ContentBrowser.AssetTileItem.FolderAreaSelectedHoverBackground");
		return FAppStyle::Get().GetBrush(SelectedHoverBackground);
	}
	else if (bIsSelected)
	{
		static const FName SelectedBackground("ContentBrowser.AssetTileItem.FolderAreaSelectedBackground");
		return FAppStyle::Get().GetBrush(SelectedBackground);
	}
	else if (bIsHoveredOrDraggedOver)
	{
		static const FName HoveredBackground("ContentBrowser.AssetTileItem.FolderAreaHoveredBackground");
		return FAppStyle::Get().GetBrush(HoveredBackground);
	}

	return FStyleDefaults::GetNoBrush();
}

TSharedPtr<SWidget> FContentBrowserTileViewWidgetConstructor::CreateThumbnailWidget(FTedsTileViewCommonArguments InTedsTileViewCommonArguments, UE::Editor::DataStorage::RowHandle& OutThumbnailWidgetRowHandle) const
{
	using namespace UE::Editor::DataStorage;

	IUiProvider::FWidgetConstructorPtr OutThumbnailWidgetConstructorPtr;

	auto AssignThumbnailWidgetToColumn = [&OutThumbnailWidgetConstructorPtr] (IUiProvider::FWidgetConstructorPtr WidgetConstructor, TConstArrayView<TWeakObjectPtr<const UScriptStruct>>)
	{
		OutThumbnailWidgetConstructorPtr = WidgetConstructor;
		return false;
	};
	
	UE::Editor::DataStorage::FMetaData ThumbnailMeta;
	ThumbnailMeta.AddOrSetMutableData(TedsAssetDataHelper::MetaDataNames::GetThumbnailStatusMetaDataName(), true);
	ThumbnailMeta.AddOrSetMutableData(TedsAssetDataHelper::MetaDataNames::GetThumbnailFadeInMetaDataName(), true);
	ThumbnailMeta.AddOrSetMutableData(TedsAssetDataHelper::MetaDataNames::GetThumbnailHintTextMetaDataName(), false);
	// TODO: use our own OnMouseEnter/Leave for setting the realtime flag, also check CB Setting
	ThumbnailMeta.AddOrSetMutableData(TedsAssetDataHelper::MetaDataNames::GetThumbnailRealTimeOnHoveredMetaDataName(), false);

	// Folder has more padding since this widget has to emulate the border that AssetThumbnails has
	if (!InTedsTileViewCommonArguments.bIsAsset)
	{
		ThumbnailMeta.AddOrSetMutableData(TedsAssetDataHelper::MetaDataNames::GetThumbnailSizeOffsetMetaDataName(), -4.f);
	}
	const FGenericMetaDataView ThumbnailMetaView = FGenericMetaDataView(ThumbnailMeta);

	// List all columns on the row so the thumbnail widget can be override based on longest match on the current row
	TArray<TWeakObjectPtr<const UScriptStruct>> ThumbnailColumns;
	InTedsTileViewCommonArguments.DataStorage->ListColumns(InTedsTileViewCommonArguments.TargetRow, [&ThumbnailColumns](const UScriptStruct& ColumnType)
		{
			ThumbnailColumns.Emplace(&ColumnType);
			return true;
		});
	const IUiProvider::FPurposeID ThumbnailPurposeID = IUiProvider::FPurposeInfo("ContentBrowser", "Thumbnail", NAME_None).GeneratePurposeID();
	const RowHandle ThumbnailPurposeRowHandle = InTedsTileViewCommonArguments.DataStorageUi->FindPurpose(ThumbnailPurposeID);
	
	InTedsTileViewCommonArguments.DataStorageUi->CreateWidgetConstructors(ThumbnailPurposeRowHandle, IUiProvider::EMatchApproach::LongestMatch,
		ThumbnailColumns, ThumbnailMetaView, AssignThumbnailWidgetToColumn);

	TSharedPtr<SWidget> ThumbnailWidget = SNullWidget::NullWidget;
	if (OutThumbnailWidgetConstructorPtr)
	{
		OutThumbnailWidgetRowHandle = InTedsTileViewCommonArguments.DataStorage->AddRow(InTedsTileViewCommonArguments.DataStorage->FindTable(TedsAssetDataHelper::TableView::GetWidgetTableName()));
		if (OutThumbnailWidgetRowHandle != InvalidRowHandle)
		{
			// Referenced Data Row
			InTedsTileViewCommonArguments.DataStorage->AddColumn(OutThumbnailWidgetRowHandle, FTypedElementRowReferenceColumn{ .Row = InTedsTileViewCommonArguments.TargetRow });

			// Parent widget row
			InTedsTileViewCommonArguments.DataStorage->AddColumn(OutThumbnailWidgetRowHandle, FTableRowParentColumn{ .Parent = InTedsTileViewCommonArguments.ParentWidgetRowHandle });

			// Padding
			InTedsTileViewCommonArguments.DataStorage->AddColumn(OutThumbnailWidgetRowHandle, FWidgetPaddingColumn_Experimental{ .Padding = InTedsTileViewCommonArguments.bIsAsset ? FMargin(0.f) : FMargin(5.f) });

			// TODO: this need to be updated when the widget is Selected/Hovered/Dragged or keep it the same if the ThumbnailWidget is kept hovered
			// Used to override the Thumbnail border image
			InTedsTileViewCommonArguments.DataStorage->AddColumn(OutThumbnailWidgetRowHandle, FOnGetWidgetSlateBrushColumn_Experimental
			{
				.OnGetWidgetSlateBrush = MakeShared<FOnGetWidgetSlateBrush>(FOnGetWidgetSlateBrush::CreateLambda([this, InTedsTileViewCommonArguments] () { return GetAssetThumbnailBorderOverride(InTedsTileViewCommonArguments); }))
			});

			// Used to retrieve the Thumbnail tooltip to use on the whole TileItem
			InTedsTileViewCommonArguments.DataStorage->AddColumn(OutThumbnailWidgetRowHandle, FLocalWidgetTooltipColumn_Experimental::StaticStruct());

			// Used to decide on the actual Thumbnail size
			if (FSizeValueColumn_Experimental* TileItemSizeColumn = InTedsTileViewCommonArguments.DataStorage->GetColumn<FSizeValueColumn_Experimental>(InTedsTileViewCommonArguments.WidgetRow))
			{
				InTedsTileViewCommonArguments.DataStorage->AddColumn(OutThumbnailWidgetRowHandle, FSizeValueColumn_Experimental{ .SizeValue = TileItemSizeColumn->SizeValue });
			}

			ThumbnailWidget = InTedsTileViewCommonArguments.DataStorageUi->ConstructWidget(OutThumbnailWidgetRowHandle, *OutThumbnailWidgetConstructorPtr, ThumbnailMetaView);
		}
	}

	return ThumbnailWidget;
}

TSharedPtr<SWidget> FContentBrowserTileViewWidgetConstructor::CreateItemTypeWidget(FTedsTileViewCommonArguments InTedsTileViewCommonArguments, UE::Editor::DataStorage::RowHandle& OutItemTypeWidgetRowHandle) const
{
	using namespace UE::Editor::DataStorage;

	IUiProvider::FWidgetConstructorPtr OutItemTypeWidgetConstructorPtr;

	auto AssignItemTypeWidgetToColumn = [&OutItemTypeWidgetConstructorPtr] (IUiProvider::FWidgetConstructorPtr WidgetConstructor, TConstArrayView<TWeakObjectPtr<const UScriptStruct>>)
	{
		OutItemTypeWidgetConstructorPtr = MoveTemp(WidgetConstructor);;
		return false;
	};

	TArray<TWeakObjectPtr<const UScriptStruct>> ItemTypeColumns = GetItemTypeColumns();

	const RowHandle DefaultPurposeRowHandle = InTedsTileViewCommonArguments.DataStorageUi->FindPurpose(InTedsTileViewCommonArguments.DataStorageUi->GetGeneralWidgetPurposeID());
	InTedsTileViewCommonArguments.DataStorageUi->CreateWidgetConstructors(DefaultPurposeRowHandle, IUiProvider::EMatchApproach::ExactMatch, ItemTypeColumns, FMetaDataView(), AssignItemTypeWidgetToColumn);

	TSharedPtr<SWidget> ItemTypeWidget = SNullWidget::NullWidget;
	if (OutItemTypeWidgetConstructorPtr)
	{
		OutItemTypeWidgetRowHandle = InTedsTileViewCommonArguments.DataStorage->AddRow(InTedsTileViewCommonArguments.DataStorage->FindTable(TedsAssetDataHelper::TableView::GetWidgetTableName()));
		if (OutItemTypeWidgetRowHandle != InvalidRowHandle)
		{
			// Referenced Data Row
			InTedsTileViewCommonArguments.DataStorage->AddColumn(OutItemTypeWidgetRowHandle, FTypedElementRowReferenceColumn{ .Row = InTedsTileViewCommonArguments.TargetRow });

			// Adding the FontStyle to use
			InTedsTileViewCommonArguments.DataStorage->AddColumn(OutItemTypeWidgetRowHandle, FFontStyleColumn_Experimental{ .FontInfo = FAppStyle::GetFontStyle("ContentBrowser.AssetTileViewClassNameFont") });

			// Adding the visibility to use
			InTedsTileViewCommonArguments.DataStorage->AddColumn(OutItemTypeWidgetRowHandle, FWidgetVisibilityColumn_Experimental{ .Visibility = GetAssetClassLabelVisibility(InTedsTileViewCommonArguments.bIsAsset) });

			// Adding the OverflowPolicy to use
			InTedsTileViewCommonArguments.DataStorage->AddColumn(OutItemTypeWidgetRowHandle, FTextOverflowPolicyColumn_Experimental{ .OverflowPolicy = ETextOverflowPolicy::Ellipsis });

			// TODO: this need to be updated when the widget is Selected/Hovered/Dragged or keep it the same if the ThumbnailWidget is kept hovered
			// Adding the ColorAndOpacity to use
			InTedsTileViewCommonArguments.DataStorage->AddColumn(OutItemTypeWidgetRowHandle, FOnGetWidgetColorAndOpacityColumn_Experimental
				{
					.OnGetWidgetColorAndOpacity = MakeShared<FOnGetWidgetColorAndOpacity>(FOnGetWidgetColorAndOpacity::CreateLambda([this, InTedsTileViewCommonArguments] { return GetAssetClassLabelTextColor(InTedsTileViewCommonArguments); }))
				});

			ItemTypeWidget = InTedsTileViewCommonArguments.DataStorageUi->ConstructWidget(OutItemTypeWidgetRowHandle, *OutItemTypeWidgetConstructorPtr, FMetaDataView());
		}
	}
	return ItemTypeWidget;
}

TArray<TWeakObjectPtr<const UScriptStruct>> FContentBrowserTileViewWidgetConstructor::GetItemTypeColumns() const
{
	static TArray<TWeakObjectPtr<const UScriptStruct>> ItemTypeColumns({ TWeakObjectPtr(FAssetClassColumn::StaticStruct()), TWeakObjectPtr(FFolderTag::StaticStruct()) });
	return ItemTypeColumns;
}

#undef LOCTEXT_NAMESPACE
