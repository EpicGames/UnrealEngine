// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/Interfaces/TypedElementDataStorageFactory.h"
#include "Elements/Interfaces/TypedElementDataStorageUiInterface.h"
#include "Fonts/SlateFontInfo.h"
#include "Internationalization/Text.h"
#include "Layout/Visibility.h"
#include "Styling/SlateColor.h"
#include "Types/SlateStructs.h"
#include "UObject/ObjectMacros.h"

#include "ContentBrowserTileViewWidget.generated.h"

class IEditorDataStorageProvider;
class UScriptStruct;
struct FSlateBrush;

namespace Purpose
{
	FName GetPurposeNamespace();
	FName GetPurposeName();
}

UCLASS()
class UContentBrowserTileViewWidgetFactory : public UEditorDataStorageFactory
{
	GENERATED_BODY()

public:
	~UContentBrowserTileViewWidgetFactory() override = default;

	TEDSCONTENTBROWSER_API virtual void RegisterWidgetPurposes(UE::Editor::DataStorage::IUiProvider& DataStorageUi) const override;

	TEDSCONTENTBROWSER_API void RegisterWidgetConstructors(UE::Editor::DataStorage::ICoreProvider& DataStorage,
		UE::Editor::DataStorage::IUiProvider& DataStorageUi) const override;
};

// Content Browser Label + Thumbnail widget
USTRUCT()
struct FContentBrowserTileViewWidgetConstructor : public FSimpleWidgetConstructor
{
	GENERATED_BODY()

public:
	TEDSCONTENTBROWSER_API FContentBrowserTileViewWidgetConstructor();

	virtual TSharedPtr<SWidget> CreateWidget(
		UE::Editor::DataStorage::ICoreProvider* DataStorage,
		UE::Editor::DataStorage::IUiProvider* DataStorageUi,
		UE::Editor::DataStorage::RowHandle TargetRow,
		UE::Editor::DataStorage::RowHandle WidgetRow, 
		const UE::Editor::DataStorage::FMetaDataView& Arguments) override;

	virtual TConstArrayView<const UScriptStruct*> GetAdditionalColumnsList() const override;

private:

	/** Common data needed for various function */
	struct FTedsTileViewCommonArguments
	{
		UE::Editor::DataStorage::ICoreProvider* DataStorage;
		UE::Editor::DataStorage::IUiProvider* DataStorageUi;
		UE::Editor::DataStorage::RowHandle TargetRow;
		UE::Editor::DataStorage::RowHandle WidgetRow;
		UE::Editor::DataStorage::RowHandle ParentWidgetRowHandle;
		bool bIsAsset;
		TSharedPtr<SWidget> ThumbnailWidget;
		TSharedPtr<SWidget> TileItem;
	};

	/** Get the folder shadow image */
	const FSlateBrush* GetFolderBackgroundShadowImage(FTedsTileViewCommonArguments InTedsTileViewCommonArguments) const;

	/** Get the folder image slot border */
	const FSlateBrush* GetFolderSlotBorder(FTedsTileViewCommonArguments InTedsTileViewCommonArguments) const;

	/** Get the asset thumbnail border to use */
	const FSlateBrush* GetAssetThumbnailBorderOverride(FTedsTileViewCommonArguments InTedsTileViewCommonArguments) const;

	/** Get the name are background image */
	const FSlateBrush* GetNameAreaBackgroundImage(FTedsTileViewCommonArguments InTedsTileViewCommonArguments) const;

	/** Get the font to use for the thumbnail label */
	FSlateFontInfo GetThumbnailFont() const;

	/** Get the name area text color */
	FSlateColor GetNameAreaTextColor(FTedsTileViewCommonArguments InTedsTileViewCommonArguments) const;

	/** Get the maximum height for the name area */
	FOptionalSize GetNameAreaMaxDesiredHeight() const;

	/** Gets the visibility of the asset class label in thumbnails */
	EVisibility GetAssetClassLabelVisibility(bool bIsAsset) const;

	/** Gets the color and opacity of the asset class type */
	FSlateColor GetAssetClassLabelTextColor(FTedsTileViewCommonArguments InTedsTileViewCommonArguments) const;

	/** Gets the background image for folders */
	const FSlateBrush* GetFolderBackgroundImage(FTedsTileViewCommonArguments InTedsTileViewCommonArguments) const;

	/** Create the thumbnail widget through TEDS */
	TSharedPtr<SWidget> CreateThumbnailWidget(FTedsTileViewCommonArguments InTedsTileViewCommonArguments, UE::Editor::DataStorage::RowHandle& OutThumbnailWidgetRowHandle) const;

	/** Create the item type widget through TEDS */
	TSharedPtr<SWidget> CreateItemTypeWidget(FTedsTileViewCommonArguments InTedsTileViewCommonArguments, UE::Editor::DataStorage::RowHandle& OutItemTypeWidgetRowHandle) const;

	/** Get the ItemType columns */
	TArray<TWeakObjectPtr<const UScriptStruct>> GetItemTypeColumns() const;
};
