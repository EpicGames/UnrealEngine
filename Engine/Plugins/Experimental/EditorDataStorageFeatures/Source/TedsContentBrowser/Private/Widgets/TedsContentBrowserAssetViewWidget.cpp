// Copyright Epic Games, Inc. All Rights Reserved.

#include "TedsContentBrowserAssetViewWidget.h"

#include "TedsQueryNode.h"
#include "TedsRowQueryResultsNode.h"
#include "TedsRowViewNode.h"
#include "ContentSources/IContentSource.h"
#include "ContentSources/Columns/ContentSourcesColumns.h"
#include "Elements/Columns/TypedElementLabelColumns.h"
#include "Elements/Columns/TypedElementTypeInfoColumns.h"
#include "Widgets/STedsTableViewer.h"
#include "Widgets/STedsTileViewer.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TedsContentBrowserAssetViewWidget)

#define LOCTEXT_NAMESPACE "ContentBrowserAssetViewWidget"

// Wrapper widget around the table viewer so we can manage the lifetime of the query and query stack manually for now
class STableViewerWrapper : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(STableViewerWrapper)
		: _InitParams()
		{}
		SLATE_ARGUMENT(UE::Editor::ContentBrowser::FTableViewerInitParams, InitParams)
		SLATE_ARGUMENT(UE::Editor::DataStorage::ICoreProvider*, Storage)
	SLATE_END_ARGS()

	void Construct(const FArguments& Arguments)
	{
		using namespace UE::Editor::DataStorage;
		Storage = Arguments._Storage;

		TSharedRef<QueryStack::FQueryNode> QueryNode = MakeShared<QueryStack::FQueryNode>(*Storage, Arguments._InitParams.QueryDescription);

		// For now we just refresh the query on update every frame since this is a prototype, in the future we can use a monitor node if perf is a concern
		RowView = MakeShared<QueryStack::FRowQueryResultsNode>(*Storage, QueryNode, QueryStack::FRowQueryResultsNode::ESyncActions::ForceRefreshOnUpdate);
		
		TSharedPtr<SWidget> ChildWidget;

		switch (Arguments._InitParams.TableViewMode)
		{
		case ETableViewMode::Type::List:
			ChildWidget = SNew(UE::Editor::DataStorage::STedsTableViewer)
					.QueryStack(RowView)
					.CellWidgetPurpose(Arguments._InitParams.CellWidgetPurpose)
					.Columns(Arguments._InitParams.Columns)
					.ListSelectionMode(ESelectionMode::Multi);
			break;
		case ETableViewMode::Type::Tile:
			ChildWidget = SNew(UE::Editor::DataStorage::STedsTileViewer)
					.QueryStack(RowView)
					.WidgetPurpose(Arguments._InitParams.CellWidgetPurpose)
					.Columns(Arguments._InitParams.Columns)
					.SelectionMode(ESelectionMode::Multi);
			break;
		case ETableViewMode::Type::Tree:

		default:
			ChildWidget = SNullWidget::NullWidget;
		}

		ChildSlot
		[
			ChildWidget.ToSharedRef()
		];
	}

	virtual ~STableViewerWrapper() override
	{
		Storage->UnregisterQuery(QueryHandle);
	}

private:
	UE::Editor::DataStorage::QueryHandle QueryHandle = UE::Editor::DataStorage::InvalidQueryHandle;
	TArray<UE::Editor::DataStorage::RowHandle> Rows;
	TSet<UE::Editor::DataStorage::RowHandle> Rows_Set;
	TSharedPtr<UE::Editor::DataStorage::QueryStack::FRowQueryResultsNode> RowView;
	UE::Editor::DataStorage::ICoreProvider* Storage = nullptr;
	
};

namespace TedsContentBrowser::Private
{
	static const UE::Editor::DataStorage::IUiProvider::FPurposeID
		Purpose(UE::Editor::DataStorage::IUiProvider::FPurposeInfo("ContentBrowser", "AssetView", NAME_None).GeneratePurposeID());

}

void UContentBrowserAssetViewWidgetFactory::RegisterWidgetPurposes(UE::Editor::DataStorage::IUiProvider& DataStorageUi) const
{
	DataStorageUi.RegisterWidgetPurpose(
		UE::Editor::DataStorage::IUiProvider::FPurposeInfo("ContentBrowser", "AssetView", NAME_None,
			UE::Editor::DataStorage::IUiProvider::EPurposeType::UniqueByName,
			LOCTEXT("ContentBrowserAssetView_PurposeDescription", "Widget that displays a table viewer in the content browser")));
}

void UContentBrowserAssetViewWidgetFactory::RegisterWidgetConstructors(UE::Editor::DataStorage::ICoreProvider& DataStorage,
	UE::Editor::DataStorage::IUiProvider& DataStorageUi) const
{
	using namespace UE::Editor::DataStorage::Queries;
	DataStorageUi.RegisterWidgetFactory<FContentBrowserAssetViewWidgetConstructor>(DataStorageUi.FindPurpose(TedsContentBrowser::Private::Purpose));
}

FContentBrowserAssetViewWidgetConstructor::FContentBrowserAssetViewWidgetConstructor()
	: FSimpleWidgetConstructor(StaticStruct())
{
}

FContentBrowserAssetViewWidgetConstructor::FContentBrowserAssetViewWidgetConstructor(const UScriptStruct* TypeInfo)
	: FSimpleWidgetConstructor(TypeInfo)
{
}

TSharedPtr<SWidget> FContentBrowserAssetViewWidgetConstructor::CreateWidget(UE::Editor::DataStorage::ICoreProvider* DataStorage,
	UE::Editor::DataStorage::IUiProvider* DataStorageUi, UE::Editor::DataStorage::RowHandle TargetRow, UE::Editor::DataStorage::RowHandle WidgetRow,
	const UE::Editor::DataStorage::FMetaDataView& Arguments)
{
	using namespace UE::Editor::DataStorage::Queries;

	if (FContentSourceColumn* ContentSourceColumn = DataStorage->GetColumn<FContentSourceColumn>(WidgetRow))
	{
		if (TSharedPtr<UE::Editor::ContentBrowser::IContentSource> ContentSourcePtr = ContentSourceColumn->ContentSource.Pin())
		{
			UE::Editor::ContentBrowser::FTableViewerInitParams InitParams;
			ContentSourcePtr->GetAssetViewInitParams(InitParams);
			
			return SNew(STableViewerWrapper)
			.InitParams(InitParams)
			.Storage(DataStorage);
		}
		
	}

	return SNullWidget::NullWidget;
}

#undef LOCTEXT_NAMESPACE
