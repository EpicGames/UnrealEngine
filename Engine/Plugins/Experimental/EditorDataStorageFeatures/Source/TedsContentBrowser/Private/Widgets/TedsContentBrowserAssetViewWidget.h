// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/Interfaces/TypedElementDataStorageFactory.h"
#include "Elements/Interfaces/TypedElementDataStorageUiInterface.h"
#include "Internationalization/Text.h"
#include "UObject/ObjectMacros.h"

#include "TedsContentBrowserAssetViewWidget.generated.h"

class UScriptStruct;

UCLASS()
class UContentBrowserAssetViewWidgetFactory : public UEditorDataStorageFactory
{
	GENERATED_BODY()

public:
	~UContentBrowserAssetViewWidgetFactory() override = default;

	TEDSCONTENTBROWSER_API virtual void RegisterWidgetPurposes(UE::Editor::DataStorage::IUiProvider& DataStorageUi) const override;

	TEDSCONTENTBROWSER_API void RegisterWidgetConstructors(UE::Editor::DataStorage::ICoreProvider& DataStorage,
		UE::Editor::DataStorage::IUiProvider& DataStorageUi) const override;
};

// Default asset view widget shown by a content source - the query and other init params are populated by the content source itself
USTRUCT()
struct FContentBrowserAssetViewWidgetConstructor : public FSimpleWidgetConstructor
{
	GENERATED_BODY()

public:
	TEDSCONTENTBROWSER_API FContentBrowserAssetViewWidgetConstructor();
	TEDSCONTENTBROWSER_API explicit FContentBrowserAssetViewWidgetConstructor(const UScriptStruct* TypeInfo);
	~FContentBrowserAssetViewWidgetConstructor() override = default;

	TEDSCONTENTBROWSER_API virtual TSharedPtr<SWidget> CreateWidget(
		UE::Editor::DataStorage::ICoreProvider* DataStorage,
		UE::Editor::DataStorage::IUiProvider* DataStorageUi,
		UE::Editor::DataStorage::RowHandle TargetRow,
		UE::Editor::DataStorage::RowHandle WidgetRow, 
		const UE::Editor::DataStorage::FMetaDataView& Arguments) override;

private:
};
