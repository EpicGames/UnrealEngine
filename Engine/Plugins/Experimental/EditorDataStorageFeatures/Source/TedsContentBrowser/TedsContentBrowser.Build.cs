// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class TedsContentBrowser : ModuleRules
{
	public TedsContentBrowser(ReadOnlyTargetRules Target) : base(Target)
	{
		// Enable truncation warnings in this module
		CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Warning;

		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		if (Target.bBuildEditor)
		{
			PublicIncludePaths.AddRange(new string[] {});
			PrivateIncludePaths.AddRange(new string[] {});

			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject"
				});
			
			PrivateDependencyModuleNames.AddRange(
            	new string[]
            	{
		            "TypedElementFramework",
		            "SlateCore",
		            "Slate",
		            "TedsTableViewer",
		            "ContentBrowser",
		            "ContentBrowserData",
		            "InputCore",
					"TedsAlerts",
		            "TedsAssetData",
		            "ToolWidgets",
		            "TedsQueryStack"
            	});
			
			DynamicallyLoadedModuleNames.AddRange(new string[] {});
		}
	}
}
