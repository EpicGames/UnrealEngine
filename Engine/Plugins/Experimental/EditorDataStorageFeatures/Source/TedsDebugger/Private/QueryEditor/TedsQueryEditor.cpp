// Copyright Epic Games, Inc. All Rights Reserved.

#include "QueryEditor/TedsQueryEditor.h"

#include "QueryEditor/TedsQueryEditorModel.h"
#include "Components/VerticalBox.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Input/SEditableComboBox.h"
#include "Widgets/QueryEditor/TedsConditionSelectionComboWidget.h"
#include "Widgets/QueryEditor/TedsConditionCollectionViewWidget.h"
#include "Widgets/QueryEditor/TedsHierarchySelectionComboWidget.h"
#include "Widgets/QueryEditor/TedsQueryEditorResultsView.h"

#define LOCTEXT_NAMESPACE "TedsQueryEditor"

namespace UE::Editor::DataStorage::Debug::QueryEditor
{
	struct SQueryEditorWidget::ColumnComboItem
	{
		const FConditionEntry* Entry = nullptr;

		bool operator==(const ColumnComboItem& Rhs) const
		{
			return Entry == Rhs.Entry;
		}
	};

	void SQueryEditorWidget::Construct(const FArguments& InArgs, FTedsQueryEditorModel& QueryEditorModel)
	{	
		ComboItems.Reset();
		Model = &QueryEditorModel;

		HierarchyComboWidget = SNew(SHierarchyComboWidget, *Model);
	
		ChildSlot
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("Brushes.Background"))
			[
				SNew(SVerticalBox)
				+SVerticalBox::Slot()
				.AutoHeight()
				[
					CreateToolbar()
				]
				+SVerticalBox::Slot()
				.AutoHeight()
				[
					SNew(SHorizontalBox)
					+SHorizontalBox::Slot()
					[
						SNew(SConditionCollectionViewWidget, *Model, EOperatorType::Select)
					]
					+SHorizontalBox::Slot()
					.AutoWidth()
					[
						SNew(SConditionComboWidget, *Model, EOperatorType::Select)
					]
				]
				+SVerticalBox::Slot()
				.AutoHeight()
				[
					SNew(SHorizontalBox)
					+SHorizontalBox::Slot()
					[
						SNew(SConditionCollectionViewWidget, *Model, EOperatorType::All)
					]
					+SHorizontalBox::Slot()
					.AutoWidth()
					[
						SNew(SConditionComboWidget, *Model, EOperatorType::All)
					]
				]
				+SVerticalBox::Slot()
				.AutoHeight()
				[
					SNew(SHorizontalBox)
					+SHorizontalBox::Slot()
					[
					SNew(SConditionCollectionViewWidget, *Model, EOperatorType::Any)
					]
					+SHorizontalBox::Slot()
					.AutoWidth()
					[
					SNew(SConditionComboWidget, *Model, EOperatorType::Any)
					]
				]
				+SVerticalBox::Slot()
				.AutoHeight()
				[
					SNew(SHorizontalBox)
					+SHorizontalBox::Slot()
					[
					SNew(SConditionCollectionViewWidget, *Model, EOperatorType::None)
					]
					+SHorizontalBox::Slot()
					.AutoWidth()
					[
					SNew(SConditionComboWidget, *Model, EOperatorType::None)
					]
				]
				+SVerticalBox::Slot()
				[
					SAssignNew(ResultsView, SResultsView, *Model)
				]
			]
			
		];
	}

	TSharedRef<SWidget> SQueryEditorWidget::CreateToolbar()
	{
		// We're directly using FMenuBuilder since this is a toolbar specific to the query editor that doesn't need extension via UToolMenus
		FSlimHorizontalToolBarBuilder ToolBarBuilder(nullptr, FMultiBoxCustomization::None);

		ToolBarBuilder.AddComboButton( 
			FUIAction(),
			FOnGetContent::CreateSP(this, &SQueryEditorWidget::GetViewModesMenuWidget),
			TAttribute<FText>::CreateSP(this, &SQueryEditorWidget::GetCurrentViewModeText),
			TAttribute<FText>(),
			TAttribute<FSlateIcon>::CreateSP(this, &SQueryEditorWidget::GetCurrentViewModeIcon),
			/*bInSimpleComboBox*/ false);

		if (HierarchyComboWidget)
		{
			ToolBarBuilder.AddWidget(
				HierarchyComboWidget.ToSharedRef(),
				NAME_None, // default value
				true, // default value
				HAlign_Fill, // default value
				FNewMenuDelegate(), // default value
				TAttribute<EVisibility>::CreateLambda([this]()
				{
					// Only show the hierarchy combo box if we are using the hierarchy viewer
					return ResultsView && ResultsView->GetViewMode() == ETableViewMode::Tree ? EVisibility::Visible : EVisibility::Collapsed;
				}));
		}

		ToolBarBuilder.AddWidget(
			SNew(SBox)
			.VAlign(VAlign_Center)
			.Padding(FMargin(8.f, 0.f))
			[
				SNew(SCheckBox)
				.ForegroundColor(FSlateColor::UseForeground())
				.IsChecked_Lambda([this]()
				{
					return Model && Model->GetShowHiddenRows() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
				})
				.OnCheckStateChanged_Lambda([this](ECheckBoxState NewState)
				{
					if (Model)
					{
						Model->SetShowHiddenRows(NewState == ECheckBoxState::Checked);
					}
				})
				.ToolTipText(LOCTEXT("ShowHiddenRowsTooltip", "Show rows tagged with FHideRowFromUITag that are normally hidden from UI viewers"))
				[
					SNew(STextBlock)
					.Text(LOCTEXT("ShowHiddenRows", "Show Hidden Rows"))
				]
			]);

		return ToolBarBuilder.MakeWidget();
	}

	TSharedRef<SWidget> SQueryEditorWidget::GetViewModesMenuWidget()
	{
		FMenuBuilder MenuBuilder(true, nullptr);

		// Manually add the list and tree (hierarchy) view - The tile view is not supported yet
		MenuBuilder.AddMenuEntry(
			GetViewModeAsText(ETableViewMode::List),
			FText::GetEmpty(),
			GetViewModeAsIcon(ETableViewMode::List), 
			FUIAction(FExecuteAction::CreateLambda([this]()
			{
				if (ResultsView)
				{
					ResultsView->SetViewMode(ETableViewMode::List);
				}
			}))
		);

		MenuBuilder.AddMenuEntry(
			GetViewModeAsText(ETableViewMode::Tree),
			FText::GetEmpty(),
			GetViewModeAsIcon(ETableViewMode::Tree), 
			FUIAction(FExecuteAction::CreateLambda([this]()
			{
				if (ResultsView)
				{
					ResultsView->SetViewMode(ETableViewMode::Tree);
				}
			}))
		);

		return MenuBuilder.MakeWidget();
	}

	FText SQueryEditorWidget::GetCurrentViewModeText() const
	{
		return ResultsView ? GetViewModeAsText(ResultsView->GetViewMode()) : FText::GetEmpty();
	}

	FSlateIcon SQueryEditorWidget::GetCurrentViewModeIcon() const
	{
		return ResultsView ? GetViewModeAsIcon(ResultsView->GetViewMode()) : FSlateIcon();
	}

	FText SQueryEditorWidget::GetViewModeAsText(ETableViewMode::Type InViewMode)
	{
		switch (InViewMode)
		{
		case ETableViewMode::List:
			return LOCTEXT("ListViewMode", "List View");
		case ETableViewMode::Tile:
			return LOCTEXT("TileViewMode", "Tile View");
		case ETableViewMode::Tree:
			return LOCTEXT("TreeViewMode", "Hierarchy View");
		default:
			return FText::GetEmpty();
		}
	}

	FSlateIcon SQueryEditorWidget::GetViewModeAsIcon(ETableViewMode::Type InViewMode)
	{
		switch (InViewMode)
		{
		case ETableViewMode::List:
			return FSlateIcon("UMGStyle", "ClassIcon.ListView");
		case ETableViewMode::Tile:
			return FSlateIcon("UMGStyle", "ClassIcon.TileView");
		case ETableViewMode::Tree:
			return FSlateIcon("UMGStyle", "ClassIcon.TreeView");
		default:
			return FSlateIcon();
		}
	}
} // namespace UE::Editor::DataStorage::Debug::QueryEditor

#undef LOCTEXT_NAMESPACE