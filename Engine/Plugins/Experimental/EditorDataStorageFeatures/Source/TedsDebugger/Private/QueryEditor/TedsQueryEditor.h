// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Framework/Views/ITypedTableView.h"
#include "Templates/SharedPointerFwd.h"
#include "Textures/SlateIcon.h"
#include "Widgets/SCompoundWidget.h"

class SDockTab;
class SWindow;

namespace UE::Editor::DataStorage::Debug::QueryEditor
{
	class SHierarchyComboWidget;
	class SResultsView;
	class FTedsQueryEditorModel;
	class SQueryEditorWidget : public SCompoundWidget
	{
	public:

		SLATE_BEGIN_ARGS(SQueryEditorWidget) 
		{
		}
		SLATE_END_ARGS()

		void Construct(const FArguments& InArgs, FTedsQueryEditorModel& QueryEditorModel);

		/**
		 * Wrapper class for a combo item in the dropdown selection
		 */
		struct ColumnComboItem;

	private:

		TSharedRef<SWidget> CreateToolbar();
		TSharedRef<SWidget> GetViewModesMenuWidget();
		FText GetCurrentViewModeText() const;
		static FText GetViewModeAsText(ETableViewMode::Type InViewMode);
		FSlateIcon GetCurrentViewModeIcon() const;
		static FSlateIcon GetViewModeAsIcon(ETableViewMode::Type InViewMode);
		
	private:
		
		FTedsQueryEditorModel* Model = nullptr;
		TArray<TSharedPtr<ColumnComboItem>> ComboItems;
		TSharedPtr<SResultsView> ResultsView;
		TSharedPtr<SHierarchyComboWidget> HierarchyComboWidget;
	};
}

