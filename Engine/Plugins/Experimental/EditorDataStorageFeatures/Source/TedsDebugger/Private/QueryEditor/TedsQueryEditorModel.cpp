// Copyright Epic Games, Inc. All Rights Reserved.
#include "QueryEditor/TedsQueryEditorModel.h"

#include "DataStorage/CommonTypes.h"
#include "DataStorage/Debug/Log.h"
#include "Elements/Framework/TypedElementColumnUtils.h"
#include "Elements/Interfaces/TypedElementDataStorageInterface.h"
#include "UObject/UObjectIterator.h"

namespace UE::Editor::DataStorage::Debug::QueryEditor
{
	bool FConditionEntryHandle::operator==(const FConditionEntryHandle& Rhs) const
	{
		return Id == Rhs.Id;
	}

	bool FConditionEntryHandle::IsValid() const
	{
		return Id != NAME_None;
	}

	void FConditionEntryHandle::Reset()
	{
		Id = NAME_None;
	}

	FTedsQueryEditorModel::FTedsQueryEditorModel(ICoreProvider& InDataStorageProvider)
		: EditorDataStorageProvider(InDataStorageProvider)
	{
	}

	void FTedsQueryEditorModel::Reset()
	{
		RegenerateColumnsList();
		ModelChangedDelegate.Broadcast();
	}

	ICoreProvider& FTedsQueryEditorModel::GetTedsInterface()
	{
		return EditorDataStorageProvider;
	}
	
	const ICoreProvider& FTedsQueryEditorModel::GetTedsInterface() const
	{
		return EditorDataStorageProvider;
	}

	FQueryDescription FTedsQueryEditorModel::GenerateQueryDescription()
	{
		FQueryDescription Description;

		for (const FConditionEntryInternal& Entry : Conditions)
		{
			const UScriptStruct* Target = Entry.Struct;

			auto ValidateCondition = [&Entry, Target = Entry.Struct]()
			{
				switch(Entry.OperatorType)
				{
				case EOperatorType::Select:
				case EOperatorType::All:
				case EOperatorType::Any:
				case EOperatorType::None:
					break;
				case EOperatorType::Unset:
				case EOperatorType::Invalid:
				default:
					return false;
				};
				
				if (Target == nullptr)
				{
					return false;
				}
				return true;
			};

			if (ValidateCondition())
			{
				if (Entry.OperatorType == EOperatorType::Select)
				{
					Description.SelectionMetaData.Emplace(FColumnMetaData());
					Description.SelectionAccessTypes.Add(EQueryAccessType::ReadOnly);
					Description.SelectionTypes.Add(Target);
				}
				else
				{
					FQueryDescription::EOperatorType OperatorType = [](const FConditionEntryInternal& Entry)
					{
						switch(Entry.OperatorType)
						{
						case EOperatorType::All:
							return FQueryDescription::EOperatorType::SimpleAll;
						case EOperatorType::Any:
							return FQueryDescription::EOperatorType::SimpleAny;
						case EOperatorType::None:
							return FQueryDescription::EOperatorType::SimpleNone;
						case EOperatorType::Invalid:
						case EOperatorType::Unset:
						default:
							check(false);
							return FQueryDescription::EOperatorType::Max;
						}
					}(Entry);
				
					Description.ConditionTypes.Add(OperatorType);
					Description.ConditionOperators.AddZeroed_GetRef().Type = Target;
				}
			}
		}

		Description.Action = FQueryDescription::EActionType::Select;

		return Description;
	}

	FQueryDescription FTedsQueryEditorModel::GenerateNoSelectQueryDescription()
	{
		FQueryDescription Description = GenerateQueryDescription();

		// Move all the selection types to Condition types
		const int32 SelectionTypeCount = Description.SelectionTypes.Num();

		for (int32 Index = 0; Index < SelectionTypeCount; ++Index)
		{
			Description.ConditionOperators.AddZeroed_GetRef().Type = Description.SelectionTypes[Index];
			Description.ConditionTypes.Add(FQueryDescription::EOperatorType::SimpleAll);
		}

		Description.SelectionTypes.Empty();
		Description.SelectionMetaData.Empty();
		Description.SelectionAccessTypes.Empty();

		Description.Action = FQueryDescription::EActionType::Count;

		return Description;
	}
	
	int32 FTedsQueryEditorModel::CountConditionsOfOperator(EOperatorType OperatorType) const
	{
		int32 Count = 0;
		for (const FConditionEntryInternal& Entry : Conditions)
		{
			if (OperatorType == Entry.OperatorType)
			{
				++Count;
			}
		}
		return Count;
	}

	void FTedsQueryEditorModel::ForEachCondition(TFunctionRef<void(const FTedsQueryEditorModel& Model, FConditionEntryHandle Handle)> Function) const
	{
		for (const FConditionEntryInternal& Entry : Conditions)
		{
			FConditionEntryHandle Handle;
			Handle.Id = Entry.Id;
			Function(*this, Handle);
		}
	}

	void FTedsQueryEditorModel::ForEachCondition(TFunctionRef<void(FTedsQueryEditorModel& Model, FConditionEntryHandle Handle)> Function)
	{
		for (FConditionEntryInternal& Entry : Conditions)
		{
			FConditionEntryHandle Handle;
			Handle.Id = Entry.Id;
			Function(*this, Handle);
		}
	}

	EErrorCode FTedsQueryEditorModel::GenerateValidOperatorChoices(EOperatorType OperatorType, TFunctionRef<void(const FTedsQueryEditorModel& Model, const FConditionEntryHandle Handle)> Function)
	{
		// There is a special rule to follow to ensure that we are generating valid queries for Mass
		// Mass cannot have only none None operators. Need to check that we have either Any or All conditions
		// before adding operators to None
		
		int32 AnyCount = 0;
		int32 AllCount = 0;
		int32 SelectCount = 0;
		for (const FConditionEntryInternal& Entry : Conditions)
		{
			if (Entry.OperatorType == EOperatorType::All)
			{
				++AllCount;
			}
			if (Entry.OperatorType == EOperatorType::Any)
			{
				++AnyCount;
			}
			if (Entry.OperatorType == EOperatorType::Select)
			{
				++SelectCount;
			}
		}

		// Constraint by Mass/TEDS is that a handle cannot be set to None if there is also not an
		// Any or All condition
		if (OperatorType == EOperatorType::None)
		{
			if (AllCount == 0 && AnyCount == 0 && SelectCount == 0)
			{
				return EErrorCode::ConstraintViolation;
			}
		}

		const UScriptStruct* TagType = FTag::StaticStruct();
		for (FConditionEntryInternal& Entry : Conditions)
		{
			if (Entry.OperatorType == EOperatorType::Unset)
			{
				// Don't show Tags or base dynamic templates for Select operators... not valid query
				if (OperatorType == EOperatorType::Select &&
						(Entry.Struct->IsChildOf(TagType) || ColumnUtils::IsDynamicTemplate(Entry.Struct)))
				{
					continue;
				}
				// Don't show base templates in All since it doesn't make much sense
				if (OperatorType == EOperatorType::All && ColumnUtils::IsDynamicTemplate(Entry.Struct))
				{
					continue;
				}
				
				FConditionEntryHandle Handle;
				Handle.Id = Entry.Id;
				Function(*this, Handle);
			}
		}

		return EErrorCode::Success;
	}

	EOperatorType FTedsQueryEditorModel::GetOperatorType(FConditionEntryHandle Handle) const
	{
		const FConditionEntryInternal* Entry = FindEntryByHandle(Handle);
		if (Entry)
		{
			return Entry->OperatorType;
		}
		return EOperatorType::Invalid;
	}

	TPair<bool, EErrorCode> FTedsQueryEditorModel::CanSetOperatorType(FConditionEntryHandle Handle, EOperatorType OperatorType)
	{
		const FConditionEntryInternal* ThisEntry = FindEntryByHandle(Handle);
		if (nullptr == ThisEntry)
		{
			return {false, EErrorCode::DoesNotExist};
		}

		int32 SelectCount = 0;
		int32 AnyCount = 0;
		int32 AllCount = 0;
		int32 NoneCount = 0;
		
		for (const FConditionEntryInternal& Entry : Conditions)
		{
			if (Entry.OperatorType == EOperatorType::Select)
			{
				++SelectCount;
			}
			if (Entry.OperatorType == EOperatorType::All)
			{
				++AllCount;
			}
			if (Entry.OperatorType == EOperatorType::Any)
			{
				++AnyCount;
			}
			if (Entry.OperatorType == EOperatorType::None)
			{
				++NoneCount;
			}
		}

		// Constraint by Mass/TEDS is that a handle cannot be set to None if there is also not an
		// Any or All condition
		if (OperatorType == EOperatorType::None)
		{
			if (AllCount == 0 && AnyCount == 0 && SelectCount == 0)
			{
				return {false, EErrorCode::ConstraintViolation};
			}
		}

		// Disallow setting an All, Any or Select operator to Unset iff there is only one
		// and there exists some None
		if (OperatorType == EOperatorType::Unset)
		{
			if (ThisEntry->OperatorType != EOperatorType::None)
			{
				if ((AllCount + AnyCount + SelectCount) == 1 && NoneCount > 0)
				{
					return {false, EErrorCode::ConstraintViolation};
				}
			}
		}

		return {true, EErrorCode::Success};
		
	}

	EErrorCode FTedsQueryEditorModel::SetOperatorType(FConditionEntryHandle Handle, EOperatorType OperatorType)
	{
		// Better to check if FConditionEntry's address is within the UnsetColumns range before doing a Find
		FConditionEntryInternal* Entry = Conditions.FindByPredicate([Handle](const FConditionEntryInternal& Entry){ return Entry.Id == Handle.Id; });
		if (Entry == nullptr)
		{
			return EErrorCode::InvalidParameter;
		}

		const EOperatorType PreviousType = Entry->OperatorType;
		Entry->OperatorType = OperatorType;

		if (PreviousType != OperatorType)
		{
			++CurrentVersion;
			ModelChangedDelegate.Broadcast();
		}
		
		return EErrorCode::Success;
	}

	const UScriptStruct* FTedsQueryEditorModel::GetColumnScriptStruct(FConditionEntryHandle Handle) const
	{
		const FConditionEntryInternal* Entry = Conditions.FindByPredicate([Handle](const FConditionEntryInternal& Entry){ return Entry.Id == Handle.Id; });
		if (Entry == nullptr)
		{
			return nullptr;
		}
		return Entry->Struct;
	}

	FTedsQueryEditorModel::FConditionEntryInternal* FTedsQueryEditorModel::FindEntryByHandle(FConditionEntryHandle Handle)
	{
		return Conditions.FindByPredicate([Handle](const FConditionEntryInternal& Entry){ return Entry.Id == Handle.Id; });
	}

	const FTedsQueryEditorModel::FConditionEntryInternal* FTedsQueryEditorModel::FindEntryByHandle(FConditionEntryHandle Handle) const
	{
		return Conditions.FindByPredicate([Handle](const FConditionEntryInternal& Entry){ return Entry.Id == Handle.Id; });
	}

	void FTedsQueryEditorModel::RegenerateColumnsList()
	{
		TArray<const UScriptStruct*> Columns;
		TArray<const UScriptStruct*> Tags;

		const UScriptStruct* ColumnType = FColumn::StaticStruct();
		const UScriptStruct* TagType = FTag::StaticStruct();
		// Not sure if there is a faster way to do this.  Would be nice to iterate only the derived classes
		for(TObjectIterator< const UScriptStruct > It; It; ++It)
		{
			const UScriptStruct* Struct = *It;

			if (Struct->IsChildOf(ColumnType) && Struct != ColumnType)
			{
				Columns.Add(Struct);
			}
			if (Struct->IsChildOf(TagType) && Struct != TagType)
			{
				Tags.Add(Struct);
			}
		}

		// Save a map of any columns that previously had valid operators so we can restore them
		TMap<FName, EOperatorType> PreviousValidOperators;
		
		ForEachCondition([&PreviousValidOperators](FTedsQueryEditorModel& Model, FConditionEntryHandle Handle)
			{
				switch(EOperatorType Operator = Model.GetOperatorType(Handle))
				{
				case EOperatorType::Select:
				case EOperatorType::All:
				case EOperatorType::Any:
				case EOperatorType::None:
					PreviousValidOperators.Emplace(Handle.Id, Operator);
					break;
				case EOperatorType::Unset:
				case EOperatorType::Invalid:
				default:
					return;
				};
			});

		Conditions.Empty();
		Conditions.Reserve(Columns.Num() + Tags.Num());

		auto GetPreviousOperator = [PreviousValidOperators] (const UScriptStruct* Column)
		{
			if (const EOperatorType* PreviousOp = PreviousValidOperators.Find(Column->GetFName()))
			{
				return *PreviousOp;
			}

			return EOperatorType::Unset;
		};

		for (const UScriptStruct* Column : Columns)
		{
			EOperatorType Operator = GetPreviousOperator(Column);
			
			Conditions.Emplace(
			FConditionEntryInternal
				{
					.Id = Column->GetFName(),
					.Struct = Column,
					.OperatorType = Operator
				});
		}
		for (const UScriptStruct* Tag : Tags)
		{
			EOperatorType Operator = GetPreviousOperator(Tag);
			
			Conditions.Emplace(
			FConditionEntryInternal
				{
					.Id = Tag->GetFName(),
					.Struct = Tag,
					.OperatorType = Operator
				});
		}
	}

	void FTedsQueryEditorModel::SetHierarchy(const FName& InHierarchy)
	{
		CurrentHierarchy = InHierarchy;
		CurrentRelationType.Reset();
		TableViewerRefreshRequiredDelegate.Broadcast();
	}

	void FTedsQueryEditorModel::SetShowHiddenRows(bool bInShowHiddenRows)
	{
		bShowHiddenRows = bInShowHiddenRows;
		TableViewerRefreshRequiredDelegate.Broadcast();
	}

	bool FTedsQueryEditorModel::GetShowHiddenRows() const
	{
		return bShowHiddenRows;
	}

	void FTedsQueryEditorModel::SetRelationType(RelationTypeHandle InRelationType)
	{
		CurrentRelationType = InRelationType;
		CurrentHierarchy = NAME_None;
		TableViewerRefreshRequiredDelegate.Broadcast();
	}

	bool FTedsQueryEditorModel::HasRelationType() const
	{
		return CurrentRelationType.IsSet();
	}

	RelationTypeHandle FTedsQueryEditorModel::GetRelationType() const
	{
		return CurrentRelationType.Get(InvalidRelationTypeHandle);
	}

	FName FTedsQueryEditorModel::GetHierarchyName() const
	{
		return CurrentHierarchy;
	}

	FHierarchyHandle FTedsQueryEditorModel::GetHierarchy() const
	{
		return EditorDataStorageProvider.FindHierarchyByName(CurrentHierarchy);
	}
} // namespace UE::Editor::DataStorage::Debug::QueryEditor
