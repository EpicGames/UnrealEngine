// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "DataStorage/Handles.h"
#include "Elements/Framework/TypedElementMetaData.h"
#include "UObject/Class.h"

namespace UE::Editor::DataStorage
{
	class ICoreProvider;
}

namespace UE::Editor::DataStorage::Debug::QueryEditor
{
	class FTedsQueryEditorModel;

	enum class EOperatorType : uint32
	{
		Invalid, // Not initialized
		Select,
		All,
		Any,
		None,
		Unset // Not associated with a set
	};

	enum class EErrorCode
	{
		Success,
		AlreadyExists,
		DoesNotExist,
		InvalidParameter,
		ConstraintViolation
	};

	struct FConditionEntryHandle
	{
		FConditionEntryHandle() = default;
		bool operator==(const FConditionEntryHandle& Rhs) const;
		bool IsValid() const;
		void Reset();

	private:
		friend class FTedsQueryEditorModel;
		// Unique ID for a condition
		// Used to identify the entry by external code (ie. list views)
		FName Id = NAME_None;
	};

	DECLARE_MULTICAST_DELEGATE(FTedsQueryEditorModel_ModelChanged);

	class FTedsQueryEditorModel
	{
	public:
		explicit FTedsQueryEditorModel(ICoreProvider& InDataStorageProvider);

		void Reset();
		ICoreProvider& GetTedsInterface();
		const ICoreProvider& GetTedsInterface() const;
		
		FQueryDescription GenerateQueryDescription();
		// Special function to generate a description that puts the Select elements as All Conditions
		// This is helpful for using a Count query type or for the TableViewer which requires
		// the row query to have no select items
		FQueryDescription GenerateNoSelectQueryDescription();

		int32 CountConditionsOfOperator(EOperatorType OperatorType) const;
		
		/**
		 * Runs given function over all conditions that match the condition type
		 * The order of the conditions is not guaranteeds to be the same each run.
		 */
		void ForEachCondition(TFunctionRef<void(const FTedsQueryEditorModel& Model, FConditionEntryHandle Handle)> Function) const;

		void ForEachCondition(TFunctionRef<void(FTedsQueryEditorModel& Model, FConditionEntryHandle Handle)> Function);

		EErrorCode GenerateValidOperatorChoices(EOperatorType OperatorType, TFunctionRef<void(const FTedsQueryEditorModel& Model, const FConditionEntryHandle Handle)> Function);
		
		EOperatorType GetOperatorType(FConditionEntryHandle Handle) const;
		TPair<bool, EErrorCode> CanSetOperatorType(FConditionEntryHandle Handle, EOperatorType OperatorType);
		/**
		 * @param OperatorType
		 * @param ColumnType - Column or tag type
		 * @return
		 *	Success - Column or tag added to collection
		 *	DuplicateInSameCondition - Column or tag already present in condition collection
		 *	ExistsInOtherCondition - Column or tag present in a different condition collection
		 *	InvalidParameter - Column or tag cannot be added to condition collection
		 */
		EErrorCode SetOperatorType(FConditionEntryHandle Handle, EOperatorType OperatorType);

		const UScriptStruct* GetColumnScriptStruct(FConditionEntryHandle Handle) const;
		
		FTedsQueryEditorModel_ModelChanged& GetModelChangedDelegate() { return ModelChangedDelegate; }

		// Fired when the table viewer widget needs to be rebuilt (hierarchy or row visibility settings changed)
		FTedsQueryEditorModel_ModelChanged& OnTableViewerRefreshRequired() { return TableViewerRefreshRequiredDelegate; }

		void SetShowHiddenRows(bool bInShowHiddenRows);
		bool GetShowHiddenRows() const;
		
		void RegenerateColumnsList();

		void SetHierarchy(const FName& InHierarchy);
		FName GetHierarchyName() const;
		FHierarchyHandle GetHierarchy() const;

		// Relation-type hierarchy selection (mutually exclusive with SetHierarchy)
		void SetRelationType(RelationTypeHandle InRelationType);
		bool HasRelationType() const;
		RelationTypeHandle GetRelationType() const;
	
	private:
		
		struct FConditionEntryInternal
		{
			// Unique ID for a condition
			// Used to identify the entry by external code (ie. list views)
			FName Id;
			const UScriptStruct* Struct;
			EOperatorType OperatorType;

			bool operator==(const FConditionEntryHandle& Rhs) const { return Id == Rhs.Id; }
		};

		FConditionEntryInternal* FindEntryByHandle(FConditionEntryHandle Handle);
		const FConditionEntryInternal* FindEntryByHandle(FConditionEntryHandle Handle) const;

		ICoreProvider& EditorDataStorageProvider;

		TArray<FConditionEntryInternal> Conditions;
		FTedsQueryEditorModel_ModelChanged ModelChangedDelegate;

		uint64 CurrentVersion = 0;
		uint64 GeneratedVersion = 0;
		QueryHandle QueryHandle;

		// The current hierarchy being viewed (only applicable for hierarchy based widgets).
		// CurrentHierarchy and CurrentRelationType are mutually exclusive — only one is active at a time.
		FName CurrentHierarchy;
		TOptional<RelationTypeHandle> CurrentRelationType;
		FTedsQueryEditorModel_ModelChanged TableViewerRefreshRequiredDelegate;

		bool bShowHiddenRows = false;
	};
} // namespace UE::Editor::DataStorage::Debug::QueryEditor
