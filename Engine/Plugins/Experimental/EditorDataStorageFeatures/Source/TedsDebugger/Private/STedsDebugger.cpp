// Copyright Epic Games, Inc. All Rights Reserved.

#include "STedsDebugger.h"

#include "SceneOutlinerPublicTypes.h"
#include "DataStorage/Features.h"
#include "Elements/Columns/TypedElementLabelColumns.h"
#include "Elements/Columns/TypedElementMiscColumns.h"
#include "Elements/Columns/TypedElementSelectionColumns.h"
#include "Elements/Columns/TypedElementTypeInfoColumns.h"
#include "Elements/Interfaces/TypedElementDataStorageInterface.h"
#include "Framework/Docking/LayoutService.h"
#include "Modules/ModuleManager.h"
#include "QueryEditor/TedsQueryEditor.h"
#include "TedsOutlinerModule.h"
#include "TedsOutlinerMode.h"
#include "Widgets/SScopeHierarchyTab.h"
#include "Widgets/STedsDiscoveryDebugger.h"
#include "Widgets/STedsTableDebugger.h"

#define LOCTEXT_NAMESPACE "STedsDebugger"

namespace UE::Editor::DataStorage::Debug
{
	namespace Private
	{
		const FName ToolbarTabName = TEXT("TEDS Debugger Toolbar");
		const FName TableTabName = TEXT("Tables");
		const FName DiscoveryTabName = TEXT("Discover");
		const FName ContextHierarchyTabName = TEXT("Scope Hierarchy");
	}

enum class EQueryEditorTabs
{
	QueryEditorOne,
	QueryEditorTwo,
	QueryEditorThree,
	QueryEditorFour,
};

STedsDebugger::~STedsDebugger()
{
	if (AreEditorDataStorageFeaturesEnabled())
	{
		GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName)->UnregisterQuery(TableViewerQuery);
	}
}

void STedsDebugger::Construct(const FArguments& InArgs, const TSharedRef<SDockTab>& ConstructUnderMajorTab, const TSharedPtr<SWindow>& ConstructUnderWindow)
{
	// Create the tab manager for our sub tabs
	TabManager = FGlobalTabmanager::Get()->NewTabManager(ConstructUnderMajorTab);
	TabManager->SetAllowWindowMenuBar(true);

	Models.SetNum(MaxQueryEditorTabs);

	// Register Tab Spawners
	RegisterTabSpawners();

	// Setup the default layout
	TSharedRef<FTabManager::FLayout> Layout = FTabManager::NewLayout("TedsDebuggerLayout_v1.3")
	->AddArea
	(
		FTabManager::NewPrimaryArea()
		->SetOrientation(Orient_Vertical)
		->Split
		(
			FTabManager::NewStack()
			->AddTab(Private::ToolbarTabName, ETabState::OpenedTab)
			->SetHideTabWell(true)
		)
		->Split
		(
			FTabManager::NewSplitter()
			->SetOrientation(Orient_Horizontal)
			->Split
			(
				FTabManager::NewStack()
					->AddTab(GetQueryTabName(EQueryEditorTabs::QueryEditorOne), ETabState::OpenedTab)
					->AddTab(GetQueryTabName(EQueryEditorTabs::QueryEditorTwo), ETabState::ClosedTab)
					->AddTab(GetQueryTabName(EQueryEditorTabs::QueryEditorThree), ETabState::ClosedTab)
					->AddTab(GetQueryTabName(EQueryEditorTabs::QueryEditorFour), ETabState::ClosedTab)
					->AddTab(Private::TableTabName, ETabState::OpenedTab)
					->AddTab(Private::DiscoveryTabName, ETabState::OpenedTab)
					->AddTab(Private::ContextHierarchyTabName, ETabState::OpenedTab)
					->SetHideTabWell(false)
					->SetForegroundTab(GetQueryTabName(EQueryEditorTabs::QueryEditorOne))
			)
		)
	);

	Layout = FLayoutSaveRestore::LoadFromConfig(GEditorLayoutIni, Layout);

	ChildSlot
	[
		TabManager->RestoreFrom(Layout, ConstructUnderWindow).ToSharedRef()
	];

	TabManager->SetOnPersistLayout(
		FTabManager::FOnPersistLayout::CreateStatic(
			[](const TSharedRef<FTabManager::FLayout>& InLayout)
			{
				if (InLayout->GetPrimaryArea().Pin().IsValid())
				{
					FLayoutSaveRestore::SaveToConfig(GEditorLayoutIni, InLayout);
				}
			}
		)
	);

	// create & initialize main menu
	FMenuBarBuilder MenuBarBuilder = FMenuBarBuilder(TSharedPtr<FUICommandList>());

	MenuBarBuilder.AddPullDownMenu(
		LOCTEXT("WindowMenuLabel", "Window"),
		FText::GetEmpty(),
		FNewMenuDelegate::CreateSP(this, &STedsDebugger::FillWindowMenu),
		"Window"
	);

	TSharedRef<SWidget> MenuWidget = MenuBarBuilder.MakeWidget();

	// Tell tab-manager about the multi-box for platforms with a global menu bar
	TabManager->SetMenuMultiBox(MenuBarBuilder.GetMultiBox(), MenuWidget);
}

void STedsDebugger::FillWindowMenu( FMenuBuilder& MenuBuilder)
{
	if (TabManager)
	{
		TabManager->PopulateLocalTabSpawnerMenu(MenuBuilder);
	}
}

QueryEditor::FTedsQueryEditorModel* STedsDebugger::FindQueryEditorModel(const EQueryEditorTabs QueryTabIdentifier)
{
	if (AreEditorDataStorageFeaturesEnabled(); ICoreProvider* DataStorageInterface = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName))
	{
		TUniquePtr<QueryEditor::FTedsQueryEditorModel>& Model = Models[static_cast<int32>(QueryTabIdentifier)];
		if (!Model)
		{
			Model = MakeUnique<QueryEditor::FTedsQueryEditorModel>(*DataStorageInterface);
		}
		return Model.Get();
	}

	return nullptr; // Return null if TEDS is disabled
}

FName STedsDebugger::GetQueryTabName(const EQueryEditorTabs QueryTabIdentifier)
{
	// Returns a name with the syntax: QueryEditorTab, QueryEditorTab_1, QueryEditorTab_2, etc...
	return FName(TEXT("QueryEditorTab"), static_cast<int32>(QueryTabIdentifier));
}

TSharedRef<SDockTab> STedsDebugger::SpawnToolbar(const FSpawnTabArgs& Args)
{
	// The toolbar is currently empty but can be used to house tools that are not specific to a specific tab in the debugger
	FSlimHorizontalToolBarBuilder ToolBarBuilder(nullptr, FMultiBoxCustomization::None);

	return SNew(SDockTab)
		.TabRole(ETabRole::PanelTab)
		.ShouldAutosize(true)
		[
			ToolBarBuilder.MakeWidget()
		];
}

TSharedRef<SDockTab> STedsDebugger::SpawnQueryEditorTab(const FSpawnTabArgs& Args, const EQueryEditorTabs QueryTabIdentifier)
{
	TSharedRef<SDockTab> DockTab = SNew(SDockTab)
		.TabRole(ETabRole::PanelTab)
		.Label(FText::Format(LOCTEXT("TedsDebugger_QueryEditorDisplayName", "Query Editor {0}"), FText::AsNumber(static_cast<int32>(QueryTabIdentifier) + 1)));

	if (QueryEditor::FTedsQueryEditorModel* QueryEditorModel = FindQueryEditorModel(QueryTabIdentifier))
	{
		QueryEditorModel->Reset();

		TSharedRef<QueryEditor::SQueryEditorWidget> QueryEditorWidget =
			SNew(QueryEditor::SQueryEditorWidget, *QueryEditorModel);
		DockTab->SetContent(QueryEditorWidget);
	}
	else
	{
		TSharedRef<STextBlock> TextBlock = SNew(STextBlock)
			.Text(LOCTEXT("TedsDebuggerModule_CannotLoadQueryEditor", "Cannot load Query Editor - Invalid Model"));
		DockTab->SetContent(TextBlock);
	}

	return DockTab;
}

TSharedRef<SDockTab> STedsDebugger::SpawnTableTab(const FSpawnTabArgs& Args)
{
	TSharedRef<SDockTab> DockTab = SNew(SDockTab)
		.TabRole(ETabRole::PanelTab)
		.Label(LOCTEXT("TedsDebugger_TableDisplayName", "Tables"));

	DockTab->SetContent(SNew(STableDebuggerTab));
	return DockTab;
}

TSharedRef<SDockTab> STedsDebugger::SpawnDiscoverTab(const FSpawnTabArgs& Args)
{
	TSharedRef<SDockTab> DockTab = SNew(SDockTab)
		.TabRole(ETabRole::PanelTab)
		.Label(LOCTEXT("TedsDebugger_DiscoverDisplayName", "Discover"));

	DockTab->SetContent(SNew(SDiscoveryDebuggerTab));
	return DockTab;
}

TSharedRef<SDockTab> STedsDebugger::SpawnContextHierarchyTab(const FSpawnTabArgs& Args)
{
	TSharedRef<SDockTab> DockTab = SNew(SDockTab)
		.TabRole(ETabRole::PanelTab)
		.Label(LOCTEXT("TedsDebugger_ContextHierarchyDisplayName", "Scope Hierarchy"));

	DockTab->SetContent(SNew(SScopeHierarchyTab));
	return DockTab;
}

void STedsDebugger::RegisterTabSpawners()
{
	const TSharedRef<FWorkspaceItem> AppMenuGroup =
		TabManager->AddLocalWorkspaceMenuCategory(LOCTEXT("TedsDebuggerGroupName", "Teds Debugger"));
	const FText QueryEditorTooltip = LOCTEXT("TedsDebugger_QueryEditorToolTip", "Open a Query Editor tab. Use this to create and view individual queries.");
	
	TabManager->RegisterTabSpawner(
		Private::ToolbarTabName,
		FOnSpawnTab::CreateRaw(this, &STedsDebugger::SpawnToolbar))
		.SetGroup(AppMenuGroup)
		.SetDisplayName(LOCTEXT("TedsDebugger_ToolbarDisplayName", "Toolbar"))
		.SetAutoGenerateMenuEntry(false);
	
	TabManager->RegisterTabSpawner(
		GetQueryTabName(EQueryEditorTabs::QueryEditorOne),
		FOnSpawnTab::CreateRaw(this, &STedsDebugger::SpawnQueryEditorTab, EQueryEditorTabs::QueryEditorOne))
		.SetGroup(AppMenuGroup)
		.SetDisplayName(LOCTEXT("TedsDebugger_QueryEditorDisplayNameOne", "Query Editor 1"))
		.SetTooltipText(QueryEditorTooltip)
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "Debug"));

	TabManager->RegisterTabSpawner(
		GetQueryTabName(EQueryEditorTabs::QueryEditorTwo),
		FOnSpawnTab::CreateRaw(this, &STedsDebugger::SpawnQueryEditorTab, EQueryEditorTabs::QueryEditorTwo))
		.SetGroup(AppMenuGroup)
		.SetDisplayName(LOCTEXT("TedsDebugger_QueryEditorDisplayNameTwo", "Query Editor 2"))
		.SetTooltipText(QueryEditorTooltip)
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "Debug"));

	TabManager->RegisterTabSpawner(
		GetQueryTabName(EQueryEditorTabs::QueryEditorThree),
		FOnSpawnTab::CreateRaw(this, &STedsDebugger::SpawnQueryEditorTab, EQueryEditorTabs::QueryEditorThree))
		.SetGroup(AppMenuGroup)
		.SetDisplayName(LOCTEXT("TedsDebugger_QueryEditorDisplayNameThree", "Query Editor 3"))
		.SetTooltipText(QueryEditorTooltip)
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "Debug"));

	TabManager->RegisterTabSpawner(
		GetQueryTabName(EQueryEditorTabs::QueryEditorFour),
		FOnSpawnTab::CreateRaw(this, &STedsDebugger::SpawnQueryEditorTab, EQueryEditorTabs::QueryEditorFour))
		.SetGroup(AppMenuGroup)
		.SetDisplayName(LOCTEXT("TedsDebugger_QueryEditorDisplayNameFour", "Query Editor 4"))
		.SetTooltipText(QueryEditorTooltip)
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "Debug"));

	TabManager->RegisterTabSpawner(
		Private::TableTabName,
		FOnSpawnTab::CreateRaw(this, &STedsDebugger::SpawnTableTab))
		.SetGroup(AppMenuGroup)
		.SetDisplayName(LOCTEXT("TedsDebugger_TableDisplayName", "Tables"))
		.SetTooltipText(LOCTEXT("TedsDebugger_TableToolTip", "Opens the Tables tab. Contains tooling to inspect tables in the data storage."))
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.TableTreeView"));

	TabManager->RegisterTabSpawner(
		Private::DiscoveryTabName,
		FOnSpawnTab::CreateRaw(this, &STedsDebugger::SpawnDiscoverTab))
		.SetGroup(AppMenuGroup)
		.SetDisplayName(LOCTEXT("TedsDebugger_DiscoverDisplayName", "Discover"))
		.SetTooltipText(LOCTEXT("TedsDebugger_DiscoverToolTip", "Opens the Discover tab. Contains special searching functionality to explore TEDS data without a query."))
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Search"));

	TabManager->RegisterTabSpawner(
		Private::ContextHierarchyTabName,
		FOnSpawnTab::CreateRaw(this, &STedsDebugger::SpawnContextHierarchyTab))
		.SetGroup(AppMenuGroup)
		.SetDisplayName(LOCTEXT("TedsDebugger_ContextHierarchyDisplayName", "Scope Hierarchy"))
		.SetTooltipText(LOCTEXT("TedsDebugger_ContextHierarchyToolTip", "Shows the ECP scope row hierarchy and lets you inspect creation callstacks and visible columns."))
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Hierarchy"));
}

} // namespace UE::Editor::DataStorage::Debug

#undef LOCTEXT_NAMESPACE
