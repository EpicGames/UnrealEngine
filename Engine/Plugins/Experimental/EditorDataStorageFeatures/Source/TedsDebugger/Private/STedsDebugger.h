// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "DataStorage/Handles.h"
#include "QueryEditor/TedsQueryEditorModel.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/SCompoundWidget.h"

class ISceneOutliner;

namespace UE::Editor::DataStorage
{
	namespace Debug
	{
		enum class EQueryEditorTabs;
		
		class STedsDebugger : public SCompoundWidget
		{
		public:

			SLATE_BEGIN_ARGS(STedsDebugger) 
			{
			}
			SLATE_END_ARGS()

		public:
			STedsDebugger() = default;
			virtual ~STedsDebugger() override;

			/**
			* Constructs the debugger.
			*
			* @param InArgs The Slate argument list.
			* @param ConstructUnderMajorTab The major tab which will contain the session front-end.
			* @param ConstructUnderWindow The window in which this widget is being constructed.
			*/
			void Construct(const FArguments& InArgs, const TSharedRef<SDockTab>& ConstructUnderMajorTab, const TSharedPtr<SWindow>& ConstructUnderWindow);
		private:
			QueryEditor::FTedsQueryEditorModel* FindQueryEditorModel(const EQueryEditorTabs QueryTabIdentifier);
			static FName GetQueryTabName(const EQueryEditorTabs QueryTabIdentifier);
			TSharedRef<SDockTab> SpawnToolbar(const FSpawnTabArgs& Args);
			TSharedRef<SDockTab> SpawnQueryEditorTab(const FSpawnTabArgs& Args, const EQueryEditorTabs QueryTabIdentifier);
			TSharedRef<SDockTab> SpawnTableTab(const FSpawnTabArgs& Args);
			TSharedRef<SDockTab> SpawnDiscoverTab(const FSpawnTabArgs& Args);
			TSharedRef<SDockTab> SpawnContextHierarchyTab(const FSpawnTabArgs& Args);
			void FillWindowMenu( FMenuBuilder& MenuBuilder);

			void RegisterTabSpawners();

		private:
	
			// Holds the tab manager that manages the front-end's tabs.
			TSharedPtr<FTabManager> TabManager;

			// Table Viewer
			QueryHandle TableViewerQuery;

			static constexpr int32 MaxQueryEditorTabs = 4;
			TArray<TUniquePtr<QueryEditor::FTedsQueryEditorModel>, TInlineAllocator<MaxQueryEditorTabs>> Models;
		};
	} // namespace Debug
} // namespace UE::Editor::DataStorage
