// Copyright Epic Games, Inc. All Rights Reserved.

#include "TedsDebuggerModule.h"

#include "STedsDebugger.h"
#include "Elements/Framework/TypedElementRegistry.h"
#include "Framework/Application/SlateApplication.h"
#include "Modules/ModuleManager.h"
#include "Widgets/Docking/SDockTab.h"
#include "WorkspaceMenuStructureModule.h"
#include "WorkspaceMenuStructure.h"

#define LOCTEXT_NAMESPACE "TedsDebuggerModule"

namespace UE::Editor::DataStorage::Debug
{
namespace Private
{
	FName TedsDebuggerTableName = TEXT("TEDS Debugger");
}

void FTedsDebuggerModule::StartupModule()
{
	IModuleInterface::StartupModule();

	FModuleManager::Get().LoadModule(TEXT("TypedElementFramework"));	

	RegisterTabSpawners();
}

void FTedsDebuggerModule::ShutdownModule()
{
	UnregisterTabSpawners();
	
	IModuleInterface::ShutdownModule();
}

void FTedsDebuggerModule::RegisterTabSpawners()
{
	FGlobalTabmanager::Get()->RegisterNomadTabSpawner(
		Private::TedsDebuggerTableName,
		FOnSpawnTab::CreateRaw(this, &FTedsDebuggerModule::OpenTedsDebuggerTab))
		.SetGroup(WorkspaceMenu::GetMenuStructure().GetDeveloperToolsDebugCategory())
		.SetDisplayName(LOCTEXT("TedsDebugger_QueryEditorDisplayName", "TEDS Debugger"))
		.SetTooltipText(LOCTEXT("TedsDebugger_QueryEditorToolTip", "Opens the TEDS Debugger"))
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "Debug"));
}

void FTedsDebuggerModule::UnregisterTabSpawners() const
{
	if (FSlateApplication::IsInitialized())
	{
		FGlobalTabmanager::Get()->UnregisterNomadTabSpawner(Private::TedsDebuggerTableName);
	}
}

TSharedRef<SDockTab> FTedsDebuggerModule::OpenTedsDebuggerTab(const FSpawnTabArgs& SpawnTabArgs)
{
	const TSharedRef<SDockTab> MajorTab = SNew(SDockTab)
	.TabRole(ETabRole::MajorTab);

	const TSharedRef<STedsDebugger> TedsDebuggerWidget = SNew(STedsDebugger, MajorTab, SpawnTabArgs.GetOwnerWindow());
	
	TedsDebuggerInstance = TedsDebuggerWidget;

	MajorTab->SetContent(TedsDebuggerWidget);
	
	return MajorTab;
}
} // namespace UE::Editor::DataStorage::Debug

IMPLEMENT_MODULE(UE::Editor::DataStorage::Debug::FTedsDebuggerModule, TedsDebugger);

#undef LOCTEXT_NAMESPACE