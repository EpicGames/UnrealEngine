// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataStorage/Handles.h"
#include "Modules/ModuleInterface.h"
#include "Templates/SharedPointer.h"

class SDockTab;
class SWidget;
class FSpawnTabArgs;
class ISceneOutliner;

namespace UE::Editor::DataStorage
{
	namespace Debug
	{
		class STedsDebugger;

		/**
		 * Implements the Teds Debugger module.
		 */
		class FTedsDebuggerModule
			: public IModuleInterface
		{
		public:

			FTedsDebuggerModule() = default;

			// IModuleInterface interface
			virtual void StartupModule() override;
			virtual void ShutdownModule() override;

		private:
			void RegisterTabSpawners();
			void UnregisterTabSpawners() const;
	
			TSharedRef<SDockTab> OpenTedsDebuggerTab(const FSpawnTabArgs& SpawnTabArgs);

		private:
			TWeakPtr<STedsDebugger> TedsDebuggerInstance;
		};
	} // namespace Debug
} // namespace UE::Editor::DataStorage
