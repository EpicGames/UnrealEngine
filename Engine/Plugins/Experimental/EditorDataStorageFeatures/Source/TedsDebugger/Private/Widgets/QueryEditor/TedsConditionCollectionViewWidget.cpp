// Copyright Epic Games, Inc. All Rights Reserved.
#include "TedsConditionCollectionViewWidget.h"

#include "Components/HorizontalBox.h"
#include "QueryEditor/TedsQueryEditorModel.h"
#include "Widgets/SOverlay.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SWrapBox.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "TedsDebuggerModule"

namespace UE::Editor::DataStorage::Debug::QueryEditor
{
	namespace Private
	{
		FText GetOperatorTypeText(EOperatorType OperatorType)
		{
			switch (OperatorType)
			{
			case EOperatorType::Select:
				return FText::FromStringView(TEXT("Select<"));
			case EOperatorType::All:
				return FText::FromStringView(TEXT("All<"));
			case EOperatorType::Any:
				return FText::FromStringView(TEXT("Any<"));
			case EOperatorType::None:
				return FText::FromStringView(TEXT("None<"));
			case EOperatorType::Unset:
				return FText::FromStringView(TEXT("Unset<"));
			case EOperatorType::Invalid:
			default:
				return FText::FromStringView(TEXT("Invalid<"));
			}
		}
	}

	SConditionCollectionViewWidget::~SConditionCollectionViewWidget()
	{
		Model->GetModelChangedDelegate().Remove(OnModelChangedDelegate);
	}

	void SConditionCollectionViewWidget::Construct(
		const FArguments& InArgs,
		FTedsQueryEditorModel& InModel, 
		QueryEditor::EOperatorType InOperatorType)
	{
		Model = &InModel;
		OperatorType = InOperatorType;

		OnModelChangedDelegate = Model->GetModelChangedDelegate().AddRaw(this, &SConditionCollectionViewWidget::OnModelChanged);

		ChildSlot
		[
			SNew(SBorder)
			.Padding(4.0, 3.0, 3.0, 4.0)
			[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SVerticalBox)
				+SVerticalBox::Slot()
				.VAlign(EVerticalAlignment::VAlign_Center)
				[
					SNew(STextBlock)
					.Text(Private::GetOperatorTypeText(OperatorType))
				]
			]
			+SHorizontalBox::Slot()
			[
			SAssignNew(ColumnButtonWrap, SWrapBox)
			.UseAllottedSize(true)
			+SWrapBox::Slot()
			.FillEmptySpace(false)
			[
				SNullWidget::NullWidget
			]
			]
			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SVerticalBox)
				+SVerticalBox::Slot()
				.VAlign(EVerticalAlignment::VAlign_Center)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("TedsQueryEditor_OpType_>", ">"))
				]
			]
			]
		];

		SetCanTick(true);
	}

	void SConditionCollectionViewWidget::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
	{
		SCompoundWidget::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);

		ColumnButtonWrap->ClearChildren();
		SetCanTick(false);

		ButtonStyle = FButtonStyle(FCoreStyle::Get().GetWidgetStyle< FButtonStyle >( "Button" ))
			.SetHoveredForeground(FSlateColor(FLinearColor(0.9f, 0.0f, 0.0f, 1.0f)));

		if (Model->CountConditionsOfOperator(OperatorType) != 0)
		{
			Model->ForEachCondition([this](FTedsQueryEditorModel&, FConditionEntryHandle Handle)
			{
				if (Model->GetOperatorType(Handle) != OperatorType)
				{
					return;
				}
				// TODO Add styling to the button to go red and put an "X" to signal clicking
				// will remove the column
				const UScriptStruct* Type = Model->GetColumnScriptStruct(Handle);
				FText TempName = FText::FromString(Type->GetName());
			
				TSharedRef<STextBlock> TextNameOfColumn = SNew(STextBlock)
					.Text(TempName);

				TSharedRef<SOverlay> ButtonContent = SNew(SOverlay);
			
				TSharedRef<SButton> Button = SNew(SButton)
					.Text(TempName)
					.OnClicked_Lambda( [this, Handle]() -> FReply 
					{
						Model->SetOperatorType(Handle, EOperatorType::Unset);
						return FReply::Handled();
					})
					.IsEnabled_Lambda( [this, Handle]() -> bool
					{
						return Model->CanSetOperatorType(Handle, EOperatorType::Unset).Key;
					})
					.Content()
					[
						TextNameOfColumn
					];
				STextBlock* TextBlockPtr = &TextNameOfColumn.Get();
				Button->SetOnHovered(FSimpleDelegate::CreateLambda([TextBlockPtr]()
				{
					TextBlockPtr->SetVisibility(EVisibility::Hidden);
				}));
				Button->SetOnUnhovered(FSimpleDelegate::CreateLambda([TextBlockPtr, TempName]()
				{
					TextBlockPtr->SetVisibility(EVisibility::Visible);
				}));

			
				ButtonContent->AddSlot()
				[
					Button
				];
			
				ColumnButtonWrap->AddSlot()
				[
					ButtonContent
				];

			});
		}
		else
		{
			ColumnButtonWrap->AddSlot()
			[
				SNullWidget::NullWidget
			];
		}
	
	}

	void SConditionCollectionViewWidget::OnModelChanged()
	{
		SetCanTick(true);
		Invalidate(EInvalidateWidgetReason::Layout);
	}
}

#undef LOCTEXT_NAMESPACE