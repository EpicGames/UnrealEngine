// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "QueryEditor/TedsQueryEditorModel.h"
#include "Styling/SlateTypes.h"
#include "Widgets/SCompoundWidget.h"

class SWrapBox;
class SHorizontalBox;

namespace UE::Editor::DataStorage::Debug::QueryEditor
{
	enum class EOperatorType : uint32;
	class FTedsQueryEditorModel;

	class SConditionCollectionViewWidget : public SCompoundWidget
	{
	public:
		SLATE_BEGIN_ARGS( SConditionCollectionViewWidget ){}
		SLATE_END_ARGS()

		~SConditionCollectionViewWidget() override;
		void Construct(const FArguments& InArgs, FTedsQueryEditorModel& InModel, EOperatorType InOperatorType);
		void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;
		
	private:

		void OnModelChanged();

		FTedsQueryEditorModel* Model = nullptr;
		EOperatorType OperatorType = EOperatorType::Unset;
		FDelegateHandle OnModelChangedDelegate;

		TSharedPtr<SWrapBox> ColumnButtonWrap;
		FButtonStyle ButtonStyle;
	};
}
