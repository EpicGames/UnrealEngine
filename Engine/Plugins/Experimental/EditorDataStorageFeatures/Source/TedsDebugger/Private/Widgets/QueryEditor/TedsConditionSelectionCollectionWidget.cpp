// Copyright Epic Games, Inc. All Rights Reserved.
#include "Widgets/QueryEditor/TedsConditionSelectionCollectionWidget.h"

namespace UE::Editor::DataStorage::Debug::QueryEditor
{
	void SConditionSelectionCollectionWidget::Construct(const FArguments& InArgs, FTedsQueryEditorModel& InModel)
	{
		Model = &InModel;
	}
} // namespace UE::Editor::DataStorage::Debug::QueryEditor
