// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "Widgets/SCompoundWidget.h"
#include "Templates/SharedPointerFwd.h"

namespace UE::Editor::DataStorage::Debug::QueryEditor
{
	class FTedsQueryEditorModel;
	
	/**
	 * Widget which shows and allows editing of columns/tabs within a given condition
	 */
	class SConditionSelectionCollectionWidget : public SCompoundWidget
	{
	public:
		SLATE_BEGIN_ARGS( SConditionSelectionCollectionWidget ){}
		SLATE_END_ARGS()

		void Construct(const FArguments& InArgs, FTedsQueryEditorModel& InModel);

	private:
		FTedsQueryEditorModel* Model = nullptr;
	};
} // namespace UE::Editor::DataStorage::Debug::QueryEditor
