// Copyright Epic Games, Inc. All Rights Reserved.
#include "TedsConditionSelectionComboWidget.h"

#include "QueryEditor/TedsQueryEditorModel.h"

#include "Algo/FindSortedStringCaseInsensitive.h"
#include "DataStorage/Debug/Log.h"
#include "Filters/SBasicFilterBar.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "TedsDebuggerComboWidget"

namespace UE::Editor::DataStorage::Debug::QueryEditor
{
	struct SConditionComboWidget::FComboItem
	{
		FConditionEntryHandle Handle;
		
		bool IsValid() const { return Handle.IsValid(); }
		void Reset() { Handle.Reset(); }
	};

	void SConditionComboWidget::OnConditionCollectionChanged()
	{		
		PopulateComboItems();
		FilterComboItems();
	}

	SConditionComboWidget::~SConditionComboWidget()
	{
		Model->GetModelChangedDelegate().Remove(ConditionCollectionChangedHandle);
	}

	void SConditionComboWidget::Construct(const FArguments& InArgs, FTedsQueryEditorModel& InModel, QueryEditor::EOperatorType InConditionType)
	{
		Model = &InModel;

		check(InConditionType != EOperatorType::Invalid);
		ConditionType = InConditionType;
		
		ConditionCollectionChangedHandle = Model->GetModelChangedDelegate().AddRaw(this, &SConditionComboWidget::OnConditionCollectionChanged);

		// This implementation is very similar to the existing SSearchableComboBox implementation, but since that only takes string types, a 
		// custom one was needed to utilize the custom FComboItem type. 
		ComboBoxMenuContent =
			SNew(SBox)
			.MinDesiredWidth(400.0f)
			.MaxDesiredHeight(500.0f)
			[
				SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				.AutoHeight()
				.HAlign(EHorizontalAlignment::HAlign_Fill)
				.Padding(4.0f, 2.0f)
				[
					SAssignNew(SearchBox, SSearchBox)
						.OnTextChanged(this, &SConditionComboWidget::OnSearchBoxTextChanged)
						.OnTextCommitted(this, &SConditionComboWidget::OnSearchBoxTextCommitted)
				]
				+SVerticalBox::Slot()
				.HAlign(EHorizontalAlignment::HAlign_Fill)
				[
					SAssignNew(ComboListView, SListView<TSharedPtr<FComboItem>>)
						.ListItemsSource(&FilteredComboItems)
						.OnSelectionChanged(this, &SConditionComboWidget::OnSelectionChanged)
						.OnGenerateRow(this, &SConditionComboWidget::GenerateMenuItemRow)
						.OnKeyDownHandler(this, &SConditionComboWidget::OnKeyDownHandler)
						.SelectionMode(ESelectionMode::Single)
				]
			];

		SComboButton::Construct(SComboButton::FArguments()
			.ComboButtonStyle(&FAppStyle::Get().GetWidgetStyle<FComboBoxStyle>("ComboBox").ComboButtonStyle)
			.ButtonContent()
			[
				SNew(STextBlock)
					.Text(FText::FromString("+"))
			]
			.MenuContent()
			[
				ComboBoxMenuContent.ToSharedRef()
			]
			.CollapseMenuOnParentFocus(true)
			.OnComboBoxOpened_Lambda([this]()
				{
					Model->RegenerateColumnsList();
					PopulateComboItems();
					SearchBox->SetText(FText::GetEmpty());
				})
		);
		SetMenuContentWidgetToFocus(SearchBox);

		PopulateComboItems();
	}

	void SConditionComboWidget::PopulateComboItems()
	{
		ComboItems.Reset();
		FilteredComboItems.Reset();
		Model->GenerateValidOperatorChoices(ConditionType, [this](const FTedsQueryEditorModel&, const QueryEditor::FConditionEntryHandle& Handle)
		{
			FComboItem ComboItem;
			ComboItem.Handle = Handle;
			ComboItems.Emplace(MakeShared<FComboItem>(ComboItem));
			
		});

		// Sort the ComboItems - this will make it only slightly easier to find them...
		Algo::Sort(ComboItems, [this](TSharedPtr<FComboItem>& A, TSharedPtr<FComboItem>& B)->bool
		{
			const UScriptStruct* AStruct = Model->GetColumnScriptStruct(A->Handle);
			const UScriptStruct* BStruct = Model->GetColumnScriptStruct(B->Handle);
			return AStruct->GetFName().Compare(BStruct->GetFName()) < 0;
		});
		
		FilteredComboItems.Append(ComboItems);
		ComboListView->RequestListRefresh();
	}

	void SConditionComboWidget::FilterComboItems()
	{
		// Filter the current combo items by the search text, since combo items are already sorted on population, these will not need to be sorted again
		FilteredComboItems.Reset();

		if (SearchText.IsEmpty())
		{
			FilteredComboItems.Append(ComboItems);
		}
		else
		{
			TArray<FString> SearchTokens;
			SearchText.ToString().ParseIntoArrayWS(SearchTokens);

			for (TSharedPtr<FComboItem> ComboItem : ComboItems)
			{
				if (const UScriptStruct* ColumnType = Model->GetColumnScriptStruct(ComboItem->Handle))
				{
					const FString ColumnName = ColumnType->GetName();

					bool bAllTokensMatch = true;
					for (const FString& SearchToken : SearchTokens)
					{
						if (ColumnName.IsEmpty() || !ColumnName.Contains(SearchToken, ESearchCase::Type::IgnoreCase))
						{
							bAllTokensMatch = false;
							break;
						}
					}

					if (bAllTokensMatch)
					{
						FilteredComboItems.Add(ComboItem);
					}
				}
			}
		}

		ComboListView->RequestListRefresh();
	}

	// Handle when the user presses enter to select an item that was navigated to via down/up keys
	FReply SConditionComboWidget::OnKeyDownHandler(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
	{
		if (InKeyEvent.GetKey() == EKeys::Enter && FilteredComboItems.Num() > 0)
		{
			if (ComboListView->GetNumItemsSelected() > 0)
			{

				SelectItem(ComboListView->GetSelectedItems()[0]);
			}
			else
			{
				SelectItem(FilteredComboItems[0]);
			}
			
			return FReply::Handled();
		}

		return FReply::Unhandled();
	}

	// Handle when the user clicks an item directly to select it
	void SConditionComboWidget::OnSelectionChanged(TSharedPtr<FComboItem> NewSelection, ESelectInfo::Type SelectInfo)
	{
		if (NewSelection && SelectInfo == ESelectInfo::OnMouseClick)
		{
			SelectItem(NewSelection);
		}
	}

	TSharedRef<SWidget> SConditionComboWidget::OnGenerateWidget(TSharedPtr<FComboItem> Item)
	{
		const UScriptStruct* ColumnType = Model->GetColumnScriptStruct(Item->Handle);
		const FText ColumnName = ColumnType ? FText::FromString(ColumnType->GetName()) : LOCTEXT("UnknownColumnName", "Unknown Column Type");

		return SNew(STextBlock)
			.Text(ColumnName);
	}

	TSharedRef<ITableRow> SConditionComboWidget::GenerateMenuItemRow(TSharedPtr<FComboItem> InItem, const TSharedRef<STableViewBase>& OwnerTable)
	{
		return SNew(SComboRow<TSharedPtr<FComboItem>>, OwnerTable)
			.Style(&FAppStyle::Get().GetWidgetStyle<FTableRowStyle>("ComboBox.Row"))
			.Padding(FMargin(8.0f, 3.f))
			[
				OnGenerateWidget(InItem)
			];
	}

	void SConditionComboWidget::OnSearchBoxTextChanged(const FText& InSearchText)
	{
		SearchText = InSearchText;
		FilterComboItems();
	}

	// Handles when the user presses enter while focused on the search box, it will select the first item in the list
	void SConditionComboWidget::OnSearchBoxTextCommitted(const FText& InSearchText, ETextCommit::Type InCommitType)
	{
		if ((InCommitType == ETextCommit::Type::OnEnter) && FilteredComboItems.Num() > 0)
		{
			SelectItem(FilteredComboItems[0]);
		}
	}

	void SConditionComboWidget::SelectItem(TSharedPtr<FComboItem> InItem)
	{
		EErrorCode ErrorCode = Model->SetOperatorType(InItem->Handle, ConditionType);
		if (ErrorCode != EErrorCode::Success)
		{
			UE_LOGF(LogEditorDataStorage, Error, "Could not set model condition: [%d]", static_cast<int>(ErrorCode));
		}
		ComboListView->ClearSelection();
	}

}

#undef LOCTEXT_NAMESPACE