// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "Filters/GenericFilter.h"
#include "Filters/SFilterBar.h"
#include "QueryEditor/TedsQueryEditorModel.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Views/SListView.h"

template<typename>
class SComboBox;

namespace UE::Editor::DataStorage::Debug::QueryEditor
{
	class FTedsQueryEditorModel;
	struct FConditionEntry;

	class SConditionComboWidget : public SComboButton
	{
	public:
		SLATE_BEGIN_ARGS( SConditionComboWidget ){}
		SLATE_END_ARGS()

		~SConditionComboWidget() override;
		void OnConditionCollectionChanged();
		void Construct(const FArguments& InArgs, FTedsQueryEditorModel& InModel, QueryEditor::EOperatorType InConditionType);

	private:
		struct FComboItem;

		void PopulateComboItems();
		void FilterComboItems();

		void OnSelectionChanged(TSharedPtr<FComboItem> NewSelection, ESelectInfo::Type SelectInfo);
		FReply OnKeyDownHandler(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent);
		TSharedRef<SWidget> OnGenerateWidget(TSharedPtr<FComboItem> Item);
		TSharedRef<ITableRow> GenerateMenuItemRow(TSharedPtr<FComboItem> InItem, const TSharedRef<STableViewBase>& OwnerTable);

		void OnSearchBoxTextChanged(const FText& InSearchText);
		void OnSearchBoxTextCommitted(const FText& InSearchText, ETextCommit::Type InCommitType);

		void SelectItem(TSharedPtr<FComboItem> InItem);

		FTedsQueryEditorModel* Model = nullptr;
		QueryEditor::EOperatorType ConditionType = QueryEditor::EOperatorType::Invalid;

		FDelegateHandle ConditionCollectionChangedHandle;

		/** Updated whenever search text is changed */
		FText SearchText;

		TArray<TSharedPtr<FComboItem>> ComboItems;
		TArray<TSharedPtr<FComboItem>> FilteredComboItems;

		/** Widget that gets passed as the combo box content containing the list and search bar */
		TSharedPtr<SWidget> ComboBoxMenuContent;

		/** The ListView that pops up with the available columns and tags */
		TSharedPtr<SListView<TSharedPtr<FComboItem>>> ComboListView;
		TSharedPtr<SSearchBox> SearchBox;

		TSharedPtr<SFilterBar<TSharedPtr<FComboItem>>> FilterThing;
	};
} // namespace UE::Editor::DataStorage::Debug::QueryEditor
