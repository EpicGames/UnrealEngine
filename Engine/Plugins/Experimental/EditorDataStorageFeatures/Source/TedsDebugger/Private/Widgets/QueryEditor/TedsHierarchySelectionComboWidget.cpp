// Copyright Epic Games, Inc. All Rights Reserved.

#include "TedsHierarchySelectionComboWidget.h"

#include "Elements/Interfaces/TypedElementDataStorageInterface.h"

namespace UE::Editor::DataStorage::Debug::QueryEditor
{
	void SHierarchyComboWidget::Construct(const FArguments& InArgs, FTedsQueryEditorModel& InModel)
	{
		Model = &InModel;

		GenerateHierarchyList();

		ChildSlot
		[
			SAssignNew(SearchableComboBox, SSearchableComboBox)
				.OptionsSource(&Hierarchies)
				.OnSelectionChanged_Lambda([this](TSharedPtr<FString> SelectedItem, ESelectInfo::Type SelectInfo)
				{
					if (!SelectedItem)
					{
						return;
					}
					if (const RelationTypeHandle* Handle = RelationHandleByDisplayName.Find(*SelectedItem))
					{
						Model->SetRelationType(*Handle);
					}
					else
					{
						Model->SetHierarchy(FName(*SelectedItem));
					}
				})
				.OnGenerateWidget_Lambda([](TSharedPtr<FString> InItem)
				{
					return SNew(STextBlock)
						.Text(FText::FromString(InItem.IsValid() ? *InItem : FString()));
				})
				.OnComboBoxOpening_Lambda([this]()
				{
					GenerateHierarchyList();
				})
				.Content()
				[
					SNew(STextBlock)
					.Text_Lambda([this]()
					{
						if (Model->HasRelationType())
						{
							const RelationTypeHandle SelectedHandle = Model->GetRelationType();
							for (const TTuple<FString, RelationTypeHandle>& Pair : RelationHandleByDisplayName)
							{
								if (Pair.Value == SelectedHandle)
								{
									return FText::FromString(Pair.Key);
								}
							}
						}
						return FText::FromName(Model->GetHierarchyName());
					})
				]
		];
	}

	void SHierarchyComboWidget::GenerateHierarchyList()
	{
		Hierarchies.Empty();
		RelationHandleByDisplayName.Empty();

		Model->GetTedsInterface().ListHierarchyNames([this](const FName& HierarchyName)
		{
			Hierarchies.Add(MakeShared<FString>(HierarchyName.ToString()));
		});

		Model->GetTedsInterface().ListRelationTypes([this](RelationTypeHandle Handle, const FName& Name)
		{
			const FTedsRelationTraits* Traits = Model->GetTedsInterface().GetRelationTypeTraits(Handle);
			if (Traits && Traits->HierarchyMode != EHierarchyMode::Disabled)
			{
				FString DisplayName = Name.ToString() + TEXT(" (relation)");
				RelationHandleByDisplayName.Add(DisplayName, Handle);
				Hierarchies.Add(MakeShared<FString>(MoveTemp(DisplayName)));
			}
		});
	}
}
