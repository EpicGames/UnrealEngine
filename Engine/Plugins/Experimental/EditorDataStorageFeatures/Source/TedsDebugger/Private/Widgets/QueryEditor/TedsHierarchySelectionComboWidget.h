// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/Input/SSearchableComboBox.h"
#include "QueryEditor/TedsQueryEditorModel.h"

namespace UE::Editor::DataStorage::Debug::QueryEditor
{
	class SHierarchyComboWidget : public SCompoundWidget
	{
	public:
		SLATE_BEGIN_ARGS( SHierarchyComboWidget ){}
		SLATE_END_ARGS()
		
		void Construct(const FArguments& InArgs, FTedsQueryEditorModel& InModel);

	protected:
		void GenerateHierarchyList();
		
	protected:
		TArray<TSharedPtr<FString>> Hierarchies;
		// Maps display name (e.g. "InheritsFrom (relation)") → RelationTypeHandle for entries added from ListRelationTypes.
		// Entries not present in this map are named hierarchies resolved via FindHierarchyByName.
		TMap<FString, RelationTypeHandle> RelationHandleByDisplayName;
		FTedsQueryEditorModel* Model = nullptr;
		TSharedPtr<SSearchableComboBox> SearchableComboBox;
	};
}
