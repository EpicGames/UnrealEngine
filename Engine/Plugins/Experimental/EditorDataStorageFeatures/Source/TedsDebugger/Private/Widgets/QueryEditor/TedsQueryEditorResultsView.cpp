// Copyright Epic Games, Inc. All Rights Reserved.

#include "TedsQueryEditorResultsView.h"

#include "SWarningOrErrorBox.h"
#include "TedsOutlinerModule.h"
#include "TedsTableViewerColumn.h"
#include "DataStorage/Features.h"
#include "DataStorage/Debug/Log.h"
#include "Elements/Framework/TypedElementQueryBuilder.h"
#include "Elements/Interfaces/TypedElementDataStorageInterface.h"
#include "Elements/Interfaces/TypedElementDataStorageUiInterface.h"
#include "Modules/ModuleManager.h"
#include "QueryEditor/TedsQueryEditorModel.h"
#include "TedsRowMonitorNode.h"
#include "TedsRowQueryResultsNode.h"
#include "TedsRowViewNode.h"
#include "Elements/Columns/TypedElementHiearchyColumns.h"
#include "TedsQueryNode.h"
#include "TedsRowSortNode.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/STedsHierarchyViewer.h"
#include "Widgets/STedsTableViewer.h"
#include "Widgets/SRowDetailsNavigator.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/STedsSearchBox.h"

#define LOCTEXT_NAMESPACE "TedsDebuggerModule"

namespace UE::Editor::DataStorage::Debug::QueryEditor
{
	SResultsView::~SResultsView()
	{
		Model->GetModelChangedDelegate().Remove(ModelChangedDelegateHandle);
		Model->OnTableViewerRefreshRequired().Remove(TableViewerRefreshRequiredDelegateHandle);

		if (CountQueryHandle != InvalidQueryHandle)
		{
			if (ICoreProvider* TedsInterface = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName))
			{
				TedsInterface->UnregisterQuery(CountQueryHandle);
			}
			CountQueryHandle = InvalidQueryHandle;
		}
	}

	void SResultsView::Construct(const FArguments& InArgs, FTedsQueryEditorModel& InModel)
	{
		using namespace UE::Editor::DataStorage::QueryStack;
		
		Model = &InModel;
		ModelChangedDelegateHandle = Model->GetModelChangedDelegate().AddRaw(this, &SResultsView::OnModelChanged);
		TableViewerRefreshRequiredDelegateHandle = Model->OnTableViewerRefreshRequired().AddRaw(this, &SResultsView::OnTableViewerRefreshRequired);
		
		UiProvider = GetMutableDataStorageFeature<IUiProvider>(UiFeatureName);
		checkf(UiProvider, TEXT("Cannot create SResultsView without TEDS UI"));

		// Create a custom column for the table viewer to display row handles
		CreateRowHandleColumn();
	
		QueryNode = MakeShared<FQueryNode>(Model->GetTedsInterface());
		
		ChildSlot
		[
			SNew(SVerticalBox)
			+SVerticalBox::Slot()
			[
				SNew(SSplitter)
				+SSplitter::Slot()
				.Value(0.5f)
				[
					SNew(SVerticalBox)
					+SVerticalBox::Slot()
					.AutoHeight()
					[
						SAssignNew(SearchBoxSlot, SBorder)
						[
							CreateSearchBox()
						]
					]
					+SVerticalBox::Slot()
					.FillContentHeight(1.0f)
					[
						SAssignNew(TableViewerSlot, SBorder)
						[
							CreateTableViewer()
						]
					]
				]
				+SSplitter::Slot()
				.Value(0.5f)
				[
					SAssignNew(RowDetailsNavigator, SRowDetailsNavigator)
				]
			]
			+SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(STextBlock)
				.Text_Lambda([this]()
				{
					ICoreProvider& TedsInterface = Model->GetTedsInterface();
					FQueryResult QueryResult = TedsInterface.RunQuery(CountQueryHandle);
					if (QueryResult.Completed == FQueryResult::ECompletion::Fully)
					{
						FString String = FString::Printf(TEXT("Element Count: %u"), QueryResult.Count.load());
						return FText::FromString(String);
					}
					else
					{
						return FText::FromString(TEXT("Invalid query"));
					}
				})
			]
		];

		RowQueryStack = MakeShared<FRowMonitorNode>(Model->GetTedsInterface(), RowQueryStack, QueryNode);

		if(RowHandleColumn)
		{
			TableViewer->AddCustomRowWidget(RowHandleColumn.ToSharedRef());
		}
	}

	void SResultsView::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
	{
		using namespace UE::Editor::DataStorage::Queries;

		ICoreProvider& TedsInterface = Model->GetTedsInterface();

		if(bModelDirty)
		{
			{
				FQueryDescription CountQueryDescription = Model->GenerateNoSelectQueryDescription();
	
				if (CountQueryHandle != InvalidQueryHandle)
				{
					TedsInterface.UnregisterQuery(CountQueryHandle);
					CountQueryHandle = InvalidQueryHandle;
				}
				CountQueryHandle = TedsInterface.RegisterQuery(MoveTemp(CountQueryDescription));
			}

			{
				FQueryDescription TableViewerQueryDescription = Model->GenerateQueryDescription();

				RowHandle GeneralPurposeRowHandle = UiProvider->FindPurpose(UiProvider->GetGeneralWidgetPurposeID());
				RowHandle DefaultPurposeRowHandle = UiProvider->FindPurpose(UiProvider->GetDefaultWidgetPurposeID());

				// Temporarily add the default purpose as a parent of the general purpose so the debugger can support both
				TedsInterface.SetParentRow(UiProvider->GetWidgetPurposeHierarchyHandle(), GeneralPurposeRowHandle, DefaultPurposeRowHandle);

				// Update the columns in the table viewer using the selection types from the query description
				TableViewer->SetColumns(TArray<TWeakObjectPtr<const UScriptStruct>>(TableViewerQueryDescription.SelectionTypes));

				// Remove the parenting chain after we have used it to generate widgets
				TedsInterface.SetParentRow(UiProvider->GetWidgetPurposeHierarchyHandle(), GeneralPurposeRowHandle, InvalidRowHandle);

				// Since setting the columns clears all columns, we are going to re-add the custom column back
				if(RowHandleColumn)
				{
					TableViewer->AddCustomRowWidget(RowHandleColumn.ToSharedRef());
				}
		
				// Mass doesn't like empty queries, so we only set it if there are actual conditions
				if(TableViewerQueryDescription.ConditionTypes.Num() || TableViewerQueryDescription.SelectionTypes.Num())
				{
					QueryNode->SetQuery(MoveTemp(TableViewerQueryDescription));
				}
				else
				{
					QueryNode->ClearQuery();
				}
			}
	
			bModelDirty = false;
		}
	}

	ETableViewMode::Type SResultsView::GetViewMode() const
	{
		return ResultsViewMode;
	}

	void SResultsView::SetViewMode(ETableViewMode::Type InViewMode)
	{
		ResultsViewMode = InViewMode;
		TableViewerSlot->SetContent(CreateTableViewer()); // Regenerate the table viewer widget
		SearchBoxSlot->SetContent(CreateSearchBox()); // Regenerate the search box since the sync flag depends on the view mode
	}

	void SResultsView::OnModelChanged()
	{
		bModelDirty = true;
		Invalidate(EInvalidateWidgetReason::Layout);
	}

	void SResultsView::OnTableViewerRefreshRequired()
	{
		if (!Model->HasRelationType())
		{
			HierarchyHandle = Model->GetHierarchy();
		}
		TableViewerSlot->SetContent(CreateTableViewer());
	}

	void SResultsView::CreateRowHandleColumn()
	{
		auto AssignWidgetToColumn = [this](IUiProvider::FWidgetConstructorPtr WidgetConstructor, TConstArrayView<TWeakObjectPtr<const UScriptStruct>>)
		{
			RowHandleColumn = MakeShared<FTedsTableViewerColumn>(TEXT("Row Handle"), WidgetConstructor);
			return false;
		};

		UiProvider->CreateWidgetConstructors(
			UiProvider->FindPurpose(IUiProvider::FPurposeInfo("General", "Cell", "RowHandle").GeneratePurposeID()),
			FMetaDataView(), AssignWidgetToColumn);
	}

	TSharedRef<SWidget> SResultsView::CreateTableViewer()
	{
		TSharedPtr<FMetaData> DebuggerMetaData = MakeShared<FMetaData>();
		DebuggerMetaData->AddOrSetMutableData("bUseDisplayNameTooltip", false);

		switch (ResultsViewMode)
		{
		case ETableViewMode::List:
			TableViewer = SNew(STedsTableViewer)
					.QueryStack(RowQueryStack)
					.GenericMetaData(DebuggerMetaData)
					.ShowHiddenRows(Model->GetShowHiddenRows())
					.EmptyRowsMessage(LOCTEXT("EmptyRowsMessage", "The provided query has no results."))
					.OnSelectionChanged(STedsTableViewer::FOnSelectionChanged::CreateLambda(
						[this](RowHandle SelectedRow, ESelectInfo::Type SelectInfo)
							{
								if(RowDetailsNavigator)
								{
									if(SelectedRow != InvalidRowHandle)
									{
										RowDetailsNavigator->SetRow(SelectedRow);
									}
									else
									{
										RowDetailsNavigator->ClearRow();
									}
								}
							}));
			break;
		
		case ETableViewMode::Tree:

			if (Model->HasRelationType())
			{
				HierarchyData = MakeShared<FRelationTypeHierarchyViewerData>(Model->GetRelationType());
			}
			else
			{
				HierarchyData = MakeShared<FHierarchyViewerData>(Model->GetTedsInterface(), HierarchyHandle);
			}

			TableViewer =
				SNew(SHierarchyViewer, HierarchyData)
					.AllNodeProvider(RowQueryStack)
					.GenericMetaData(DebuggerMetaData)
					.ShowHiddenRows(Model->GetShowHiddenRows())
					.EmptyRowsMessage(LOCTEXT("EmptyRowsMessage", "The provided query has no results."))
					.OnSelectionChanged(STedsTableViewer::FOnSelectionChanged::CreateLambda(
						[this](RowHandle SelectedRow, ESelectInfo::Type SelectInfo)
							{
								if(RowDetailsNavigator)
								{
									if(SelectedRow != InvalidRowHandle)
									{
										RowDetailsNavigator->SetRow(SelectedRow);
									}
									else
									{
										RowDetailsNavigator->ClearRow();
									}
								}
							}));
			break;

			// NYI
		case ETableViewMode::Tile:
			[[fallthrough]];
		default:
			checkf(false, TEXT("Unknown view mode requested! Make sure the options in the UI to select view modes matches the implementation")); 
			return SNullWidget::NullWidget;
		}

		if (RowDetailsNavigator)
		{
			RowDetailsNavigator->ClearRow();
		}

		bModelDirty = true; // We need to set the columns again
		
		return TableViewer->AsWidget();
	}

	TSharedRef<SWidget> SResultsView::CreateSearchBox()
	{
		// We cannot detect hierarchy changes by checking the cached state since they are data changes, so we use force refresh when viewing hierarchies
		STedsSearchBox::ESearchableQueryFlags QueryFlags =
			(ResultsViewMode == ETableViewMode::Tree)
			? STedsSearchBox::ESearchableQueryFlags::ForceRefreshOnUpdate 
			: STedsSearchBox::ESearchableQueryFlags::RefreshOnUpdate;
		
		return SNew(STedsSearchBox)
			.InSearchableQueryNode(QueryNode)
			.InSearchableQueryFlags(QueryFlags)
			.OutSearchNode(&RowQueryStack);
	}
} // namespace UE::Editor::DataStorage::Debug::QueryEditor

#undef LOCTEXT_NAMESPACE
