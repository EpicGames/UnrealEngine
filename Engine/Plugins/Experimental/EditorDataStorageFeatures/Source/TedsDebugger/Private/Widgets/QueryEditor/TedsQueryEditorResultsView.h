// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "HierarchyViewerIntefaces.h"
#include "Elements/Interfaces/TypedElementDataStorageInterface.h"
#include "Framework/Views/ITypedTableView.h"
#include "Widgets/SCompoundWidget.h"


class ISceneOutliner;
class SSceneOutliner;
class SHorizontalBox;
struct FTypedElementWidgetConstructor;
class SBorder;

namespace UE::Editor::DataStorage
{
	class ITableViewer;
	class STedsTableViewer;
	class FTedsTableViewerColumn;
	class SRowDetailsNavigator;
	class IUiProvider;

	namespace QueryStack
	{
		class FTopLevelRowsNode;
		class FQueryNode;
		class IRowNode;
	}

	namespace Debug::QueryEditor
	{
		class FTedsQueryEditorModel;

		class SResultsView : public SCompoundWidget
		{
		public:
			SLATE_BEGIN_ARGS( SResultsView ){}
			SLATE_END_ARGS()

			~SResultsView() override;
			void Construct(const FArguments& InArgs, FTedsQueryEditorModel& InModel);
			void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

			ETableViewMode::Type GetViewMode() const;
			void SetViewMode(ETableViewMode::Type InViewMode);

		private:

			void OnModelChanged();
			void OnTableViewerRefreshRequired();
			void CreateRowHandleColumn();
			TSharedRef<SWidget> CreateTableViewer();
			TSharedRef<SWidget> CreateSearchBox();
		
			FTedsQueryEditorModel* Model = nullptr;
			FDelegateHandle ModelChangedDelegateHandle;
			FDelegateHandle TableViewerRefreshRequiredDelegateHandle;
			bool bModelDirty = true;


			QueryHandle CountQueryHandle = InvalidQueryHandle;
			
			TArray<RowHandle> TableViewerRows;
			TSharedPtr<ITableViewer> TableViewer;
			TSharedPtr<QueryStack::FQueryNode> QueryNode;
			TSharedPtr<QueryStack::IRowNode> RowQueryStack;

			// Hierarchy Data used by SHierarchyView to resolve parenting.
			// May be FHierarchyViewerData (named hierarchy) or FRelationTypeHierarchyViewerData (raw hierarchical relation).
			TSharedPtr<IHierarchyViewerDataInterface> HierarchyData;
			// The current named hierarchy the widget is viewing (only valid when a named hierarchy is selected)
			FHierarchyHandle HierarchyHandle;
			
			// Custom column for the table viewer to display row handles
			TSharedPtr<FTedsTableViewerColumn> RowHandleColumn;

			// Breadcrumb-navigable stack of row detail panels
			TSharedPtr<SRowDetailsNavigator> RowDetailsNavigator;
			
			IUiProvider* UiProvider = nullptr;

			// Border inside which the table viewer is stored so we can re-create it when needed
			TSharedPtr<SBorder> TableViewerSlot;

			// Border inside which the search box is stored so we can re-create it when needed
			TSharedPtr<SBorder> SearchBoxSlot;

			ETableViewMode::Type ResultsViewMode = ETableViewMode::List;
		};

	} // namespace Debug::QueryEditor
} // namespace UE::Editor::DataStorage
