// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/RowReferenceWidget.h"

#include "Columns/TedsOutlinerColumns.h"
#include "DataStorage/Features.h"
#include "Elements/Columns/TypedElementHiearchyColumns.h"
#include "Elements/Columns/TypedElementLabelColumns.h"
#include "Elements/Columns/TypedElementMiscColumns.h"
#include "Elements/Interfaces/TypedElementDataStorageInterface.h"
#include "ISceneOutliner.h"
#include "Modules/ModuleManager.h"
#include "TedsDebuggerModule.h"
#include "Elements/Columns/TypedElementSlateWidgetColumns.h"
#include "Elements/Framework/TypedElementAttributeBinding.h"
#include "Columns/SceneOutlinerColumns.h"
#include "Widgets/Input/SHyperlink.h"
#include "Widgets/Layout/SBox.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RowReferenceWidget)

#define LOCTEXT_NAMESPACE "RowReferenceWidget"

namespace UE::Editor::DataStorage::Debug::Private
{
	void OnNavigateHyperlink(ICoreProvider* DataStorage,  RowHandle TargetRowHandle, RowHandle UiRowHandle)
	{
		const FSceneOutlinerColumn* TedsOutlinerColumn = DataStorage->GetColumn<FSceneOutlinerColumn>(UiRowHandle);

		if(!TedsOutlinerColumn)
		{
			return;
		}
		
		TSharedPtr<ISceneOutliner> OwningTableViewer = TedsOutlinerColumn->Outliner.Pin();
		
		if(!OwningTableViewer)
		{
			return;
		}

		// If the item was found in this table viewer, select it and navigate to it
		if(FSceneOutlinerTreeItemPtr TreeItem = OwningTableViewer->GetTreeItem(TargetRowHandle))
		{
			OwningTableViewer->SetSelection([TreeItem](ISceneOutlinerTreeItem& Item)
			{
				return Item.GetID() == TreeItem->GetID();
			});
			
			OwningTableViewer->FrameSelectedItems();

			return;
		}
	}

} // namespace UE::Editor::DataStorage::Debug::Private

URowReferenceWidgetFactory::~URowReferenceWidgetFactory()
{
}

void URowReferenceWidgetFactory::RegisterWidgetConstructors(UE::Editor::DataStorage::ICoreProvider& DataStorage,
	UE::Editor::DataStorage::IUiProvider& DataStorageUi) const
{
	using namespace UE::Editor::DataStorage::Queries;

	// TEDS UI TODO: We can re-use this widget for FTypedElementParentColumn
	DataStorageUi.RegisterWidgetFactory<FRowReferenceWidgetConstructor>(
		DataStorageUi.FindPurpose(IUiProvider::FPurposeInfo("SceneOutliner", "Cell", NAME_None).GeneratePurposeID()),
		TColumn<FTypedElementRowReferenceColumn>());

	DataStorageUi.RegisterWidgetFactory<FRowReferenceWidgetConstructor>(
		DataStorageUi.FindPurpose(DataStorageUi.GetGeneralWidgetPurposeID()),
		TColumn<FTypedElementRowReferenceColumn>());
}

FRowReferenceWidgetConstructor::FRowReferenceWidgetConstructor()
	: Super(FRowReferenceWidgetConstructor::StaticStruct())
{
}

TSharedPtr<SWidget> FRowReferenceWidgetConstructor::CreateWidget(UE::Editor::DataStorage::ICoreProvider* DataStorage,
	UE::Editor::DataStorage::IUiProvider* DataStorageUi,
	UE::Editor::DataStorage::RowHandle TargetRow, UE::Editor::DataStorage::RowHandle WidgetRow,
	const UE::Editor::DataStorage::FMetaDataView& Arguments)
{
	using namespace UE::Editor::DataStorage;
	
	FAttributeBinder Binder(TargetRow, DataStorage);
	
	FNumberFormattingOptions NumberFormattingOptions;
	NumberFormattingOptions.SetUseGrouping(false);
	
	return SNew(SBox)
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
		[
			SNew(SHyperlink)
					.Text(Binder.BindData(&FTypedElementRowReferenceColumn::Row, [NumberFormattingOptions] (const RowHandle& Row)
					{
						return FText::AsNumber(Row, &NumberFormattingOptions);
					}))
					.Style(FAppStyle::Get(), "Common.GotoBlueprintHyperlink")
					.ToolTipText(Binder.BindData(&FTypedElementRowReferenceColumn::Row, [DataStorage] (const RowHandle& Row)
					{
						if (FTypedElementLabelColumn* LabelColumn = DataStorage->GetColumn<FTypedElementLabelColumn>(Row))
						{
							return FText::FromString(LabelColumn->Label);
						}

						return FText::GetEmpty();
					}))
					.OnNavigate(FSimpleDelegate::CreateStatic(&Debug::Private::OnNavigateHyperlink, DataStorage, TargetRow, WidgetRow))
		];
}

#undef LOCTEXT_NAMESPACE
