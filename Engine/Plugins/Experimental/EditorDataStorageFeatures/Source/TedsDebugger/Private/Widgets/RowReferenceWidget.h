// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/Interfaces/TypedElementDataStorageFactory.h"
#include "Elements/Interfaces/TypedElementDataStorageUiInterface.h"

#include "RowReferenceWidget.generated.h"

class SWidget;

/*
 * Widget for the TEDS Debugger that visualizes a reference to another row
 */
UCLASS()
class URowReferenceWidgetFactory : public UEditorDataStorageFactory
{
	GENERATED_BODY()

public:
	~URowReferenceWidgetFactory() override;

	void RegisterWidgetConstructors(UE::Editor::DataStorage::ICoreProvider& DataStorage,
		UE::Editor::DataStorage::IUiProvider& DataStorageUi) const override;
};

USTRUCT()
struct FRowReferenceWidgetConstructor : public FTypedElementWidgetConstructor
{
	GENERATED_BODY()

public:
	FRowReferenceWidgetConstructor();
	~FRowReferenceWidgetConstructor() override = default;

	virtual TSharedPtr<SWidget> CreateWidget(
		UE::Editor::DataStorage::ICoreProvider* DataStorage,
		UE::Editor::DataStorage::IUiProvider* DataStorageUi,
		UE::Editor::DataStorage::RowHandle TargetRow,
		UE::Editor::DataStorage::RowHandle WidgetRow,
		const UE::Editor::DataStorage::FMetaDataView& Arguments) override;
};