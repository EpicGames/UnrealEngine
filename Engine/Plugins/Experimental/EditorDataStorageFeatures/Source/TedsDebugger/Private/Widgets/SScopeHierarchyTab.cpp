// Copyright Epic Games, Inc. All Rights Reserved.

#include "SScopeHierarchyTab.h"

#include "ScopeRowLabelWidget.h"
#include "DataStorage/Scope/EditorDataScope.h"
#include "DataStorage/Scope/EditorDataScopeColumns.h"
#include "Elements/Framework/TypedElementQueryBuilder.h"
#include "Elements/Columns/TypedElementMiscColumns.h"
#include "Elements/Columns/TypedElementTypeInfoColumns.h"
#include "Elements/Interfaces/TypedElementDataStorageInterface.h"
#include "Elements/Interfaces/TypedElementDataStorageUiInterface.h"
#include "HAL/PlatformApplicationMisc.h"
#include "HAL/PlatformStackWalk.h"
#include "TedsTableViewerColumn.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SMultiLineEditableTextBox.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Layout/SSplitter.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/STedsHierarchyViewer.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/SHeaderRow.h"

#define LOCTEXT_NAMESPACE "SScopeHierarchyTab"

namespace
{
	const FName EditorWidgetTableName(TEXT("Editor_WidgetTable"));
}

namespace UE::Editor::DataStorage::Debug
{

void SScopeHierarchyTab::Construct(const FArguments& InArgs)
{
	Storage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName);
	checkf(Storage, TEXT("Cannot create the Scope Hierarchy Tab without TEDS Core"));
	DataStorageUi = GetMutableDataStorageFeature<IUiProvider>(UiFeatureName);

	ChildSlot
	[
		SNew(SBorder)
		.BorderImage(FAppStyle::GetBrush("Brushes.Background"))
		[
			SNew(SSplitter)
			+SSplitter::Slot()
			.Value(0.5f)
			[
				CreateHierarchyViewer()
			]
			+SSplitter::Slot()
			.Value(0.5f)
			[
				CreateDetailsPanel()
			]
		]
	];
}

TSharedRef<SWidget> SScopeHierarchyTab::CreateHierarchyViewer()
{
	// Build a query matching all scope rows (by tag) and selecting FScopeRowSourceInfo
	FQueryDescription QueryDesc = Queries::Select()
		.ReadOnly<FScopeRowSourceInfo>()
		.Where()
		.All<FDataStorageScopeDataTag>()
		.Compile();

	ContextQueryNode = MakeShared<QueryStack::FQueryNode>(*Storage, MoveTemp(QueryDesc));
	ScopeRowsNode = MakeShared<QueryStack::FRowQueryResultsNode>(
		*Storage, ContextQueryNode, QueryStack::FRowQueryResultsNode::ESyncActions::RefreshOnUpdate);
	ScopeRowsNode->Refresh();

	// Create hierarchy data wrapping the editor scope hierarchy
	FHierarchyHandle HierarchyHandle = Storage->GetScopeHierarchy();
	if (Storage->IsValidHierarchyHandle(HierarchyHandle))
	{
		HierarchyData = MakeShared<FHierarchyViewerData>(*Storage, HierarchyHandle);
	}

	HierarchyViewer = SNew(SHierarchyViewer, HierarchyData)
		.TableViewerIdentifier(TEXT("ScopeHierarchyViewer"))
		.AllNodeProvider(ScopeRowsNode)
		.Columns({})
		.ExpandNewRows(true)
		.EmptyRowsMessage(LOCTEXT("EmptyRowsMessage", "No scope rows found. Is TEDS initialized?"))
		.OnSelectionChanged(
			SHierarchyViewer::FOnSelectionChanged::CreateSP(this, &SScopeHierarchyTab::OnHierarchySelectionChanged));

	// Add a label display column — shows FTypedElementLabelColumn text when available,
	// falls back to the stringified RowHandle otherwise.
	{
		IUiProvider::FWidgetConstructorPtr LabelConstructor = MakeShared<FScopeRowLabelWidgetConstructor>();
		TSharedRef<FTedsTableViewerColumn> LabelColumn =
			MakeShared<FTedsTableViewerColumn>(TEXT("Scope Row"), LabelConstructor);
		HierarchyViewer->AddCustomRowWidget(LabelColumn);
	}

	return HierarchyViewer.ToSharedRef();
}

TSharedRef<SWidget> SScopeHierarchyTab::CreateDetailsPanel()
{
	return SNew(SVerticalBox)
		// Callstack header + copy button
		+SVerticalBox::Slot()
		.AutoHeight()
		.Padding(4.f)
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.FillWidth(1.f)
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("CallstackHeader", "Creation Callstack"))
				.Font(FAppStyle::GetFontStyle("BoldFont"))
			]
			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SButton)
				.ButtonStyle(FAppStyle::Get(), "SimpleButton")
				.ToolTipText(LOCTEXT("CopyCallstackTooltip", "Copy callstack to clipboard"))
				.OnClicked(this, &SScopeHierarchyTab::CopyCallstackToClipboard)
				[
					SNew(SImage)
					.Image(FAppStyle::GetBrush("Icons.Clipboard"))
					.ColorAndOpacity(FSlateColor::UseForeground())
				]
			]
		]
		// Callstack text (read-only, selectable, no wrap, horizontal scroll)
		+SVerticalBox::Slot()
		.FillHeight(0.5f)
		.Padding(4.f)
		[
			SAssignNew(CallstackTextBox, SMultiLineEditableTextBox)
			.IsReadOnly(true)
			.Text(LOCTEXT("NoSelection", "Select a scope row to view details."))
			.AutoWrapText(false)
			.Font(FCoreStyle::GetDefaultFontStyle("Mono", 9))
			.AlwaysShowScrollbars(true)
		]
		// Visible columns header
		+SVerticalBox::Slot()
		.AutoHeight()
		.Padding(4.f)
		[
			SNew(STextBlock)
			.Text(LOCTEXT("VisibleColumnsHeader", "Visible Columns"))
			.Font(FAppStyle::GetFontStyle("BoldFont"))
		]
		// Visible columns list
		+SVerticalBox::Slot()
		.FillHeight(0.5f)
		.Padding(4.f)
		[
			SAssignNew(ColumnListView, SListView<FScopeColumnItemPtr>)
			.ListItemsSource(&ColumnItems)
			.OnGenerateRow(this, &SScopeHierarchyTab::MakeColumnListRow)
			.HeaderRow(
				SNew(SHeaderRow)
				+SHeaderRow::Column(TEXT("Origin"))
					.DefaultLabel(FText())
					.FixedWidth(24.f)
				+SHeaderRow::Column(TEXT("ColumnType"))
					.DefaultLabel(LOCTEXT("ColumnTypeHeader", "Column"))
					.FillWidth(0.3f)
				+SHeaderRow::Column(TEXT("Value"))
					.DefaultLabel(LOCTEXT("ValueHeader", "Value"))
					.FillWidth(0.7f)
			)
		];
}

void SScopeHierarchyTab::OnHierarchySelectionChanged(RowHandle Row, ESelectInfo::Type SelectInfo)
{
	if (Storage->IsRowAvailable(Row))
	{
		RebuildDetailsForRow(Row);
	}
	else
	{
		CachedCallstackString.Empty();
		CallstackTextBox->SetText(LOCTEXT("NoSelection", "Select a scope row to view details."));
		ColumnItems.Empty();
		ColumnListView->RequestListRefresh();
	}
}

void SScopeHierarchyTab::RebuildDetailsForRow(RowHandle Row)
{
	// Resolve callstack (skip first frame — it's always the CaptureStackBackTrace call)
	CachedCallstackString.Empty();
	if (const FScopeRowSourceInfo* SourceInfo =
			Storage->GetScopeData<FScopeRowSourceInfo>(Row))
	{
		ANSICHAR LineBuffer[1024];
		for (int32 i = 1; i < SourceInfo->CallstackDepth; ++i)
		{
			LineBuffer[0] = 0;
			FPlatformStackWalk::ProgramCounterToHumanReadableString(
				i, SourceInfo->Callstack[i], LineBuffer, UE_ARRAY_COUNT(LineBuffer));
			if (CachedCallstackString.Len() > 0)
			{
				CachedCallstackString += TEXT("\n");
			}
			CachedCallstackString += ANSI_TO_TCHAR(LineBuffer);
		}
		CallstackTextBox->SetText(FText::FromString(CachedCallstackString));
	}
	else
	{
		CallstackTextBox->SetText(LOCTEXT("NoCallstack", "No callstack info available for this row."));
	}

	// Rebuild visible columns
	ColumnItems.Empty();
	TArray<const UScriptStruct*> VisibleColumns = Storage->GetAllVisibleScopeColumns(Row);
	for (const UScriptStruct* ColumnType : VisibleColumns)
	{
		auto Item = MakeShared<FScopeColumnItem>();
		Item->TypeName = ColumnType ? ColumnType->GetName() : TEXT("(null)");
		Item->OwnerRow = Row;
		Item->bInherited = !Storage->HasColumns(Row, TConstArrayView<const UScriptStruct*>(&ColumnType, 1));
		Item->ColumnType = ColumnType;

		if (DataStorageUi && ColumnType)
		{
			DataStorageUi->CreateWidgetConstructors(
				DataStorageUi->FindPurpose(DataStorageUi->GetDefaultWidgetPurposeID()),
				FMetaDataView(),
				[&Item](IUiProvider::FWidgetConstructorPtr Constructor, TConstArrayView<TWeakObjectPtr<const UScriptStruct>>)
				{
					Item->WidgetConstructor = MoveTemp(Constructor);
					return false;
				});
		}

		ColumnItems.Add(Item);
	}
	ColumnListView->RequestListRefresh();
}

FReply SScopeHierarchyTab::CopyCallstackToClipboard()
{
	if (!CachedCallstackString.IsEmpty())
	{
		FPlatformApplicationMisc::ClipboardCopy(*CachedCallstackString);
	}
	return FReply::Handled();
}

TSharedRef<ITableRow> SScopeHierarchyTab::MakeColumnListRow(
	FScopeColumnItemPtr Item, const TSharedRef<STableViewBase>& OwnerTable)
{
	class SColumnRow : public SMultiColumnTableRow<FScopeColumnItemPtr>
	{
	public:
		SLATE_BEGIN_ARGS(SColumnRow) {}
			SLATE_ARGUMENT(FScopeColumnItemPtr, Item)
			SLATE_ARGUMENT(ICoreProvider*, DataStorage)
			SLATE_ARGUMENT(IUiProvider*, DataStorageUi)
		SLATE_END_ARGS()

		void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTable)
		{
			RowItem = InArgs._Item;
			DataStorage = InArgs._DataStorage;
			DataStorageUi = InArgs._DataStorageUi;
			SMultiColumnTableRow::Construct(FSuperRowType::FArguments(), InOwnerTable);
		}

		TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override
		{
			if (ColumnName == TEXT("Origin"))
			{
				const FName BrushName = RowItem->bInherited
					? TEXT("Icons.ArrowDown")
					: TEXT("Icons.FilledCircle");

				const FText Tooltip = RowItem->bInherited
					? FText::Format(LOCTEXT("InheritedTooltip", "Inherited from row {0}"),
						FText::FromString(LexToString(RowItem->OwnerRow)))
					: LOCTEXT("OwnTooltip", "Owned by this row");

				return SNew(SBox)
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Center)
					.WidthOverride(16.f)
					.HeightOverride(16.f)
					[
						SNew(SImage)
						.Image(FAppStyle::GetBrush(BrushName))
						.ColorAndOpacity(RowItem->bInherited
							? FSlateColor(FLinearColor(0.5f, 0.5f, 0.8f))
							: FSlateColor(FLinearColor(0.2f, 0.8f, 0.2f)))
						.ToolTipText(Tooltip)
					];
			}
			if (ColumnName == TEXT("ColumnType"))
			{
				return SNew(STextBlock).Text(FText::FromString(RowItem->TypeName));
			}
			if (ColumnName == TEXT("Value"))
			{
				return SNew(STextBlock)
					.Text_Lambda([WeakThis = TWeakPtr<SColumnRow>(StaticCastSharedRef<SColumnRow>(this->AsShared()))]()
						{
							if (TSharedPtr<SColumnRow> ColumnRowWidget = WeakThis.Pin())
							{
								if (!ColumnRowWidget->RowItem.IsValid())
								{
									return LOCTEXT("ColumnNotFoundTextEarly", "Column not found on row");
								}
								ICoreProvider* DataStorage = ColumnRowWidget->DataStorage;
								RowHandle ScopeRow = ColumnRowWidget->RowItem->OwnerRow;
								if (!DataStorage || !DataStorage->IsRowAvailable(ScopeRow))
								{
									return LOCTEXT("ColumnNotFoundTextEarly", "Column not found on row");
								}

								TStrongObjectPtr<const UScriptStruct> TypeInfo = ColumnRowWidget->RowItem->ColumnType.Pin();
								if (!TypeInfo)
								{
									return LOCTEXT("ColumnNotFoundTextEarly", "Column not found on row");
								}
								
								// Tags have no data — show blank
								if (TypeInfo->IsChildOf(FEditorDataStorageTag::StaticStruct()))
								{
									return FText::GetEmpty();
								}

								const void* Data = DataStorage->GetScopeDataRaw(ScopeRow, TypeInfo.Get());
								if (!Data)
								{
									return LOCTEXT("ColumnNotFoundTextEarly", "Column not found on row");
								}

								FString Label;
								TypeInfo->ExportText(Label, Data, Data, nullptr, PPF_None, nullptr);
								return FText::FromString(Label);
								
							}
							return LOCTEXT("ColumnNotFoundTextEarly", "Column not found on row");
						});
			}
			return SNullWidget::NullWidget;
		}

	private:
		FScopeColumnItemPtr RowItem;
		ICoreProvider* DataStorage = nullptr;
		IUiProvider* DataStorageUi = nullptr;
	};

	return SNew(SColumnRow, OwnerTable).Item(Item).DataStorage(Storage).DataStorageUi(DataStorageUi);
}

} // namespace UE::Editor::DataStorage::Debug

#undef LOCTEXT_NAMESPACE
