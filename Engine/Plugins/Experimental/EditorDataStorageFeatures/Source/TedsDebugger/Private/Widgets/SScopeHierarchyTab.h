// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "DataStorage/Scope/EditorDataScope.h"
#include "DataStorage/Features.h"
#include "Elements/Interfaces/TypedElementDataStorageUiInterface.h"
#include "HierarchyViewerIntefaces.h"
#include "TedsQueryNode.h"
#include "TedsRowQueryResultsNode.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/SListView.h"

class SMultiLineEditableTextBox;
class STextBlock;

namespace UE::Editor::DataStorage
{
	class SHierarchyViewer;
}

namespace UE::Editor::DataStorage::Debug
{
	/** Item for the visible-columns list view on the details side. */
	struct FScopeColumnItem
	{
		FString TypeName;
		RowHandle OwnerRow = InvalidRowHandle;
		bool bInherited = false;
		TWeakObjectPtr<const UScriptStruct> ColumnType;
		IUiProvider::FWidgetConstructorPtr WidgetConstructor;
		RowHandle WidgetRow = InvalidRowHandle;
	};
	using FScopeColumnItemPtr = TSharedPtr<FScopeColumnItem>;

	/**
	 * Debugger tab showing the Editor Scope Tree row hierarchy.
	 * Left: SHierarchyViewer tree of all scope rows.
	 * Right: On selection, resolved creation callstack + visible columns list.
	 */
	class SScopeHierarchyTab final : public SCompoundWidget
	{
	public:
		SLATE_BEGIN_ARGS(SScopeHierarchyTab) {}
		SLATE_END_ARGS()

		void Construct(const FArguments& InArgs);

	private:
		TSharedRef<SWidget> CreateHierarchyViewer();
		TSharedRef<SWidget> CreateDetailsPanel();
		void OnHierarchySelectionChanged(RowHandle Row, ESelectInfo::Type SelectInfo);
		void RebuildDetailsForRow(RowHandle Row);
		FReply CopyCallstackToClipboard();

		TSharedRef<ITableRow> MakeColumnListRow(
			FScopeColumnItemPtr Item, const TSharedRef<STableViewBase>& OwnerTable);

		ICoreProvider* Storage = nullptr;
		IUiProvider* DataStorageUi = nullptr;

		// Hierarchy viewer
		TSharedPtr<SHierarchyViewer> HierarchyViewer;
		TSharedPtr<FHierarchyViewerData> HierarchyData;
		TSharedPtr<QueryStack::FQueryNode> ContextQueryNode;
		TSharedPtr<QueryStack::FRowQueryResultsNode> ScopeRowsNode;

		// Details panel
		TSharedPtr<SMultiLineEditableTextBox> CallstackTextBox;
		FString CachedCallstackString;
		TSharedPtr<SListView<FScopeColumnItemPtr>> ColumnListView;
		TArray<FScopeColumnItemPtr> ColumnItems;
	};
} // namespace UE::Editor::DataStorage::Debug
