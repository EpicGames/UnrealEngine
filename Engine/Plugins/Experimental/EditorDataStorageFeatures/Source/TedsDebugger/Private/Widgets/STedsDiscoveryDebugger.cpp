// Copyright Epic Games, Inc. All Rights Reserved.

#include "STedsDiscoveryDebugger.h"

#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "TedsTableViewerColumn.h"
#include "TedsRowArrayNode.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/STedsTableViewer.h"
#include "Widgets/SRowDetails.h"

#define LOCTEXT_NAMESPACE "TedsDiscoveryDebuggerTab"

namespace UE::Editor::DataStorage::Debug
{
	void SDiscoveryDebuggerTab::Construct(const FArguments& InArgs)
	{
		Storage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName);
		UiProvider = GetMutableDataStorageFeature<IUiProvider>(UiFeatureName);
        checkf(Storage && UiProvider, TEXT("Cannot create the Discovery Tab without TEDS Core or TEDS UI"));

		SetRowQueryStack();
		
		ChildSlot
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("Brushes.Background"))
			[
				SNew(SVerticalBox)
				+SVerticalBox::Slot()
				.AutoHeight()
				[
					CreateToolbar()
				]
				+SVerticalBox::Slot()
				.FillHeight(1.f)
				.Padding(0.f, 4.0f)
				[
					SNew(SSplitter)
					+SSplitter::Slot()
					.Value(0.5f)
					[
						SNew(SVerticalBox)
						+SVerticalBox::Slot()
						.AutoHeight()
						[
							CreateSearchBox()
						]
						+SVerticalBox::Slot()
						.FillContentHeight(1.0f)
						[
							CreateTableViewer()
						]
					]
					+SSplitter::Slot()
					.Value(0.5f)
					[
						SAssignNew(RowDetailsWidget, SRowDetails)
					]
				]
			]
		];
	}

	TSharedRef<SWidget> SDiscoveryDebuggerTab::CreateToolbar()
	{
		FSlimHorizontalToolBarBuilder ToolBarBuilder(nullptr, FMultiBoxCustomization::None);

		ToolBarBuilder.AddComboButton( 
			FUIAction(),
			FOnGetContent::CreateSP(this, &SDiscoveryDebuggerTab::GetSearchModesMenuWidget),
			TAttribute<FText>::CreateSP(this, &SDiscoveryDebuggerTab::GetCurrentSearchModeAsText),
			FText(),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Search"),
			false);

		return ToolBarBuilder.MakeWidget();
	}

	TSharedRef<SWidget> SDiscoveryDebuggerTab::GetSearchModesMenuWidget()
	{
		FMenuBuilder MenuBuilder(true, nullptr);

		MenuBuilder.AddMenuEntry(
			GetSearchModeAsText(ESearchMode::RowHandle),
			LOCTEXT("RowHandleSearchTooltip", "Search for a specific Row Handle and observe its properties"),
			FSlateIcon(), 
			FUIAction(FExecuteAction::CreateLambda([this]()
			{
				SearchMode = ESearchMode::RowHandle;
				if (ColumnSearchNode)
				{
					ColumnSearchNode->ClearSearch();
				}
				SetRowQueryStack();
			}))
		);

		MenuBuilder.AddMenuEntry(
			GetSearchModeAsText(ESearchMode::Column),
			LOCTEXT("ColumnSearchTooltip", "Search on all rows and their columns tagged as searchable"),
			FSlateIcon(), 
			FUIAction(FExecuteAction::CreateLambda([this]()
			{
				SearchMode = ESearchMode::Column;
				SetRowQueryStack();
			}))
		);

		return MenuBuilder.MakeWidget();
	}
	
	FText SDiscoveryDebuggerTab::GetCurrentSearchModeAsText() const
	{
		return GetSearchModeAsText(SearchMode);
	}

	FText SDiscoveryDebuggerTab::GetSearchModeAsText(const ESearchMode InSearchMode)
	{
		if (InSearchMode == ESearchMode::RowHandle)
		{
			return LOCTEXT("RowHandleComboBoxText", "Row Handle Search");
		}
		else if (InSearchMode == ESearchMode::Column)
		{
			return LOCTEXT("ColumnSearchComboBoxText", "Column Search");
		}
		return FText();
	}
	
	void SDiscoveryDebuggerTab::InitColumnSearch()
	{
		// No need to initialize this twice, we want to search on all available columns always
		if (!ColumnSearchNode)
		{
			// Uses the same node tech as the Universal Search but avoids making groups and searches without conditions
			// Use ESyncActions::Refresh since this is a debugger, and we prefer accuracy to performance
			ColumnSearchNode = MakeShared<QueryStack::FColumnsSearchNode>(*Storage, QueryStack::FColumnsSearchNode::ESyncActions::Refresh);

			// Retrieve searchable columns.
			QueryStack::Searching::ListSearchableColumns([this](const UScriptStruct* Column)
			{
				ColumnSearchNode->RegisterColumn(Column);
			});
		}
	}
	
	void SDiscoveryDebuggerTab::SetRowQueryStack()
	{
		RowHandle SearchedRowHandle = InvalidRowHandle;
		const FString SearchString = SearchText.ToString();
		if (SearchMode == ESearchMode::RowHandle)
		{
			FRowHandleArray Rows;
			LexFromString(SearchedRowHandle, *SearchString);
			if(Storage->IsRowAvailable(SearchedRowHandle))
			{
				Rows.Add(SearchedRowHandle);
			}

			RowQueryStack = MakeShared<QueryStack::FRowArrayNode>(Rows);
		}
		else if (SearchMode == ESearchMode::Column)
		{
			// Init the ColumnSearchNode if not done already (it checks if already done within the function)
			InitColumnSearch();
			if (ColumnSearchNode.IsValid())
			{
				RowQueryStack = ColumnSearchNode;
				if (SearchString.IsEmpty())
				{
					ColumnSearchNode->ClearSearch();
				}
				else
				{
					ColumnSearchNode->StartSearch(*SearchString);
				}
			}
		}

		if (TableViewer)
		{
			TableViewer->SetQueryStack(RowQueryStack);
			TableViewer->SetSelection(SearchedRowHandle, true, ESelectInfo::Direct);
			if(SearchedRowHandle == InvalidRowHandle && RowDetailsWidget)
			{
				RowDetailsWidget->ClearRow();
			}
		}
	}

	TSharedRef<SWidget> SDiscoveryDebuggerTab::CreateTableViewer()
	{
		CreateRowHandleColumn();
		
		TableViewer = SNew(STedsTableViewer)
			.QueryStack(RowQueryStack)
			.EmptyRowsMessage(LOCTEXT("EmptyRowsMessage", "No results for the current search."))
			.OnSelectionChanged(STedsTableViewer::FOnSelectionChanged::CreateLambda(
				[this](RowHandle SelectedRow, ESelectInfo::Type SelectInfo)
					{
						if(RowDetailsWidget)
						{
							if(SelectedRow != InvalidRowHandle)
							{
								RowDetailsWidget->SetRow(SelectedRow);
							}
							else
							{
								RowDetailsWidget->ClearRow();
							}
						}
					}));

		if(RowHandleColumn)
		{
			TableViewer->AddCustomRowWidget(RowHandleColumn.ToSharedRef());
		}
		
		return TableViewer->AsWidget();
	}

	void SDiscoveryDebuggerTab::CreateRowHandleColumn()
	{
		auto AssignWidgetToColumn = [this](IUiProvider::FWidgetConstructorPtr WidgetConstructor, TConstArrayView<TWeakObjectPtr<const UScriptStruct>>)
			{
				RowHandleColumn = MakeShared<FTedsTableViewerColumn>(TEXT("Row Handle"), WidgetConstructor);
				return false;
			};

		UiProvider->CreateWidgetConstructors(
			UiProvider->FindPurpose(IUiProvider::FPurposeInfo("General", "Cell", "RowHandle").GeneratePurposeID()),
			FMetaDataView(), AssignWidgetToColumn);
	}

	TSharedRef<SWidget> SDiscoveryDebuggerTab::CreateSearchBox()
	{
		SearchBox = SNew(SSearchBox)
			.SelectAllTextWhenFocused(true)
			.DelayChangeNotificationsWhileTyping(false)
			.OnTextChanged_Lambda([this](const FText& InText)
			{
				SearchText = InText;

				if (SearchText.IsEmpty())
				{
					if (TableViewer)
					{
						TableViewer->ClearSelection();
					}
					if (RowDetailsWidget)
					{
						RowDetailsWidget->ClearRow();
					}
					if (ColumnSearchNode)
					{
						ColumnSearchNode->ClearSearch();
					}
				}

				if (SearchMode == ESearchMode::RowHandle)
				{
					SetRowQueryStack();
				}
				else if (SearchMode == ESearchMode::Column && ColumnSearchNode && !SearchText.IsEmpty())
				{
					ColumnSearchNode->StartSearch(*SearchText.ToString());
				}
			})
			.HintText(LOCTEXT("SearchBox_Hint", "Search for TEDS data..."));
			
		return SearchBox.ToSharedRef();
	}
	
} // namespace UE::Editor::DataStorage::Debug

#undef LOCTEXT_NAMESPACE
