// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "DataStorage/Features.h"
#include "QueryEditor/TedsQueryEditorResultsView.h"
#include "TedsColumnsSearchNode.h"
#include "Widgets/SCompoundWidget.h"

namespace UE::Editor::DataStorage { class SRowDetails; }
class SSearchBox;
class STedsTableViewer;
class FTedsTableViewerColumn;

namespace UE::Editor::DataStorage::Debug
{
	using UE::Editor::DataStorage::SRowDetails;

	enum class ESearchMode : uint8
	{
		// Search for a specific RowHandle for specific debugging
		RowHandle,
		// Search on all rows on columns with Searchable properties
		Column
	};
	
	class SDiscoveryDebuggerTab final : public SCompoundWidget
	{
	public:

		SLATE_BEGIN_ARGS(SDiscoveryDebuggerTab) 
		{}
		SLATE_END_ARGS()

		void Construct(const FArguments& InArgs);

	private:

		TSharedRef<SWidget> CreateToolbar();
		TSharedRef<SWidget> GetSearchModesMenuWidget();
		FText GetCurrentSearchModeAsText() const;
		static FText GetSearchModeAsText(const ESearchMode InSearchMode);

		void SetRowQueryStack();
		void InitColumnSearch();
		TSharedRef<SWidget> CreateTableViewer();
		void CreateRowHandleColumn();
		TSharedRef<SWidget> CreateSearchBox();

		TSharedPtr<STedsTableViewer> TableViewer;
		TSharedPtr<SSearchBox> SearchBox;
		// Widget that displays details of a row
		TSharedPtr<SRowDetails> RowDetailsWidget;
		// Custom column for the table viewer to display row handles
		TSharedPtr<FTedsTableViewerColumn> RowHandleColumn;
		
		TSharedPtr<QueryStack::IRowNode> RowQueryStack;
		// Hold onto the ColumnSearchNode so it doesn't need to be re-initialized when switching back to column search
		TSharedPtr<QueryStack::FColumnsSearchNode> ColumnSearchNode;

		ICoreProvider* Storage = nullptr;
		IUiProvider* UiProvider = nullptr;
		FText SearchText;
		ESearchMode SearchMode = ESearchMode::RowHandle;
	};
}
