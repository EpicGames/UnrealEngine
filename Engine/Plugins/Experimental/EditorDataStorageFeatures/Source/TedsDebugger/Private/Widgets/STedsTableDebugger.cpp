// Copyright Epic Games, Inc. All Rights Reserved.

#include "STedsTableDebugger.h"

#include "DataStorage/Features.h"
#include "Elements/Columns/TypedElementHiearchyColumns.h"
#include "Elements/Interfaces/TypedElementDataStorageInterface.h"
#include "Elements/Interfaces/TypedElementDataStorageUiInterface.h"
#include "TedsRowArrayNode.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Layout/SSplitter.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/STedsTableViewer.h"

#define LOCTEXT_NAMESPACE "TedsTableDebuggerTab"

namespace UE::Editor::DataStorage::Debug
{
	void STableDebuggerTab::Construct(const FArguments& InArgs)
	{
		Storage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName);
		UiProvider = GetMutableDataStorageFeature<IUiProvider>(UiFeatureName);
        checkf(Storage && UiProvider, TEXT("Cannot create the Table Debugger tab without TEDS Core or TEDS UI"));
		
		RefreshRootTables();

		ChildSlot
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("Brushes.Background"))
			[
				SNew(SSplitter)
				.Orientation(Orient_Horizontal)
				+SSplitter::Slot()
				.Value(1.0f)
				[
					SNew(SSplitter)
					.Orientation(Orient_Vertical)
					+SSplitter::Slot()
					.Value(1.0f)
					[
						CreateTreeView()
					]
					+SSplitter::Slot()
					.Value(1.0f)
					[
						CreateInfoView()
					]
				]
				+SSplitter::Slot()
				.Value(2.0f)
				[
					CreateRowView()
				]
			]
		];
	}

	void STableDebuggerTab::RefreshRootTables()
	{
		RootTables.Reset();
		Storage->ListTables(ETableType::Declared, [this](TableHandle Table)
			{
				RootTables.Emplace(Table);
			});
	}

	void STableDebuggerTab::OnTableSelection(TreeViewItem Item, ESelectInfo::Type SelectInfo)
	{
		// Fill in the row view
		FRowHandleArray& Rows = QueryStack->GetMutableRows();
		Rows.Reset();
		if (Item)
		{
			Storage->ListTableRows(Item, [&Rows](FRowHandleArrayView Chunk)
				{
					Rows.Append(Chunk);
				});

			RowHandle GeneralPurposeRowHandle = UiProvider->FindPurpose(UiProvider->GetGeneralWidgetPurposeID());
			RowHandle DefaultPurposeRowHandle = UiProvider->FindPurpose(UiProvider->GetDefaultWidgetPurposeID());

			// Temporarily add the default purpose as a parent of the general purpose so the debugger can support both
			Storage->AddColumn(GeneralPurposeRowHandle, FTableRowParentColumn{ .Parent = DefaultPurposeRowHandle });

			const UScriptStruct* TagType = FEditorDataStorageTag::StaticStruct();
			TArray<TWeakObjectPtr<const UScriptStruct>> Columns;
			Storage->ListTableColumns(Item, [&Columns, TagType](const UScriptStruct* ColumnType)
				{
					Columns.Add(ColumnType);
					return true;
				});
			RowView->SetColumns(Columns);
			if (RowForeignKeyColumn && !Columns.IsEmpty())
			{
				RowView->AddCustomRowWidget(RowForeignKeyColumn.ToSharedRef());
			}

			// Remove the parenting chain after we have used it to generate widgets
			Storage->RemoveColumns<FTableRowParentColumn>(GeneralPurposeRowHandle);
		}
		QueryStack->MarkDirty();

		// Fill in the info view
		auto TableTypeToString = [](ETableType Type)
			{
				switch (Type)
				{
				case ETableType::Invalid:
					return FString(TEXT("Invalid"));
				case ETableType::Declared:
					return FString(TEXT("Declared"));
				case ETableType::Derivative:
					return FString(TEXT("Derivative"));
				case ETableType::Variant:
					return FString(TEXT("Variant"));
				default:
					return FString(TEXT("Unknown"));
				}
			};

		InfoLines.Reset();
		if (Item)
		{
			FTableInfoView TableInfo = Storage->GetTableInfo(Item);
			if (TableInfo.Type != ETableType::Invalid)
			{
				InfoLines.Add(MakeShared<FTableInfoEntry>(FTableInfoEntry
					{
						.Key = TEXT("Name"),
						.Value = TableInfo.Name.ToString()
					}));
				InfoLines.Add(MakeShared<FTableInfoEntry>(FTableInfoEntry
					{
						.Key = TEXT("Type"),
						.Value = TableTypeToString(TableInfo.Type)
					}));
				InfoLines.Add(MakeShared<FTableInfoEntry>(FTableInfoEntry
					{
						.Key = TEXT("Derivative count"),
						.Value = FString::FromInt(TableInfo.Derivatives.Num())
					}));
				InfoLines.Add(MakeShared<FTableInfoEntry>(FTableInfoEntry
					{
						.Key = TEXT("Variant count"),
						.Value = FString::FromInt(TableInfo.Variants.Num())
					}));

				TStringBuilder<256> TempString;
				Storage->ListTableForeignKeyDomains(Item, false, [&TempString](const FName& DomainName)
					{
						TempString.Append(DomainName.ToString());
						TempString.Add(TEXT('\n'));
					});
				InfoLines.Add(MakeShared<FTableInfoEntry>(FTableInfoEntry
					{
						.Key = TEXT("Foreign key domains (owned)"),
						.Value = TempString.ToString()
					}));

				TempString.Reset();
				Storage->ListTableForeignKeyDomains(Item, true, [&TempString](const FName& DomainName)
					{
						TempString.Append(DomainName.ToString());
						TempString.Add(TEXT('\n'));
					});
				InfoLines.Add(MakeShared<FTableInfoEntry>(FTableInfoEntry
					{
						.Key = TEXT("Foreign key domains (all)"),
						.Value = TempString.ToString()
					}));

				TempString.Reset();
				Storage->ListTableColumns(Item, [&TempString](const UScriptStruct* Column)
					{
						TempString.Append(Column->GetName());
						TempString.Add(TEXT('\n'));
						return true;
					});
				InfoLines.Add(MakeShared<FTableInfoEntry>(FTableInfoEntry
					{
						.Key = TEXT("Columns"),
						.Value = TempString.ToString()
					}));
			}
		}

		InfoView->RequestListRefresh();
	}

	TSharedRef<SWidget> STableDebuggerTab::CreateTreeView()
	{
		return SNew(STreeView<TreeViewItem>)
			.TreeItemsSource(&RootTables)
			.OnGenerateRow(this, &STableDebuggerTab::MakeTableRowWidget)
			.OnGetChildren(this, &STableDebuggerTab::HandleGetChildrenForTree)
			.SelectionMode(ESelectionMode::Single)
			.OnSelectionChanged(this, &STableDebuggerTab::OnTableSelection);
	}

	TSharedRef<SWidget> STableDebuggerTab::CreateInfoView()
	{
		return SAssignNew(InfoView, SListView<InfoItem>)
			.ListItemsSource(&InfoLines)
			.HeaderRow
			(
				SNew(SHeaderRow)
				+SHeaderRow::Column(TEXT("Key"))
					.DefaultLabel(FText::GetEmpty())
				+SHeaderRow::Column(TEXT("Value"))
					.DefaultLabel(FText::GetEmpty())
			)
			.OnGenerateRow(this, &STableDebuggerTab::MakeInfoRowWidget);
	}

	TSharedRef<SWidget> STableDebuggerTab::CreateRowView()
	{
		CreateRowForeignKeyColumn();

		QueryStack = MakeShared<QueryStack::FRowArrayNode>();

		SAssignNew(RowView, STedsTableViewer)
			.QueryStack(QueryStack)
			.EmptyRowsMessage(LOCTEXT("NoSelectionMessageForTable", "Select a table to see content."));
		return RowView.ToSharedRef();
	}
	
	void STableDebuggerTab::CreateRowForeignKeyColumn()
	{
		auto AssignWidgetToColumn = [this](IUiProvider::FWidgetConstructorPtr WidgetConstructor, TConstArrayView<TWeakObjectPtr<const UScriptStruct>>)
			{
				RowForeignKeyColumn = MakeShared<FTedsTableViewerColumn>(TEXT("Row Foreign Key"), WidgetConstructor);
				return false;
			};

		UiProvider->CreateWidgetConstructors(
			UiProvider->FindPurpose(IUiProvider::FPurposeInfo("General", "Cell", "RowForeignKey").GeneratePurposeID()),
			FMetaDataView(), AssignWidgetToColumn);
	}

	TSharedRef<ITableRow> STableDebuggerTab::MakeTableRowWidget(TreeViewItem Item, const TSharedRef<STableViewBase>& Tree)
	{
		return SNew(STableTreeViewRow, *Storage, Item, Tree);
	}

	TSharedRef<ITableRow> STableDebuggerTab::MakeInfoRowWidget(InfoItem Item, const TSharedRef<STableViewBase>& Tree)
	{
		return SNew(SInfoRow, Item, Tree);
	}

	void STableDebuggerTab::HandleGetChildrenForTree(TreeViewItem Item, TArray<TreeViewItem>& Children)
	{
		FTableInfoView TableInfo = Storage->GetTableInfo(Item);
		checkf(TableInfo.Type != ETableType::Invalid, TEXT("Received an invalid table handle while retrieving children."));

		for (TableHandle Derivative : TableInfo.Derivatives)
		{
			Children.Emplace(Derivative);
		}
		for (TableHandle Variant : TableInfo.Variants)
		{
			Children.Emplace(Variant);
		}
	}

	void STableDebuggerTab::STableTreeViewRow::Construct(const FArguments& InArgs, const ICoreProvider& InStorage, const TreeViewItem& InItem, 
		const TSharedRef<STableViewBase>& InOwner)
	{
		FTableInfoView TableInfo = InStorage.GetTableInfo(InItem);
		checkf(TableInfo.Type != ETableType::Invalid, TEXT("Received an invalid table handle for construction."));
		
		Super::Construct(Super::FArguments().Content()
		[
			SNew(STextBlock).Text(FText::FromName(TableInfo.Name))
		], InOwner);
	}

	void STableDebuggerTab::SInfoRow::Construct(const FArguments& InArgs, const InfoItem& InItem, const TSharedRef<STableViewBase>& InOwner)
	{
		Item = InItem;

		SMultiColumnTableRow::Construct(FSuperRowType::FArguments(), InOwner);
	}

	TSharedRef<SWidget> STableDebuggerTab::SInfoRow::GenerateWidgetForColumn(const FName& ColumnName)
	{
		if (ColumnName == TEXT("Key"))
		{
			return SNew(STextBlock)
				.Text(FText::FromString(Item->Key));
		}
		else if (ColumnName == TEXT("Value"))
		{
			return SNew(STextBlock)
				.Text(FText::FromString(Item->Value));
		}

		return SNullWidget::NullWidget;
	}
	
} // namespace UE::Editor::DataStorage::Debug

#undef LOCTEXT_NAMESPACE
