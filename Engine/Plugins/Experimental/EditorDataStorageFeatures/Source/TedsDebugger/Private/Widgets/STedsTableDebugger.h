// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/UnrealString.h"
#include "DataStorage/CommonTypes.h"
#include "TedsTableViewerColumn.h"
#include "TypedElementUITypes.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/STreeView.h"
#include "Widgets/Views/SListView.h"

namespace UE::Editor::DataStorage
{
	namespace QueryStack
	{
		class FRowArrayNode;
	}

	class ICoreProvider;
	class IUiProvider;
	class STedsTableViewer;
}

namespace UE::Editor::DataStorage::Debug
{
	class STableDebuggerTab final : public SCompoundWidget
	{
	public:

		SLATE_BEGIN_ARGS(STableDebuggerTab)
		{}
		SLATE_END_ARGS()

		void Construct(const FArguments& InArgs);

	private:
		using TreeViewItem = FTedsTableHandle;
		class STableTreeViewRow final : public STableRow<TreeViewItem>
		{
		public:
			using Super = STableRow<TreeViewItem>;

			SLATE_BEGIN_ARGS(STableTreeViewRow) {}
			SLATE_END_ARGS()

			void Construct(const FArguments& InArgs, const ICoreProvider& InStorage, const TreeViewItem& InItem,
				const TSharedRef<STableViewBase>& InOwner);
		};

		struct FTableInfoEntry final
		{
			FString Key;
			FString Value;
		};

		using InfoItem = TSharedPtr<FTableInfoEntry>;
		class SInfoRow final : public SMultiColumnTableRow<InfoItem>
		{
		public:
			SLATE_BEGIN_ARGS(SInfoRow) {}
			SLATE_END_ARGS()

			void Construct(const FArguments& InArgs, const InfoItem& InItem, const TSharedRef<STableViewBase>& InOwner);
			
		protected:
			virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override;

		private:
			InfoItem Item;
		};

		void RefreshRootTables();
		void OnTableSelection(TreeViewItem Item, ESelectInfo::Type SelectInfo);

		TSharedRef<SWidget> CreateTreeView();
		TSharedRef<SWidget> CreateInfoView();
		TSharedRef<SWidget> CreateRowView();
		void CreateRowForeignKeyColumn();
		TSharedRef<ITableRow> MakeTableRowWidget(TreeViewItem Item, const TSharedRef<STableViewBase>& Tree);
		TSharedRef<ITableRow> MakeInfoRowWidget(InfoItem Item, const TSharedRef<STableViewBase>& Tree);
		void HandleGetChildrenForTree(TreeViewItem Item, TArray<TreeViewItem>& Children);

		TArray<TreeViewItem> RootTables;
		TArray<InfoItem> InfoLines;
		TSharedPtr<SListView<InfoItem>> InfoView;
		TSharedPtr<STedsTableViewer> RowView;
		TSharedPtr<QueryStack::FRowArrayNode> QueryStack;
		TSharedPtr<FTedsTableViewerColumn> RowForeignKeyColumn;
		ICoreProvider* Storage = nullptr;
		IUiProvider* UiProvider = nullptr;
	};
}
