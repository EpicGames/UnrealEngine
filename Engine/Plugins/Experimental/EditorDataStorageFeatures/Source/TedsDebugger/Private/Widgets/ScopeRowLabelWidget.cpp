// Copyright Epic Games, Inc. All Rights Reserved.

#include "ScopeRowLabelWidget.h"

#include "Elements/Columns/TypedElementLabelColumns.h"
#include "Elements/Interfaces/TypedElementDataStorageInterface.h"
#include "Widgets/Text/STextBlock.h"

FScopeRowLabelWidgetConstructor::FScopeRowLabelWidgetConstructor()
	: FSimpleWidgetConstructor(StaticStruct())
{
}

TSharedPtr<SWidget> FScopeRowLabelWidgetConstructor::CreateWidget(
	UE::Editor::DataStorage::ICoreProvider* DataStorage,
	UE::Editor::DataStorage::IUiProvider* DataStorageUi,
	UE::Editor::DataStorage::RowHandle TargetRow,
	UE::Editor::DataStorage::RowHandle WidgetRow,
	const UE::Editor::DataStorage::FMetaDataView& Arguments)
{
	using namespace UE::Editor::DataStorage;

	const FString RowHandleString = LexToString(TargetRow);

	return SNew(STextBlock)
		.Text_Lambda([DataStorage, TargetRow, RowHandleString]()
		{
			if (DataStorage && DataStorage->IsRowAvailable(TargetRow))
			{
				// NOTE: We _don't_ use the scope APIs here because we are displaying the data about the individual scope row itself
				// We don't want to show the label for a row above if this one doesn't have one or if it is empty.
				if (const FTypedElementLabelColumn* LabelColumn = DataStorage->GetColumn<FTypedElementLabelColumn>(TargetRow))
				{
					if (!LabelColumn->Label.IsEmpty())
					{
						return FText::FromString(LabelColumn->Label);
					}
				}
			}
			return FText::FromString(RowHandleString);
		})
		.ToolTipText(FText::FromString(RowHandleString));
}
