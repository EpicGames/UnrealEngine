// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/Interfaces/TypedElementDataStorageUiInterface.h"

#include "ScopeRowLabelWidget.generated.h"

/**
 * Widget constructor for scope hierarchy rows that shows the FTypedElementLabelColumn
 * text when available, falling back to the stringified RowHandle. When a label is shown,
 * the tooltip displays the RowHandle for debugging.
 */
USTRUCT()
struct FScopeRowLabelWidgetConstructor : public FSimpleWidgetConstructor
{
	GENERATED_BODY()

public:
	FScopeRowLabelWidgetConstructor();
	~FScopeRowLabelWidgetConstructor() override = default;

	TSharedPtr<SWidget> CreateWidget(
		UE::Editor::DataStorage::ICoreProvider* DataStorage,
		UE::Editor::DataStorage::IUiProvider* DataStorageUi,
		UE::Editor::DataStorage::RowHandle TargetRow,
		UE::Editor::DataStorage::RowHandle WidgetRow,
		const UE::Editor::DataStorage::FMetaDataView& Arguments) override;
};
