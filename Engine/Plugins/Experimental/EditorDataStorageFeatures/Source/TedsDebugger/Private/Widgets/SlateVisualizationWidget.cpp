// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SlateVisualizationWidget.h"

#include "Elements/Columns/TypedElementMiscColumns.h"
#include "Elements/Columns/TypedElementSlateWidgetColumns.h"
#include "Elements/Interfaces/TypedElementDataStorageInterface.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SBoxPanel.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SlateVisualizationWidget)


void USlateVisualizationWidgetFactory::RegisterWidgetConstructors(UE::Editor::DataStorage::ICoreProvider& DataStorage, UE::Editor::DataStorage::IUiProvider& DataStorageUi) const
{
	using namespace UE::Editor::DataStorage::Queries;

	DataStorageUi.RegisterWidgetFactory<FSlateVisualizationWidgetConstructor>(DataStorageUi.FindPurpose(DataStorageUi.GetGeneralWidgetPurposeID()),
		TColumn<FTypedElementSlateWidgetReferenceColumn>());
}

FSlateVisualizationWidgetConstructor::FSlateVisualizationWidgetConstructor()
	: Super(FSlateVisualizationWidgetConstructor::StaticStruct())
{

}

TSharedPtr<SWidget> FSlateVisualizationWidgetConstructor::CreateWidget(const UE::Editor::DataStorage::FMetaDataView& Arguments)
{
	return SNew(SHorizontalBox);
}

bool FSlateVisualizationWidgetConstructor::FinalizeWidget(UE::Editor::DataStorage::ICoreProvider* DataStorage,
	UE::Editor::DataStorage::IUiProvider* DataStorageUi,
	UE::Editor::DataStorage::RowHandle Row, const TSharedPtr<SWidget>& Widget)
{
	checkf(Widget, TEXT("Referenced widget is not valid. A constructed widget may not have been cleaned up. This can "
	"also happen if this processor is running in the same phase as the processors responsible for cleaning up old "
	"references."));

	UE::Editor::DataStorage::RowHandle TargetRow = DataStorage->GetColumn<FTypedElementRowReferenceColumn>(Row)->Row;

	if (const FTypedElementSlateWidgetReferenceColumn* SlateWidgetReferenceColumn = DataStorage->GetColumn<FTypedElementSlateWidgetReferenceColumn>(TargetRow))
	{
		if(TSharedPtr<SWidget> ActualWidget = SlateWidgetReferenceColumn->Widget.Pin())
		{
			checkf(Widget->GetType() == SHorizontalBox::StaticWidgetClass().GetWidgetType(),
				TEXT("Stored widget with FTypedElementLabelWidgetConstructor doesn't match type %s, but was a %s."),
				*(SHorizontalBox::StaticWidgetClass().GetWidgetType().ToString()),
				*(Widget->GetTypeAsString()));

			SHorizontalBox* WidgetInstance = static_cast<SHorizontalBox*>(Widget.Get());

			// Simply display the widget for now
			// TEDS UI TODO: We can also display some debug info about the widget as a tooltip etc
			WidgetInstance->AddSlot()
			.AutoWidth()
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			[
				ActualWidget.ToSharedRef()
			];
		}
	}

	return true;
}
