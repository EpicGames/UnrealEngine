// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/Interfaces/TypedElementDataStorageFactory.h"
#include "Elements/Interfaces/TypedElementDataStorageUiInterface.h"

#include "SlateVisualizationWidget.generated.h"

class SWidget;

/*
 * Widget for the TEDS Debugger that shows a slate widget reference
 */
UCLASS()
class USlateVisualizationWidgetFactory : public UEditorDataStorageFactory
{
	GENERATED_BODY()

public:
	~USlateVisualizationWidgetFactory() override = default;

	void RegisterWidgetConstructors(UE::Editor::DataStorage::ICoreProvider& DataStorage,
		UE::Editor::DataStorage::IUiProvider& DataStorageUi) const override;
};

USTRUCT()
struct FSlateVisualizationWidgetConstructor : public FTypedElementWidgetConstructor
{
	GENERATED_BODY()

public:
	FSlateVisualizationWidgetConstructor();
	~FSlateVisualizationWidgetConstructor() override = default;

protected:
	TSharedPtr<SWidget> CreateWidget(const UE::Editor::DataStorage::FMetaDataView& Arguments) override;
	bool FinalizeWidget(UE::Editor::DataStorage::ICoreProvider* DataStorage,
		UE::Editor::DataStorage::IUiProvider* DataStorageUi,
		UE::Editor::DataStorage::RowHandle Row, const TSharedPtr<SWidget>& Widget) override;
};