// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class TedsDebugger : ModuleRules
{
	public TedsDebugger(ReadOnlyTargetRules Target) : base(Target)
	{
		// Enable truncation warnings in this module
		CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Warning;

		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		if (Target.bBuildEditor)
		{
			PublicIncludePaths.AddRange(new string[] {});
			PrivateIncludePaths.AddRange(new string[] {});

			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject"
				});
			
			PrivateDependencyModuleNames.AddRange(
            	new string[]
            	{
            		"TedsOutliner",
		            "WorkspaceMenuStructure",
		            "TypedElementFramework",
		            "SceneOutliner",
		            "SlateCore",
		            "Slate",
		            "InputCore",
		            "EditorWidgets",
		            "ToolWidgets",
		            "TedsTableViewer",
		            "TedsQueryStack",
		            "ApplicationCore"
            	});
			
			DynamicallyLoadedModuleNames.AddRange(new string[] {});
		}
	}
}
