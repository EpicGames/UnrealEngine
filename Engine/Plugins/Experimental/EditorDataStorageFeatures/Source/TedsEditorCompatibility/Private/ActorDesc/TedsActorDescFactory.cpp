// Copyright Epic Games, Inc. All Rights Reserved.

#include "TedsActorDescFactory.h"

#include "Editor.h"
#include "EditorActorFolders.h"
#include "ActorDesc/TedsActorDescColumns.h"
#include "ActorDesc/TedsActorDescUtils.h"
#include "ActorFolders/TedsActorFolderUtils.h"
#include "DataStorage/Features.h"
#include "Elements/Columns/TypedElementCompatibilityColumns.h"
#include "Elements/Columns/TypedElementLabelColumns.h"
#include "Elements/Columns/TypedElementMiscColumns.h"
#include "Elements/Columns/TypedElementTypeInfoColumns.h"
#include "Elements/Columns/TypedElementUIColumns.h"
#include "Elements/Framework/TypedElementQueryBuilder.h"
#include "Elements/Framework/TypedElementQueryContext.h"
#include "GameFramework/Actor.h"
#include "LevelInstance/LevelInstanceSubsystem.h"
#include "LevelInstance/LevelInstanceInterface.h"
#include "Misc/FileHelper.h"
#include "WorldPartition/ActorDescContainer.h"
#include "WorldPartition/WorldPartitionActorDescInstance.h"
#include "WorldPartition/WorldPartition.h"
#include "WorldPartition/WorldPartitionHelpers.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TedsActorDescFactory)

#define LOCTEXT_NAMESPACE "TedsUnloadedActorFactory"

namespace UE::Editor::DataStorage::UnloadedActor
{
	static bool bRegisterActorDescsInTeds = false;
	
	// CVar to control registration of actor descs in TEDS
	static FAutoConsoleVariableRef CvarRegisterActorDescsInTeds(
		TEXT("TEDS.Feature.UnloadedActors"),
		bRegisterActorDescsInTeds,
		TEXT("Populate Unloaded Actors in TEDS. Must be set at startup."));

	// Get the first logical parent for an actor desc in the editor
	bool GetEditorParentForActorDesc(const FWorldPartitionActorDescInstance* ActorDescInstance, UWorld* World, FName& OutParentMappingDomain, FMapKey& OutParentMappingKey)
	{
		if (!World)
		{
			return false;
		}

		// Parent folder
		const FFolder ActorDescFolder = FActorFolders::GetActorDescInstanceFolder(*World, ActorDescInstance);
		if (!ActorDescFolder.IsNone())
		{
			OutParentMappingDomain = ActorFolders::GetFolderMappingDomain(ActorDescFolder);
			OutParentMappingKey = ActorFolders::GetFolderIndex(ActorDescFolder);
			return true;
		}
		
		// Parent Actor (Actor attachement / parenting)
		const FGuid& ParentActorGuid = ActorDescInstance->GetSceneOutlinerParent();
		if (ParentActorGuid.IsValid())
		{
			if (UActorDescContainerInstance* ContainerInstance = ActorDescInstance->GetContainerInstance())
			{
				if (const FWorldPartitionActorDescInstance* ParentActorDesc = ContainerInstance->GetActorDescInstance(ParentActorGuid))
				{
					// If parent actor is loaded
					constexpr bool bEvenIfPendingKill = false;
					if (AActor* ParentActor = ParentActorDesc->GetActor(bEvenIfPendingKill))
					{
						// Depending on when the sync query is run, this can be called after the parent is marked for destruction (unloaded)
						// but before it is fully destroyed. E.g when an actor desc is marked as visible in the UI by the OnRemove observer for the
						// loaded actor being destroyed/unloaded. In that case we don't want it to parent to the actor since it's going to be removed
						// from TEDS soon
						if (IsValid(ParentActor) && !ParentActor->HasAnyFlags(RF_BeginDestroyed))
						{
							OutParentMappingDomain = ICompatibilityProvider::ObjectMappingDomain;
							OutParentMappingKey = FMapKey(ParentActor);
							return true;
						}
					}
					else
					{
						OutParentMappingDomain = GetActorDescMappingDomain();
						OutParentMappingKey = GetActorDescMappingKey(ParentActorDesc);
						return true;
					}

				}
			}
		}
		// Parent container
		else if (UActorDescContainerInstance* ContainerInstance = ActorDescInstance->GetContainerInstance())
		{
			if (const ULevelInstanceSubsystem* LevelInstanceSubsystem = UWorld::GetSubsystem<ULevelInstanceSubsystem>(World))
			{
				UWorld* OuterWorld = ContainerInstance->GetTypedOuter<UWorld>();
				// If parent actor is loaded
				if (AActor* ParentActor = OuterWorld ? Cast<AActor>(LevelInstanceSubsystem->GetOwningLevelInstance(OuterWorld->PersistentLevel)) : nullptr)
				{
					// Depending on when the sync query is run, this can be called after the parent is marked for destruction (unloaded)
					// but before it is fully destroyed. E.g when an actor desc is marked as visible in the UI by the OnRemove observer for the
					// loaded actor being destroyed/unloaded. In that case we don't want it to parent to the actor since it's going to be removed
					// from TEDS soon
					if (IsValid(ParentActor) && !ParentActor->HasAnyFlags(RF_BeginDestroyed))
					{
						OutParentMappingDomain = ICompatibilityProvider::ObjectMappingDomain;
						OutParentMappingKey = FMapKey(ParentActor);
						return true;
					}
				}
			}
			// If parent actor is not loaded
			if (UActorDescContainerInstance* ParentContainerInstance = Cast<UActorDescContainerInstance>(ContainerInstance->GetOuter()))
			{
				if (const FWorldPartitionActorDescInstance* ParentActorDescInstance = ParentContainerInstance->GetActorDescInstance(ContainerInstance->GetContainerActorGuid()))
				{
					OutParentMappingDomain = GetActorDescMappingDomain();
					OutParentMappingKey = GetActorDescMappingKey(ParentActorDescInstance);
					return true;
				}
			}
		}
		
		// Fallback to parent world
		OutParentMappingDomain = ICompatibilityProvider::ObjectMappingDomain;
		OutParentMappingKey = FMapKey(World);
		return true;
	}
}

void UTedsActorDescFactory::RegisterTables(UE::Editor::DataStorage::ICoreProvider& DataStorage)
{
	using namespace UE::Editor::DataStorage::Queries;
	using namespace UE::Editor::DataStorage::UnloadedActor;

	TableHandle ActorDescTable = DataStorage.RegisterTable<FActorDescTag, FTypedElementLabelColumn, FTypedElementClassTypeInfoColumn,
	FTypedElementWorldColumn, FWorldPartitionHandleColumn, FGuidColumn>(
		GetActorDescTableName());
	
	DataStorage.RegisterTableForeignKey(ActorDescTable, FName("Serialization"), 
			[](TConstQueryContext<RowBatchInfo> Context, TResult<FForeignKey>& Result, TConstBatch<FWorldPartitionHandleColumn> HandleColumns)
			{
				Context.ForEachRow([&Result](RowHandle Row, const FWorldPartitionHandleColumn& HandleColumn)
					{
						if (const FWorldPartitionActorDescInstance* const ActorDescInstance = *HandleColumn.Handle)
						{
							return GetActorDescMappingKey(ActorDescInstance);
						}
						
						return FForeignKey(FKeyIssue(TEXT("Row doesn't contain a valid actor desc.")));
					}, HandleColumns);
			});
}

void UTedsActorDescFactory::RegisterQueries(UE::Editor::DataStorage::ICoreProvider& DataStorage)
{
	using namespace UE::Editor::DataStorage::Queries;
	using namespace UE::Editor::DataStorage::UnloadedActor;
	
	if (!CvarRegisterActorDescsInTeds->GetBool())
	{
		return;
	}
	
	// Since getting the FWorldPartitionActorDescInstance is the expensive part and locked to the game thread, we have a single processor handling
	// all the updates
	DataStorage.RegisterQuery(
		Select(
			TEXT("Sync actor desc data from world to TEDS"),
			FProcessor(EQueryTickPhase::DuringPhysics, DataStorage.GetQueryTickGroupName(EQueryTickGroups::SyncDataStorageToExternal))
				.SetExecutionMode(EExecutionMode::GameThread),
				[](IQueryContext& Context, RowHandle ActorDescRow, const FWorldPartitionHandleColumn& ActorDescColumn, FGuidColumn& GuidColumn,
					FTypedElementLabelColumn& LabelColumn, FTypedElementClassTypeInfoColumn& TypeInfoColumn, const FTypedElementWorldColumn& WorldColumn)
				{
					if (const FWorldPartitionActorDescInstance* const ActorDescInstance = *ActorDescColumn.Handle)
					{
						// Sync label
						{
							LabelColumn.Label  = ActorDescInstance->GetActorLabelString();
						}
						
						// Sync type info
						{
							TypeInfoColumn.TypeInfo = ActorDescInstance->GetActorNativeClass();
						}
						
						// Sync GUID
						{
							GuidColumn.Guid = ActorDescInstance->GetGuid();
						}
						
						// Sync parent row
						{
							FMapKey IdKey;
							FName MappingDomain;
										
							if (UnloadedActor::GetEditorParentForActorDesc(ActorDescInstance, WorldColumn.World.Get(), MappingDomain, IdKey))
							{
								RowHandle ParentRow = Context.LookupMappedRow(MappingDomain, IdKey);
										
								if (Context.IsRowAvailable(ParentRow))
								{
									Context.SetParentRow(ActorDescRow, ParentRow);
								}
								else
								{
									Context.SetUnresolvedParent(ActorDescRow, IdKey, MappingDomain);
								}
							}
							else
							{
								Context.SetParentRow(ActorDescRow, InvalidRowHandle);
							}
						}
					}
				}
			)
		.AccessesHierarchy("EditorObjectHierarchy")
		.Where(TColumn<FActorDescTag>() && TColumn<FTypedElementSyncFromWorldTag>())
		.Compile()
		);
	
	// Actor descs always exist in memory even if the actual actor is loaded, so we match that lifecycle in TEDS.
	// If the actual actor is loaded, we hide the actor desc from UI and if the actual actor is unloaded we show the actor desc in UI
	// We'll simply use TEDS to track the lifecycle of the actors.
	DataStorage.RegisterQuery(
		Select(
			TEXT("Hide actor desc on actor loaded"),
			FObserver::OnAdd<FTypedElementUObjectColumn>().SetExecutionMode(EExecutionMode::GameThread),
			[](IQueryContext& Context, RowHandle Row, const FTypedElementUObjectColumn& ObjectColumn)
			{
				if (AActor* Actor = Cast<AActor>(ObjectColumn.Object))
				{
					if (UWorldPartition* WorldPartition = FWorldPartitionHelpers::GetWorldPartition(Actor))
					{
						const FGuid& ActorGuid = Actor->GetActorGuid();
						if (FWorldPartitionActorDescInstance* ActorDescInstance = WorldPartition->GetActorDescInstance(ActorGuid))
						{
							RowHandle ActorDescRow = Context.LookupMappedRow(GetActorDescMappingDomain(), GetActorDescMappingKey(ActorDescInstance));
							Context.AddColumns<FHideRowFromUITag>(ActorDescRow);
						}
					}
				}
			})
		.Where()
			.All<FTypedElementActorTag>()
		.Compile());
	
	DataStorage.RegisterQuery(
		Select(
			TEXT("Show actor desc on actor unloaded"),
			FObserver::OnRemove<FTypedElementUObjectColumn>().SetExecutionMode(EExecutionMode::GameThread),
			[](IQueryContext& Context, RowHandle Row, const FTypedElementUObjectColumn& ObjectColumn)
			{
				if (AActor* Actor = Cast<AActor>(ObjectColumn.Object.Get(true)))
				{
					if (UWorldPartition* WorldPartition = FWorldPartitionHelpers::GetWorldPartition(Actor))
					{
						const FGuid& ActorGuid = Actor->GetActorGuid();
						if (FWorldPartitionActorDescInstance* ActorDescInstance = WorldPartition->GetActorDescInstance(ActorGuid))
						{
							RowHandle ActorDescRow = Context.LookupMappedRow(GetActorDescMappingDomain(), GetActorDescMappingKey(ActorDescInstance));
							
							Context.RemoveColumns<FHideRowFromUITag>(ActorDescRow);
							
							// Also sync data before showing the actor desc in case it's needed
							Context.AddColumns<FTypedElementSyncFromWorldTag>(ActorDescRow);
						}
					}
				}
			})
		.Where()
			.All<FTypedElementActorTag>()
		.Compile());
}

void UTedsActorDescFactory::PreRegister(UE::Editor::DataStorage::ICoreProvider& DataStorage)
{
	if (UE::Editor::DataStorage::UnloadedActor::CvarRegisterActorDescsInTeds->GetBool())
	{
		UActorDescContainerInstance::OnActorDescInstanceAddedToContainer.AddUObject(this, &UTedsActorDescFactory::RegisterActorDesc);
		UActorDescContainerInstance::OnActorDescInstanceRemovedFromContainer.AddUObject(this, &UTedsActorDescFactory::UnregisterActorDesc);
		FWorldPartitionActorDescInstance::OnActorDescUpdated.AddUObject(this, &UTedsActorDescFactory::OnActorDescUpdated);
	}
}

void UTedsActorDescFactory::PreShutdown(UE::Editor::DataStorage::ICoreProvider& DataStorage)
{
	if (UE::Editor::DataStorage::UnloadedActor::CvarRegisterActorDescsInTeds->GetBool())
	{
		UActorDescContainerInstance::OnActorDescInstanceAddedToContainer.RemoveAll(this);
		UActorDescContainerInstance::OnActorDescInstanceRemovedFromContainer.RemoveAll(this);
		FWorldPartitionActorDescInstance::OnActorDescUpdated.RemoveAll(this);
	}
}

void UTedsActorDescFactory::RegisterActorDesc(UActorDescContainerInstance* ContainerInstance, FWorldPartitionActorDescInstance* ActorDescInstance)
{
	using namespace UE::Editor::DataStorage;
	using namespace UE::Editor::DataStorage::UnloadedActor;
	
	if (ICoreProvider* Storage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName))
	{
		TableHandle ActorDescTable = Storage->FindTable(GetActorDescTableName());
		
		if (Storage->IsTableValid(ActorDescTable))
		{
			FMapKey Key = GetActorDescMappingKey(ActorDescInstance);
			FName MappingDomain = GetActorDescMappingDomain();
			
			// The same actor desc can get here multiple times, e.g with level instances since we create another container instance when
			// we load the level instance's world. However from a data standpoint, those two actors descs are exactly the same so we can just
			// ignore the duplicate
			if (!Storage->IsRowAvailable(Storage->LookupMappedRow(MappingDomain, Key)))
			{
				RowHandle ActorDescRowHandle = Storage->AddRow(ActorDescTable);
				Storage->MapRow(MappingDomain, Key, ActorDescRowHandle);
					
				// These are columns that we either need immediately or require local data to initialize
				Storage->AddColumn(ActorDescRowHandle, FWorldPartitionHandleColumn{.Handle = FWorldPartitionHandle(ActorDescInstance)});
				Storage->AddColumn(ActorDescRowHandle, FTypedElementWorldColumn{.World = ContainerInstance->GetWorld()});
					
				// Let processors sync the rest of the columns
				Storage->AddColumn<FTypedElementSyncFromWorldTag>(ActorDescRowHandle);
			}
		}
	}
}

void UTedsActorDescFactory::UnregisterActorDesc(UActorDescContainerInstance* ContainerInstance, FWorldPartitionActorDescInstance* ActorDescInstance)
{
	using namespace UE::Editor::DataStorage;
	using namespace UE::Editor::DataStorage::UnloadedActor;

	if (ICoreProvider* Storage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName))
	{
		RowHandle ActorDescRow = Storage->LookupMappedRow(GetActorDescMappingDomain(), GetActorDescMappingKey(ActorDescInstance));
		
		// Similar to RegisterActorDesc, the same actor desc can get into this call multiple times e.g when a level instance container is unloaded
		// In that case the actual actor desc is still in memory and we don't want to remove the row, so we make sure that the actor desc being
		// unregistered is actually the one represented by this row and not the duplicate from a level instance container being unloaded
		const FWorldPartitionHandleColumn* HandleColumn = Storage->GetColumn<FWorldPartitionHandleColumn>(ActorDescRow);
		if (HandleColumn && *HandleColumn->Handle == ActorDescInstance)
		{
			Storage->RemoveRow(ActorDescRow);
		}
	}
}

void UTedsActorDescFactory::OnActorDescUpdated(FWorldPartitionActorDescInstance* ActorDescInstance)
{
	using namespace UE::Editor::DataStorage;
	using namespace UE::Editor::DataStorage::UnloadedActor;

	// When the actor desc is updated, the data in TEDS needs to be re-synced
	if (ICoreProvider* Storage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName))
	{
		RowHandle ActorDescRow = Storage->LookupMappedRow(GetActorDescMappingDomain(), GetActorDescMappingKey(ActorDescInstance));
		Storage->AddColumn<FTypedElementSyncFromWorldTag>(ActorDescRow);
	}
}

#undef LOCTEXT_NAMESPACE
