// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/Interfaces/TypedElementDataStorageCompatibilityInterface.h"
#include "Elements/Interfaces/TypedElementDataStorageInterface.h"
#include "Elements/Interfaces/TypedElementDataStorageFactory.h"
#include "Engine/World.h"

#include "TedsActorDescFactory.generated.h"

class FWorldPartitionActorDescInstance;
class UActorDescContainerInstance;
class UWorldPartition;

// Factory to manage queries for unloaded actors in TEDS
UCLASS()
class UTedsActorDescFactory : public UEditorDataStorageFactory
{
	GENERATED_BODY()

public:
	virtual void RegisterTables(UE::Editor::DataStorage::ICoreProvider& DataStorage) override;
	virtual void RegisterQueries(UE::Editor::DataStorage::ICoreProvider& DataStorage) override;
	virtual void PreRegister(UE::Editor::DataStorage::ICoreProvider& DataStorage) override;
	virtual void PreShutdown(UE::Editor::DataStorage::ICoreProvider& DataStorage) override;
	
protected:
	void RegisterActorDesc(UActorDescContainerInstance* ContainerInstance, FWorldPartitionActorDescInstance* ActorDescInstance);
	void UnregisterActorDesc(UActorDescContainerInstance* ContainerInstance, FWorldPartitionActorDescInstance* ActorDescInstance);
	void OnActorDescUpdated(FWorldPartitionActorDescInstance* ActorDescInstance);

};