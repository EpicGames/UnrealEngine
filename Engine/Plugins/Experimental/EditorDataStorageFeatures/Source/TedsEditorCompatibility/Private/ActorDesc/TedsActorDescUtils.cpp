// Copyright Epic Games, Inc. All Rights Reserved.

#include "ActorDesc/TedsActorDescUtils.h"
#include "WorldPartition/ActorDescContainerInstance.h"
#include "WorldPartition/WorldPartitionActorDescInstance.h"

namespace UE::Editor::DataStorage::UnloadedActor
{
	namespace Private
	{
		static const FName MappingDomain = "ActorDesc";
		static const FName TableName = "Editor_ActorDescTable";
	}
	
	FName GetActorDescTableName()
	{
		return Private::TableName;
	}
	
	FName GetActorDescMappingDomain()
	{
		return Private::MappingDomain;
	}
	
	FMapKey GetActorDescMappingKey(const FWorldPartitionActorDescInstance* ActorDescInstance)
	{
		// Actor descs can be uniquely identified by combining their container ID + actor desc ID
		return FMapKey(FString::Printf(TEXT("%s.%s"), 
			*ActorDescInstance->GetContainerInstance()->GetContainerID().ToString(), // Container ID
			*ActorDescInstance->GetGuid().ToString())); // Actor desc instance ID
	}

}
