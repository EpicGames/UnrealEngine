// Copyright Epic Games, Inc. All Rights Reserved.

#include "TedsWorldPartitionPinnedQueries.h"

#include "ActorDesc/TedsActorDescColumns.h"
#include "DataStorage/Queries/Types.h"
#include "Elements/Columns/TypedElementCompatibilityColumns.h"
#include "Elements/Columns/TypedElementMiscColumns.h"
#include "Elements/Framework/TypedElementQueryBuilder.h"
#include "GameFramework/Actor.h"
#include "WorldPartition/LoaderAdapter/LoaderAdapterPinnedActors.h"
#include "WorldPartition/WorldPartition.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TedsWorldPartitionPinnedQueries)

void UActorWorldPartitionPinnedDataStorageFactory::RegisterQueries(UE::Editor::DataStorage::ICoreProvider& DataStorage)
{
	RegisterActorAddWorldPartitionPinnedColumn(DataStorage);
	RegisterActorDescAddWorldPartitionPinnedColumn(DataStorage);
	RegisterWorldToColumnQuery(DataStorage);
	RegisterColumnToWorldQuery(DataStorage);
}

void UActorWorldPartitionPinnedDataStorageFactory::PreRegister(UE::Editor::DataStorage::ICoreProvider& DataStorage)
{
	TickerHandle = FTSTicker::GetCoreTicker().AddTicker(
		FTickerDelegate::CreateUObject(this, &UActorWorldPartitionPinnedDataStorageFactory::Tick));
}

void UActorWorldPartitionPinnedDataStorageFactory::PreShutdown(UE::Editor::DataStorage::ICoreProvider& DataStorage)
{
	FTSTicker::GetCoreTicker().RemoveTicker(TickerHandle);
}

bool UActorWorldPartitionPinnedDataStorageFactory::Tick(float)
{
	for (const TPair<TWeakObjectPtr<UWorldPartition>, TArray<FGuid>>& PendingPins : PendingPinMap)
	{
		if (UWorldPartition* WorldPartition = PendingPins.Key.Get())
		{
			WorldPartition->PinActors(PendingPins.Value);
		}
	}
	PendingPinMap.Empty();
	
	for (const TPair<TWeakObjectPtr<UWorldPartition>, TArray<FGuid>>& PendingUnpins : PendingUnpinMap)
	{
		if (UWorldPartition* WorldPartition = PendingUnpins.Key.Get())
		{
			WorldPartition->UnpinActors(PendingUnpins.Value);
		}
	}
	PendingUnpinMap.Empty();
	return true;
}

void UActorWorldPartitionPinnedDataStorageFactory::RegisterActorAddWorldPartitionPinnedColumn(UE::Editor::DataStorage::ICoreProvider& DataStorage) const
{
	using namespace UE::Editor::DataStorage::Queries;

	// Only actors that support pinning get the column. Actor descs are handled separately by
	// RegisterActorDescAddWorldPartitionPinnedColumn since pin support is checked against the actor desc instance.
	DataStorage.RegisterQuery(
		Select(
			TEXT("Actor World Partition Pinned Object to New Column"),
			FProcessor(EQueryTickPhase::DuringPhysics, DataStorage.GetQueryTickGroupName(EQueryTickGroups::SyncExternalToDataStorage)).SetExecutionMode(EExecutionMode::GameThread),
			[](IQueryContext& Context, RowHandle Row, const FTypedElementUObjectColumn& Object, const FGuidColumn& GuidColumn, const FTypedElementWorldColumn& WorldColumn)
			{
				if (AActor* Actor = Cast<AActor>(Object.Object))
				{
					if (FLoaderAdapterPinnedActors::SupportsPinning(Actor))
					{
						bool bIsPinned = false;
						if (const UWorld* World = WorldColumn.World.Get())
						{
							if (const UWorldPartition* WorldPartition = World->GetWorldPartition())
							{
								bIsPinned = WorldPartition->IsActorPinned(GuidColumn.Guid);
							}
						}
						Context.AddColumn(Row, FWorldPartitionPinnedColumn{ .bIsPinned = bIsPinned });
					}
				}
			})
		.Where()
			.All<FTypedElementActorTag, FTypedElementSyncFromWorldTag>()
			.None<FWorldPartitionPinnedColumn>()
		.Compile()
	);
}

void UActorWorldPartitionPinnedDataStorageFactory::RegisterActorDescAddWorldPartitionPinnedColumn(UE::Editor::DataStorage::ICoreProvider& DataStorage) const
{
	using namespace UE::Editor::DataStorage::Queries;

	DataStorage.RegisterQuery(
		Select(
			TEXT("Actor Desc World Partition Pinned Object to New Column"),
			FProcessor(EQueryTickPhase::DuringPhysics, DataStorage.GetQueryTickGroupName(EQueryTickGroups::SyncExternalToDataStorage)).SetExecutionMode(EExecutionMode::GameThread),
			[](IQueryContext& Context, RowHandle Row, const FWorldPartitionHandleColumn& HandleColumn, const FGuidColumn& GuidColumn, const FTypedElementWorldColumn& WorldColumn)
			{
				if (FLoaderAdapterPinnedActors::SupportsPinning(*HandleColumn.Handle))
				{
					bool bIsPinned = false;
					if (const UWorld* World = WorldColumn.World.Get())
					{
						if (const UWorldPartition* WorldPartition = World->GetWorldPartition())
						{
							bIsPinned = WorldPartition->IsActorPinned(GuidColumn.Guid);
						}
					}
					Context.AddColumn(Row, FWorldPartitionPinnedColumn{ .bIsPinned = bIsPinned });
				}
			})
		.Where()
			.All<FActorDescTag, FTypedElementSyncFromWorldTag>()
			.None<FWorldPartitionPinnedColumn>()
		.Compile()
	);
}

void UActorWorldPartitionPinnedDataStorageFactory::RegisterWorldToColumnQuery(UE::Editor::DataStorage::ICoreProvider& DataStorage) const
{
	using namespace UE::Editor::DataStorage::Queries;

	DataStorage.RegisterQuery(
		Select(
			TEXT("World Partition Pinned Object to Column"),
			FProcessor(EQueryTickPhase::PrePhysics, DataStorage.GetQueryTickGroupName(EQueryTickGroups::SyncExternalToDataStorage)).SetExecutionMode(EExecutionMode::GameThread),
			[](IQueryContext&, RowHandle, const FGuidColumn& GuidColumn, FWorldPartitionPinnedColumn& PinnedColumn, const FTypedElementWorldColumn& WorldColumn)
			{
				if (UWorld* World = WorldColumn.World.Get())
				{
					if (const UWorldPartition* WorldPartition = World->GetWorldPartition())
					{
						PinnedColumn.bIsPinned = WorldPartition->IsActorPinned(GuidColumn.Guid);
					}
				}
			})
		.Where(TColumn<FTypedElementSyncFromWorldTag>() && (TColumn<FTypedElementActorTag>() ||TColumn<FActorDescTag>()))
		.Compile()
	);
}

void UActorWorldPartitionPinnedDataStorageFactory::RegisterColumnToWorldQuery(UE::Editor::DataStorage::ICoreProvider& DataStorage)
{
	using namespace UE::Editor::DataStorage::Queries;

	// Usually we would be able to call PinActors/UnpinActors via a deferred command, but pinning/unpinning actors ends up creating new actor or actor desc
	// rows, which in turn fire lots of other observers that also end up pushing command. This gets into a situation where we end up adding new commands
	// to the command buffer while we are iterating it, which is currently not supported.
	// To work around this we defer the pinning/unpinning to tick
	DataStorage.RegisterQuery(
		Select(
			TEXT("Actor World Partition Pinned Column to Object"),
			FProcessor(EQueryTickPhase::PrePhysics, DataStorage.GetQueryTickGroupName(EQueryTickGroups::SyncDataStorageToExternal)).SetExecutionMode(EExecutionMode::GameThread),
			[this](IQueryContext& Context, RowHandle Row, const FGuidColumn& GuidColumn, const FWorldPartitionPinnedColumn& PinnedColumn, const FTypedElementWorldColumn& WorldColumn)
			{
				if (UWorld* World = WorldColumn.World.Get())
				{
					if (UWorldPartition* WorldPartition = World->GetWorldPartition())
					{
						if (PinnedColumn.bIsPinned != WorldPartition->IsActorPinned(GuidColumn.Guid))
						{
							if (PinnedColumn.bIsPinned)
							{
								TArray<FGuid>& PendingPins = PendingPinMap.FindOrAdd(WorldPartition);
								PendingPins.Add(GuidColumn.Guid);
							}
							else
							{
								TArray<FGuid>& PendingUnpins = PendingUnpinMap.FindOrAdd(WorldPartition);
								PendingUnpins.Add(GuidColumn.Guid);
							}
						}
					}
				}
			})
		.Where(TColumn<FTypedElementSyncBackToWorldTag>() && (TColumn<FTypedElementActorTag>() ||TColumn<FActorDescTag>()))
		.Compile()
	);
}
