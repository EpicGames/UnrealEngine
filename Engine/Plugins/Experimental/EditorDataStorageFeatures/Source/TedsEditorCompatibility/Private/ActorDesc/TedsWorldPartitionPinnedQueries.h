// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Ticker.h"
#include "Elements/Interfaces/TypedElementDataStorageFactory.h"
#include "UObject/ObjectMacros.h"

#include "TedsWorldPartitionPinnedQueries.generated.h"

class UWorldPartition;
/**
 * Registers queries that keep FWorldPartitionPinnedColumn in sync with the World Partition
 * pinned actor/actor desc state.
 */
UCLASS()
class UActorWorldPartitionPinnedDataStorageFactory : public UEditorDataStorageFactory
{
	GENERATED_BODY()

public:
	~UActorWorldPartitionPinnedDataStorageFactory() override = default;

	virtual void RegisterQueries(UE::Editor::DataStorage::ICoreProvider& DataStorage) override;
	virtual void PreRegister(UE::Editor::DataStorage::ICoreProvider& DataStorage) override;
	virtual void PreShutdown(UE::Editor::DataStorage::ICoreProvider& DataStorage) override;
private:
	/**
	 * Adds FWorldPartitionPinnedColumn to actor rows that support pinning
	 * and do not already have the column.
	 */
	void RegisterActorAddWorldPartitionPinnedColumn(UE::Editor::DataStorage::ICoreProvider& DataStorage) const;

	/**
	 * Adds FWorldPartitionPinnedColumn to actor desc rows that support pinning
	 * and do not already have the column.
	 */
	void RegisterActorDescAddWorldPartitionPinnedColumn(UE::Editor::DataStorage::ICoreProvider& DataStorage) const;

	/**
	 * Syncs the World Partition pinned state to FWorldPartitionPinnedColumn if they differ.
	 */
	void RegisterWorldToColumnQuery(UE::Editor::DataStorage::ICoreProvider& DataStorage) const;

	/**
	 * Applies the column's bIsPinned value back to the World Partition via PinActors / UnpinActors if the FTypedElementSyncBackToWorldTag
	 * has been set and the bIsPinned differs.
	 */
	void RegisterColumnToWorldQuery(UE::Editor::DataStorage::ICoreProvider& DataStorage);
	
	/**
	 * Custom ticker to process pending pins/unpins since they cannot be done inside a processor
	 */
	bool Tick(float);
	
private:
	
	// Store maps of the actors/actor descs that need to be pinned/unpinned, keyed by the world partition they belong to 
	// so we can batch pin/unpin them at once
	TMap<TWeakObjectPtr<UWorldPartition>, TArray<FGuid>> PendingPinMap;
	TMap<TWeakObjectPtr<UWorldPartition>, TArray<FGuid>> PendingUnpinMap;
	
	FTSTicker::FDelegateHandle TickerHandle;

};
