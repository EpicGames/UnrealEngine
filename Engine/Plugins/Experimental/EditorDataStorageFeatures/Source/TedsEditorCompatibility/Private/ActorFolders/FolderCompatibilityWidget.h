// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/Interfaces/TypedElementDataStorageFactory.h"
#include "Elements/Interfaces/TypedElementDataStorageInterface.h"
#include "Elements/Interfaces/TypedElementDataStorageUiInterface.h"

#include "FolderCompatibilityWidget.generated.h"

UCLASS()
class UFolderCompatibilityWidgetFactory : public UEditorDataStorageFactory
{
	GENERATED_BODY()

public:
	~UFolderCompatibilityWidgetFactory() override = default;

	void RegisterWidgetConstructors(UE::Editor::DataStorage::ICoreProvider& DataStorage,
		UE::Editor::DataStorage::IUiProvider& DataStorageUi) const override;
};

USTRUCT()
struct FFolderCompatibilityWidgetConstructor : public FSimpleWidgetConstructor
{
	GENERATED_BODY()

public:
	FFolderCompatibilityWidgetConstructor();
	~FFolderCompatibilityWidgetConstructor() override = default;

	virtual TSharedPtr<SWidget> CreateWidget(
		UE::Editor::DataStorage::ICoreProvider* DataStorage,
		UE::Editor::DataStorage::IUiProvider* DataStorageUi,
		UE::Editor::DataStorage::RowHandle TargetRow,
		UE::Editor::DataStorage::RowHandle WidgetRow,
		const UE::Editor::DataStorage::FMetaDataView& Arguments) override;
};