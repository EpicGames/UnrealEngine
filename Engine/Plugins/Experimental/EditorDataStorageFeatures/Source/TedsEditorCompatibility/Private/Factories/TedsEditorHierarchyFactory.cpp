// Copyright Epic Games, Inc. All Rights Reserved.

#include "Factories/TedsEditorHierarchyFactory.h"

#include "Elements/Interfaces/TypedElementDataStorageInterface.h"

FName UTedsEditorHierarchyFactory::EditorObjectHierarchyName("EditorObjectHierarchy");

void UTedsEditorHierarchyFactory::RegisterHierarchies(UE::Editor::DataStorage::ICoreProvider& DataStorage)
{
	using namespace UE::Editor::DataStorage;
	
	UE::Editor::DataStorage::FHierarchyRegistrationParams WorldHierarchyParams
	{
		.Name = EditorObjectHierarchyName, 
		.bEnableParentChangedColumn = true
	};
	
	DataStorage.RegisterHierarchy(WorldHierarchyParams);
}