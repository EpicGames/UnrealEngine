// Copyright Epic Games, Inc. All Rights Reserved.

#include "TedsLevelFactory.h"

#include "DataStorage/Features.h"
#include "Elements/Columns/TypedElementCompatibilityColumns.h"
#include "Elements/Columns/TypedElementIconOverrideColumns.h"
#include "Elements/Columns/TypedElementLabelColumns.h"
#include "Elements/Columns/TypedElementMiscColumns.h"
#include "Elements/Columns/TypedElementUIColumns.h"
#include "Elements/Framework/TypedElementQueryBuilder.h"
#include "Elements/Interfaces/TypedElementDataStorageCompatibilityInterface.h"
#include "Elements/Interfaces/TypedElementDataStorageInterface.h"
#include "Engine/Level.h"
#include "Engine/World.h"
#include "Factories/TedsEditorHierarchyFactory.h"
#include "LevelInstance/LevelInstanceSubsystem.h"
#include "Misc/PackageName.h"
#include "Styling/SlateIconFinder.h"
#include "UObject/Package.h"

namespace UE::Editor::DataStorage::Levels
{
	static const FName LevelTableName("Editor_Level");
}

void UTedsLevelFactory::PostRegister(UE::Editor::DataStorage::ICoreProvider& DataStorage)
{
	using namespace UE::Editor::DataStorage;

	// When loading the editor, the start-up level might be created before TEDS is initialized, so we register it manually here.
	if (UWorld* World = GWorld.GetReference())
	{
		if (ULevel* Level = World->PersistentLevel.Get())
		{
			ICompatibilityProvider* Compatibility = GetMutableDataStorageFeature<ICompatibilityProvider>(CompatibilityFeatureName);
			checkf(Compatibility, TEXT("TEDS Factory cannot be init before the data storage compatibility feature is available"));
			
			Compatibility->AddCompatibleObjectExplicit(Level);
		}
	}
}

void UTedsLevelFactory::RegisterTables(UE::Editor::DataStorage::ICoreProvider& DataStorage)
{
	using namespace UE::Editor::DataStorage;
	
	TableHandle LevelTable = DataStorage.RegisterTable<FLevelTag, FTypedElementLabelColumn, FTypedElementIconOverrideColumn>(Levels::LevelTableName);

	ICompatibilityProvider* Compatibility = GetMutableDataStorageFeature<ICompatibilityProvider>(CompatibilityFeatureName);
	checkf(Compatibility, TEXT("TEDS Factory cannot be init before the data storage compatibility feature is available"));
	
	Compatibility->RegisterTypeTableAssociation(ULevel::StaticClass(), LevelTable);
}

void UTedsLevelFactory::RegisterQueries(UE::Editor::DataStorage::ICoreProvider& DataStorage)
{
	using namespace UE::Editor::DataStorage;
	using namespace UE::Editor::DataStorage::Queries;
	
	DataStorage.RegisterQuery(
		Select(
			TEXT("Sync level label to column"),
			FProcessor(EQueryTickPhase::PrePhysics, DataStorage.GetQueryTickGroupName(EQueryTickGroups::SyncExternalToDataStorage))
				.SetExecutionMode(EExecutionMode::GameThread),
			[](const FTypedElementUObjectColumn& LevelColumn, FTypedElementLabelColumn& Label)
			{
				if (const ULevel* Level = Cast<ULevel>(LevelColumn.Object); Level != nullptr)
				{
					Label.Label = FPackageName::GetShortName(Level->GetOutermost()->GetName());
				}
			}
		)
		.Where()
			.All<FLevelTag, FTypedElementSyncFromWorldTag>()
		.Compile());

	DataStorage.RegisterQuery(
		Select(TEXT("Add Icon override to level"),
			FObserver::OnAdd<FTypedElementUObjectColumn>(),
			[](IQueryContext& Context, RowHandle Row, FTypedElementIconOverrideColumn& IconOverrideColumn)
			{
				// ULevels show up with the icon for UWorld in the editor (e.g Outliner)
				FSlateIcon WorldIcon = FSlateIconFinder::FindIconForClass(UWorld::StaticClass());
				IconOverrideColumn.IconName = WorldIcon.GetStyleName();
			})
		.Where()
			.All<FLevelTag>()
		.Compile());
	
	DataStorage.RegisterQuery(
		Select(TEXT("Hide persistent level from UI"),
			FObserver::OnAdd<FTypedElementUObjectColumn>()
				.SetExecutionMode(EExecutionMode::GameThread),
			[](IQueryContext& Context, RowHandle Row, const FTypedElementUObjectColumn& LevelColumn)
			{
				if (const ULevel* Level = Cast<ULevel>(LevelColumn.Object); Level != nullptr)
				{
					if (Level->IsPersistentLevel())
					{
						// The persistent level is hidden from editor UI like the Outliner
						Context.AddColumns<FHideRowFromUITag>(Row);
					}
				}
			})
		.Where()
			.All<FLevelTag>()
		.Compile());

	DataStorage.RegisterQuery(
		Select(TEXT("Hide level instance levels from UI"),
			FObserver::OnAdd<FTypedElementWorldColumn>()
			.SetExecutionMode(EExecutionMode::GameThread),
			[](IQueryContext& Context, RowHandle Row, const FTypedElementUObjectColumn& LevelColumn, const FTypedElementWorldColumn& WorldColumn)
			{
				if (const ULevel* Level = Cast<ULevel>(LevelColumn.Object); Level != nullptr)
				{
					if (const UWorld* World = WorldColumn.World.Get())
					{
						if (const ULevelInstanceSubsystem* LevelInstanceSubsystem = World->GetSubsystem<ULevelInstanceSubsystem>())
						{
							if (LevelInstanceSubsystem->GetOwningLevelInstance(Level) != nullptr)
							{
								Context.AddColumns<FHideRowFromUITag>(Row);
							}
						}
					}
				}
			})
		.Where(TColumn<FLevelTag>())
		.Compile());
	
	// There's no real need for an observer for FPIEObjectTag being removed right now since a PIE level cannot just become an editor level
	DataStorage.RegisterQuery(
		Select(TEXT("Hide PIE levels from UI"),
			FObserver::OnAdd<FPieObjectTag>(),
			[](IQueryContext& Context, RowHandle Row)
			{
				// PIE levels are not meant to be shown in the UI
				Context.AddColumns<FHideRowFromUITag>(Row);
			})
		.Where()
			.All<FLevelTag>()
		.Compile());
	
	DataStorage.RegisterQuery(
		Select(
			TEXT("Sync Level -> World hierarchy"),
			FProcessor(EQueryTickPhase::PrePhysics, DataStorage.GetQueryTickGroupName(EQueryTickGroups::SyncExternalToDataStorage))
			.SetExecutionMode(EExecutionMode::GameThread),
			[](IQueryContext& Context, RowHandle TargetRow, const FTypedElementUObjectColumn& LevelColumn)
			{
				if (const ULevel* Level = Cast<ULevel>(LevelColumn.Object); Level != nullptr)
				{
					if (const UWorld* ParentWorld = Level->GetWorld())
					{
						FMapKeyView IdKey = FMapKeyView(ParentWorld);
						RowHandle NewParentRow = Context.LookupMappedRow(ICompatibilityProvider::ObjectMappingDomain, IdKey);
						RowHandle CurrentParentRow = Context.GetParentRow(TargetRow);

						if (CurrentParentRow != NewParentRow)
						{
							if (Context.IsRowAvailable(NewParentRow))
							{
								Context.SetParentRow(TargetRow, NewParentRow);
							}
							else
							{
								Context.SetUnresolvedParent(TargetRow, FMapKey(ParentWorld), ICompatibilityProvider::ObjectMappingDomain);
							}
						}
					}
					// In case this Level doesn't have a world for some reason, we add a fallback to reset the hierarchy
					else
					{
						Context.SetParentRow(TargetRow, InvalidRowHandle);
					}
				}
			})
		.AccessesHierarchy(UTedsEditorHierarchyFactory::EditorObjectHierarchyName)
		.Where()
			.All<FTypedElementSyncFromWorldTag, FLevelTag>()
		.Compile());
}
