// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/Interfaces/TypedElementDataStorageFactory.h"

#include "TedsLevelFactory.generated.h"

UCLASS()
class UTedsLevelFactory : public UEditorDataStorageFactory
{
	GENERATED_BODY()

public:
	virtual ~UTedsLevelFactory() override = default;

	virtual void RegisterTables(UE::Editor::DataStorage::ICoreProvider& DataStorage) override;
	virtual void RegisterQueries(UE::Editor::DataStorage::ICoreProvider& DataStorage) override;
	virtual void PostRegister(UE::Editor::DataStorage::ICoreProvider& DataStorage) override;
};