// Copyright Epic Games, Inc. All Rights Reserved.

#include "TedsWorldFactory.h"

#include "DataStorage/Features.h"
#include "Elements/Columns/TypedElementCompatibilityColumns.h"
#include "Elements/Interfaces/TypedElementDataStorageCompatibilityInterface.h"
#include "Elements/Interfaces/TypedElementDataStorageInterface.h"
#include "Elements/Columns/TypedElementLabelColumns.h"
#include "Elements/Columns/TypedElementMiscColumns.h"
#include "Elements/Framework/TypedElementQueryBuilder.h"
#include "Engine/Engine.h"
#include "Engine/World.h"

namespace UE::Editor::DataStorage::World
{
	static const FName WorldTableName("Editor_World");
}

void UTedsWorldFactory::RegisterTables(UE::Editor::DataStorage::ICoreProvider& DataStorage)
{
	using namespace UE::Editor::DataStorage;
	
	TableHandle WorldTable = DataStorage.RegisterTable<FWorldTag, FTypedElementLabelColumn>(World::WorldTableName);

	DataStorageCompat = GetMutableDataStorageFeature<ICompatibilityProvider>(CompatibilityFeatureName);
	checkf(DataStorageCompat, TEXT("TEDS Factory cannot be init before the data storage compatibility feature is available"));
	
	DataStorageCompat->RegisterTypeTableAssociation(UWorld::StaticClass(), WorldTable);
}

void UTedsWorldFactory::PreRegister(UE::Editor::DataStorage::ICoreProvider& DataStorage)
{
	FWorldDelegates::OnPostWorldInitialization.AddUObject(this, &UTedsWorldFactory::PostWorldInitialized);
	FWorldDelegates::OnPreWorldFinishDestroy.AddUObject(this, &UTedsWorldFactory::OnPreWorldFinishDestroy);
}

void UTedsWorldFactory::PreShutdown(UE::Editor::DataStorage::ICoreProvider& DataStorage)
{
	FWorldDelegates::OnPostWorldInitialization.RemoveAll(this);
	FWorldDelegates::OnPreWorldFinishDestroy.RemoveAll(this);
}

void UTedsWorldFactory::RegisterQueries(UE::Editor::DataStorage::ICoreProvider& DataStorage)
{
	using namespace UE::Editor::DataStorage;
	using namespace UE::Editor::DataStorage::Queries;
	
	DataStorage.RegisterQuery(
		Select(
			TEXT("Sync world label to column"),
			FProcessor(EQueryTickPhase::PrePhysics, DataStorage.GetQueryTickGroupName(EQueryTickGroups::SyncExternalToDataStorage))
				.SetExecutionMode(EExecutionMode::GameThread),
			[](const FTypedElementUObjectColumn& WorldColumn, FTypedElementLabelColumn& Label)
			{
				if (const UWorld* World = Cast<UWorld>(WorldColumn.Object); World != nullptr)
				{
					// Show the type of world in the name (e.g "MyWorld (Editor)")
					FString WorldTypeString = LexToString(World->WorldType);
					Label.Label = FString::Printf(TEXT("%s (%s)"), *World->GetName(), *WorldTypeString);
				}
			}
		)
		.Where()
			.All<FWorldTag, FTypedElementSyncFromWorldTag>()
		.Compile());
}

void UTedsWorldFactory::PostWorldInitialized(UWorld* InWorld, const UWorld::InitializationValues IVS) const
{
	if (DataStorageCompat)
	{
		DataStorageCompat->AddCompatibleObject(InWorld);
	}
}

void UTedsWorldFactory::OnPreWorldFinishDestroy(UWorld* World) const
{
	if (DataStorageCompat)
	{
		DataStorageCompat->RemoveCompatibleObject(World);
	}
}
