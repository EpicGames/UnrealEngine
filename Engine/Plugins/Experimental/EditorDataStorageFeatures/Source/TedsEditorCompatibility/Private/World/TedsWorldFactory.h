// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataStorage/Handles.h"
#include "Elements/Interfaces/TypedElementDataStorageFactory.h"
#include "Engine/World.h"

#include "TedsWorldFactory.generated.h"

UCLASS()
class UTedsWorldFactory : public UEditorDataStorageFactory
{
	GENERATED_BODY()

public:
	~UTedsWorldFactory() override = default;

	virtual void RegisterTables(UE::Editor::DataStorage::ICoreProvider& DataStorage) override;
	virtual void PreRegister(UE::Editor::DataStorage::ICoreProvider& DataStorage) override;
	virtual void PreShutdown(UE::Editor::DataStorage::ICoreProvider& DataStorage) override;
	virtual void RegisterQueries(UE::Editor::DataStorage::ICoreProvider& DataStorage) override;

private:

	void PostWorldInitialized(UWorld* InWorld, const UWorld::InitializationValues IVS) const;
	void OnPreWorldFinishDestroy(UWorld* World) const;

private:
	UE::Editor::DataStorage::ICompatibilityProvider* DataStorageCompat;
};