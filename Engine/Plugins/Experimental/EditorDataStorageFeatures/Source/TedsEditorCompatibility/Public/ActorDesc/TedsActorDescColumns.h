// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataStorage/CommonTypes.h"
#include "WorldPartition/WorldPartitionHandle.h"

#include "TedsActorDescColumns.generated.h"

/**
 * Tag to identify a row with an unloaded actor
 */
USTRUCT(meta = (DisplayName = "Unloaded Actor"))
struct FEditorDataStorageActorDescTag final : public FEditorDataStorageTag
{
	GENERATED_BODY()
};

/**
 * Column containing the world partition handle for an actor desc in TEDS
 */
USTRUCT(meta = (DisplayName = "World Partition Handle"))
struct FEditorDataStorageWorldPartitionHandleColumn final : public FEditorDataStorageColumn
{
	GENERATED_BODY()
	FWorldPartitionHandle Handle;
};

/**
 * Column that stores the World Partition pinned state of a row
 */
USTRUCT(meta = (DisplayName = "World Partition Pinned"))
struct FEditorDataStorageWorldPartitionPinnedColumn final : public FEditorDataStorageColumn
{
	GENERATED_BODY()

	UPROPERTY()
	bool bIsPinned = false;
};

namespace UE::Editor::DataStorage
{
	using FActorDescTag = FEditorDataStorageActorDescTag;
	using FWorldPartitionHandleColumn = FEditorDataStorageWorldPartitionHandleColumn;
	using FWorldPartitionPinnedColumn = FEditorDataStorageWorldPartitionPinnedColumn;
}