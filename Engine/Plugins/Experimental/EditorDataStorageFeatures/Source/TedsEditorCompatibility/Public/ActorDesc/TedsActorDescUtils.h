// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataStorage/CommonTypes.h"

#define UE_API TEDSEDITORCOMPATIBILITY_API

class FWorldPartitionActorDescInstance;

namespace UE::Editor::DataStorage::UnloadedActor
{
	UE_API FName GetActorDescTableName();
	
	UE_API FName GetActorDescMappingDomain();
	
	UE_API FMapKey GetActorDescMappingKey(const FWorldPartitionActorDescInstance* ActorDescInstance);
}

#undef UE_API