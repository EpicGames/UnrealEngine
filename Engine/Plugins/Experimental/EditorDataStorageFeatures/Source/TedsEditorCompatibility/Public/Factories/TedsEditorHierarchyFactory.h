// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataStorage/CommonTypes.h"
#include "DataStorage/Handles.h"
#include "Elements/Interfaces/TypedElementDataStorageFactory.h"
#include "UObject/ObjectMacros.h"

#include "TedsEditorHierarchyFactory.generated.h"

UCLASS()
class UTedsEditorHierarchyFactory : public UEditorDataStorageFactory
{
	GENERATED_BODY()

public:
	~UTedsEditorHierarchyFactory() override = default;
	
	// Hierarchy for objects in the editor (e.g UWorld, ULevel, AActor)
	TEDSEDITORCOMPATIBILITY_API static FName EditorObjectHierarchyName;

	TEDSEDITORCOMPATIBILITY_API virtual void RegisterHierarchies(UE::Editor::DataStorage::ICoreProvider& DataStorage) override;
};