// Copyright Epic Games, Inc. All Rights Reserved.

#include "Context/TedsPickerContextUtil.h"

#include "DataStorage/Features.h"
#include "DataStorage/Handles.h"
#include "Elements/Framework/TypedElementQueryBuilder.h"

#include "Elements/Columns/TypedElementLabelColumns.h"
#include "Elements/Columns/TypedElementTypeInfoColumns.h"
#include "Elements/Columns/TedsTypeInfoColumns.h"
#include "Elements/Framework/TypedElementQueryContext.h"

#include "Widgets/STedsHierarchyViewer.h"
#include "Widgets/STedsTableViewer.h"

#include "Widgets/SEverythingPicker.h"
#include "Widgets/STedsSearchBox.h"

#include "TedsHierarchyNode.h"
#include "TedsRowArrayNode.h"
#include "TedsQueryNode.h"
#include "TedsRowQueryResultsNode.h"
#include "TedsRowMergeNode.h"

#define LOCTEXT_NAMESPACE "TedsPickerContextUtil"

namespace UE::Editor::DataStorage::Picker
{
	void SObjectReferenceContextView::Construct(const FArguments& InArgs)
	{
		using namespace UE::Editor::DataStorage::Queries;

		ICoreProvider* Storage = GetMutableDataStorageFeature<ICoreProvider>(UE::Editor::DataStorage::StorageFeatureName);

		if (!ensureMsgf(Storage, TEXT("Cannot create a TEDS ObjectReferenceContext widget before TEDS is initialized")))
		{
			ChildSlot
				[
					SNew(STextBlock)
						.Text(LOCTEXT("ObjectReferenceErrorCheck", "No valid editor data storage available."))
				];
			return;
		}

		TSharedPtr<QueryStack::IRowNode> FinalNode = nullptr;
		if (!InArgs._QueryStack.IsValid())
		{
			TSharedRef<QueryStack::FQueryNode> ReferenceQueryNode =
				MakeShared<QueryStack::FQueryNode>(*Storage, InArgs._Query);

			if (ReferenceQueryNode->GetQuery() != InvalidQueryHandle)
			{
				FinalNode =
					MakeShared<QueryStack::FRowQueryResultsNode>(*Storage,
						ReferenceQueryNode, QueryStack::FRowQueryResultsNode::ESyncActions::ForceRefreshOnUpdate);
			}
			else
			{
				FinalNode = MakeShared<QueryStack::FRowArrayNode>(FRowHandleArray());
			}
		}
		else
		{
			FinalNode = InArgs._QueryStack;
		}

		// Append a search node on the end
		TSharedPtr<QueryStack::IRowNode> SearchNode;
		TSharedPtr<SBox> SearchBoxContainer = SNew(SBox);
		if(InArgs._SearchingEnabled)
		{
			SearchBoxContainer->SetContent(SNew(STedsSearchBox)
				.InSearchableRowNode(FinalNode)
				.OutSearchNode(&SearchNode));
		}

		if(SearchNode)
		{
			FinalNode = SearchNode;
		}
		
		// Using the Teds Outliner Purpose for matching customization for now
		IUiProvider::FPurposeID PurposeId = IUiProvider::FPurposeInfo("SceneOutliner", "RowLabel", NAME_None).GeneratePurposeID();

		TSharedPtr<FHierarchyViewerData> HierarchyData = MakeShared<FHierarchyViewerData>(*Storage, InArgs._HierarchyHandle);
		
		TSharedPtr<SHierarchyViewer> ReferenceView = SNew(SHierarchyViewer, HierarchyData)
			.AllNodeProvider(FinalNode)
			.Columns(InArgs._Columns)
			.CellWidgetPurpose(PurposeId)
			.OnSelectionChanged(InArgs._OnSelectionChanged);

		ReferenceView->ExpandAll();
		
		ChildSlot
			[
				SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(4.0f, 4.0f)
				[
					SearchBoxContainer.ToSharedRef()
				]
				+ SVerticalBox::Slot()
				[
					ReferenceView.ToSharedRef()
				]
			];
	}

	void STypeListContextView::Construct(const FArguments& InArgs)
	{
		using namespace UE::Editor::DataStorage::Queries;

		ICoreProvider* Storage = GetMutableDataStorageFeature<ICoreProvider>(UE::Editor::DataStorage::StorageFeatureName);

		if (!ensureMsgf(Storage, TEXT("Cannot create a TEDS TypeListContext widget before TEDS is initialized")))
		{
			ChildSlot
				[
					SNew(STextBlock)
						.Text(LOCTEXT("TypeListErrorText", "No valid editor data storage available."))
				];
			return;
		}

		// Default FinalNode to an empty row list
		TSharedPtr<QueryStack::IRowNode> FinalNode = nullptr;

		bool bHasValidQuery = false;
		if (!InArgs._QueryStack.IsValid())
		{
			TSharedRef<QueryStack::FQueryNode> TypeQueryNode =
				MakeShared<QueryStack::FQueryNode>(*Storage, InArgs._Query);

			bHasValidQuery = TypeQueryNode->GetQuery() != InvalidQueryHandle;
			if (bHasValidQuery)
			{
				FinalNode =
					MakeShared<QueryStack::FRowQueryResultsNode>(*Storage,
						TypeQueryNode, QueryStack::FRowQueryResultsNode::ESyncActions::ForceRefreshOnUpdate);
			}
			else
			{
				FinalNode = MakeShared<QueryStack::FRowArrayNode>(FRowHandleArray());
			}
		}
		else
		{
			FinalNode = InArgs._QueryStack;
			bHasValidQuery = true;
		}

		if (InArgs._BaseType)
		{
			RowHandle BaseClassRowHandle = Storage->LookupMappedRow(TypeInfo::TypeMappingDomain, FMapKey(InArgs._BaseType));

			if (BaseClassRowHandle != InvalidRowHandle)
			{
				FRowHandleArray BaseClassHandleArray;
				BaseClassHandleArray.Add(BaseClassRowHandle);

				TSharedPtr<QueryStack::IRowNode> BaseClassRowNode =
					MakeShared<QueryStack::FRowArrayNode>(BaseClassHandleArray);

				// Acquire the row list of BaseClass + SubClasses
				FHierarchyHandle ClassHierarchyHandle = Storage->FindHierarchyByName(TEXT("ClassHierarchy"));
				TSharedRef<QueryStack::FHierarchyRowNode> HierarchyNode =
					MakeShared<QueryStack::FHierarchyRowNode>(*Storage, ClassHierarchyHandle, BaseClassRowNode, QueryStack::FHierarchyRowNode::ESyncFlags::IncrementWhenDifferent);

				// Do an initial update to populate the node
				QueryStack::FExplicitUpdateExecutor("STypeListContextView Query Stack one-shot", HierarchyNode).Update();

				// If we had a valid query and a valid base class we will take the intersection of the two lists as our final
				if (bHasValidQuery)
				{
					TSharedPtr<QueryStack::IRowNode> MergeNodes[] = { FinalNode, HierarchyNode.ToSharedPtr() };
					FinalNode =
						MakeShared<QueryStack::FRowMergeNode>(MergeNodes, QueryStack::FRowMergeNode::EMergeApproach::Repeating);
				}
				else
				{
					// No query but valid base class. We'll display the hierarchy list of BaseClass + SubClasses
					FinalNode = HierarchyNode;
				}
			}
		}

		TSharedPtr<QueryStack::IRowNode> SearchNode;
		TSharedPtr<SBox> SearchBoxContainer = SNew(SBox);
		if (InArgs._SearchingEnabled)
		{
			SearchBoxContainer->SetContent(SNew(STedsSearchBox)
				.InSearchableRowNode(FinalNode)
				.OutSearchNode(&SearchNode));
		}

		if (SearchNode)
		{
			FinalNode = SearchNode;
		}

		TSharedPtr<STedsTableViewer> TypeListView = SNew(STedsTableViewer)
			.QueryStack(FinalNode)
			.Columns({ FTypedElementClassTypeInfoColumn::StaticStruct() })
			.OnSelectionChanged(InArgs._OnSelectionChanged);

		ChildSlot
			[
				SNew(SVerticalBox)
					+ SVerticalBox::Slot()
					.AutoHeight()
					.Padding(4.0f, 4.0f)
					[
						SearchBoxContainer.ToSharedRef()
					]
					+ SVerticalBox::Slot()
					[
						TypeListView.ToSharedRef()
					]
			];
	}
}
#undef LOCTEXT_NAMESPACE // TedsPickerContextUtil