// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modules/ModuleManager.h"
#include "Modules/ModuleInterface.h"

class FTedsEverythingPickerModule : public IModuleInterface
{
public:

	virtual void StartupModule() override
	{
	}

	virtual void ShutdownModule() override
	{
	}
};

IMPLEMENT_MODULE(FTedsEverythingPickerModule, TedsEverythingPicker);

