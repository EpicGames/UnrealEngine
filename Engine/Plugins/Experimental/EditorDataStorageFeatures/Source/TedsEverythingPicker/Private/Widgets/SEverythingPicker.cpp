// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SEverythingPicker.h"

#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Layout/SWidgetSwitcher.h"
#define LOCTEXT_NAMESPACE "SEverythingPicker"

namespace UE::Editor::DataStorage::Picker
{
	SLATE_IMPLEMENT_WIDGET(SEverythingPicker)
	void SEverythingPicker::PrivateRegisterAttributes(FSlateAttributeInitializer& AttributeInitializer)
	{
		SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION(AttributeInitializer, WidthOverride, EInvalidateWidgetReason::Layout);
		SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION(AttributeInitializer, HeightOverride, EInvalidateWidgetReason::Layout);
		SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION(AttributeInitializer, MinDesiredWidth, EInvalidateWidgetReason::Layout);
		SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION(AttributeInitializer, MinDesiredHeight, EInvalidateWidgetReason::Layout);
		SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION(AttributeInitializer, MaxDesiredWidth, EInvalidateWidgetReason::Layout);
		SLATE_ADD_MEMBER_ATTRIBUTE_DEFINITION(AttributeInitializer, MaxDesiredHeight, EInvalidateWidgetReason::Layout);
	}

	SEverythingPicker::SEverythingPicker()
		: WidthOverride(*this, 350.0f)
		, HeightOverride(*this, 300.0f)
		, MinDesiredWidth(*this)
		, MinDesiredHeight(*this)
		, MaxDesiredWidth(*this)
		, MaxDesiredHeight(*this)
	{
	}

	SEverythingPicker::~SEverythingPicker() = default;

	TAutoConsoleVariable<bool> CVarPickerSystemEnabled(
		TEXT("TEDS.Feature.PickerEnabled"),
		false, // Defaulting to false until we hit feature parity (search bar, custom filtering, etc.)
		TEXT("If true, enables uses of TED's EverythingPicker for testing."));

	void SEverythingPicker::Construct(const FArguments& InArgs)
	{
		SetWidthOverride(InArgs._WidthOverride);
		SetHeightOverride(InArgs._HeightOverride);

		SetMinDesiredWidth(InArgs._MinDesiredWidth);
		SetMinDesiredHeight(InArgs._MinDesiredHeight);
		SetMaxDesiredWidth(InArgs._MaxDesiredWidth);
		SetMaxDesiredHeight(InArgs._MaxDesiredHeight);

		ContextTabs = SNew(SHorizontalBox);
		ContextTabViews = SNew(SWidgetSwitcher);

		for (const FPickerContext::FSlotArguments& ContextArgs : InArgs._Contexts)
		{
			AddContext(ContextArgs);
		}

		// In cases where we have a single tab (i.e. single context), it might not be desirable to
		//  still show the tab label.
		const bool bShowTabs = ContextTabViews->GetNumWidgets() > 1 || InArgs._ShowTabLabelWhenSingle;

		ContainerBox = SNew(SBox)
			.WidthOverride(WidthOverride.Get())
			.HeightOverride(HeightOverride.Get())
			.MinDesiredWidth(MinDesiredWidth.Get())
			.MinDesiredHeight(MinDesiredHeight.Get())
			.MaxDesiredWidth(MaxDesiredWidth.Get())
			.MaxDesiredHeight(MaxDesiredHeight.Get())
			[
				// When showing tabs, we use a vertical box with the tab display at the top and contents below
				bShowTabs ? TSharedRef<SWidget>(
					SNew(SVerticalBox)
					+ SVerticalBox::Slot()
					.AutoHeight()
					.Padding(8, 2)
					[
						ContextTabs.ToSharedRef()
					]
					+ SVerticalBox::Slot()
					.VAlign(VAlign_Fill)
					.FillHeight(1.0)
					.Padding(3)
					[
						ContextTabViews.ToSharedRef()
					])
				// When not showing tabs, we just have the section contents
				: TSharedRef<SWidget>(ContextTabViews.ToSharedRef())
			];

		if (bShowTabs)
		{
			// When showing tabs, we also wrap the whole thing in a menu with a "View" section label
			FMenuBuilder MenuBuilder(true, nullptr);
			MenuBuilder.BeginSection(NAME_None, LOCTEXT("PickerView", "View"));
			MenuBuilder.AddWidget(ContainerBox.ToSharedRef(), FText::GetEmpty(), true);
			MenuBuilder.EndSection();

			ChildSlot[ MenuBuilder.MakeWidget() ];
		}
		else
		{
			// When not showing tabs, the content is bare, without a label
			ChildSlot[ ContainerBox.ToSharedRef() ];
		}
	}

	void SEverythingPicker::AddContext(const FPickerContext::FSlotArguments& SlotArgs)
	{
		int32 NextContextId = ContextTabViews->GetNumWidgets();

		ContextTabs->AddSlot()
		.AutoWidth()
		.Padding(4)
		[
			SNew(SCheckBox)
			.Style(&FAppStyle::GetWidgetStyle<FCheckBoxStyle>("ToggleButtonCheckBox"))
			.IsChecked(this, &SEverythingPicker::IsContextTabSelected, NextContextId)
			.OnCheckStateChanged(this, &SEverythingPicker::OnActiveContextChanged, NextContextId)
			[
				SNew(STextBlock)
				.Text(SlotArgs._Label)
			]
		];

		TSharedPtr<SWidget> ContextWidget = SlotArgs.GetAttachedWidget();
		if (ContextWidget)
		{
			ContextTabViews->AddSlot()
			[
				ContextWidget.ToSharedRef()
			];
		}
		else
		{
			ContextTabViews->AddSlot()
			[
				SNew(SBox)
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("EmptyContextViewText", "Empty View"))
				]
			];
		}
	}

	ECheckBoxState SEverythingPicker::IsContextTabSelected(int32 ContextId) const
	{
		return (ContextId == ActiveContextId) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
	}

	void SEverythingPicker::OnActiveContextChanged(ECheckBoxState State, int32 ContextId)
	{
		if (State != ECheckBoxState::Checked)
		{
			return;
		}

		if (ContextId < 0 || ContextId >= ContextTabViews->GetNumWidgets())
		{
			return;
		}

		ActiveContextId = ContextId;
		ContextTabViews->SetActiveWidgetIndex(ActiveContextId);
	}

	void SEverythingPicker::SetWidthOverride(TAttribute<FOptionalSize> InWidthOverride)
	{
		WidthOverride.Assign(*this, InWidthOverride);

		if (ContainerBox)
		{
			ContainerBox->SetWidthOverride(WidthOverride.Get());
		}
	}

	void SEverythingPicker::SetHeightOverride(TAttribute<FOptionalSize> InHeightOverride)
	{
		HeightOverride.Assign(*this, InHeightOverride);

		if (ContainerBox)
		{
			ContainerBox->SetHeightOverride(HeightOverride.Get());
		}
	}

	void SEverythingPicker::SetMinDesiredWidth(TAttribute<FOptionalSize> InMinDesiredWidth)
	{
		MinDesiredWidth.Assign(*this, InMinDesiredWidth);

		if (ContainerBox)
		{
			ContainerBox->SetMinDesiredWidth(MinDesiredWidth.Get());
		}
	}

	void SEverythingPicker::SetMinDesiredHeight(TAttribute<FOptionalSize> InMinDesiredHeight)
	{
		MinDesiredHeight.Assign(*this, InMinDesiredHeight);

		if (ContainerBox)
		{
			ContainerBox->SetMinDesiredHeight(MinDesiredHeight.Get());
		}
	}

	void SEverythingPicker::SetMaxDesiredWidth(TAttribute<FOptionalSize> InMaxDesiredWidth)
	{
		MaxDesiredWidth.Assign(*this, InMaxDesiredWidth);

		if (ContainerBox)
		{
			ContainerBox->SetMaxDesiredWidth(MaxDesiredWidth.Get());
		}
	}

	void SEverythingPicker::SetMaxDesiredHeight(TAttribute<FOptionalSize> InMaxDesiredHeight)
	{
		MaxDesiredHeight.Assign(*this, InMaxDesiredHeight);

		if (ContainerBox)
		{
			ContainerBox->SetMaxDesiredHeight(MaxDesiredHeight.Get());
		}
	}
}

#undef LOCTEXT_NAMESPACE // "SEverythingPicker"