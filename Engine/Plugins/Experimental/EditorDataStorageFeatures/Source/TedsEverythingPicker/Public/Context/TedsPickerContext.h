// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SlotBase.h"
#include "Templates/SharedPointer.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "ITedsTableViewer.h"
#include "TedsQueryStackInterfaces.h"

namespace UE::Editor::DataStorage
{
	class FPickerContext : public TSlotBase<FPickerContext>
	{
	public:
		using TSlotBase<FPickerContext>::TSlotBase;

		SLATE_SLOT_BEGIN_ARGS(FPickerContext, TSlotBase<FPickerContext>)
			SLATE_ARGUMENT(FText, Label)
		SLATE_SLOT_END_ARGS()
	};
}
