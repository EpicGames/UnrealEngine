// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataStorage/CommonTypes.h"
#include "Context/TedsPickerContext.h"
#include "Elements/Columns/TypedElementLabelColumns.h"
#include "Elements/Columns/TypedElementTypeInfoColumns.h"
#include "Elements/Framework/TypedElementQueryBuilder.h"

#include "Widgets/SCompoundWidget.h"

#define UE_API TEDSEVERYTHINGPICKER_API

namespace UE::Editor::DataStorage::QueryStack
{
	class FRowFilterNode;
	class IRowNode;
}

namespace UE::Editor::DataStorage::Picker
{
	DECLARE_DELEGATE_TwoParams(FOnSelectionChanged, RowHandle, ESelectInfo::Type);

	class SObjectReferenceContextView : public SCompoundWidget
	{
	public:
		SLATE_BEGIN_ARGS(SObjectReferenceContextView)
			: _SearchingEnabled(false)
			, _Columns({ FTypedElementLabelColumn::StaticStruct(), FTypedElementClassTypeInfoColumn::StaticStruct() })
			{}
			SLATE_ARGUMENT(FQueryDescription, Query)
			SLATE_ARGUMENT(TSharedPtr<QueryStack::IRowNode>, QueryStack) // Advanced: QueryStack allows finer control over displayed data. Will be used instead of Query if specified
			SLATE_ARGUMENT(FOnSelectionChanged, OnSelectionChanged)
			SLATE_ARGUMENT(FHierarchyHandle, HierarchyHandle)
			SLATE_ARGUMENT(bool, SearchingEnabled)
			SLATE_ARGUMENT(TArray<TWeakObjectPtr<const UScriptStruct>>, Columns) // Advanced: Columns to display in the view.
		SLATE_END_ARGS()

		UE_API void Construct(const FArguments& InArgs);
	};

	class STypeListContextView : public SCompoundWidget
	{
	public:
		SLATE_BEGIN_ARGS(STypeListContextView)
			: _SearchingEnabled(false)
			{}
			SLATE_ARGUMENT(FQueryDescription, Query) // If set will perform the requested query. Ex: Select().Where().All<FClassTypeInfoTag>().Compile(). Results Intersect with BaseType
			SLATE_ARGUMENT(TSharedPtr<QueryStack::IRowNode>, QueryStack) // Advanced: QueryStack allows finer control over displayed data. Will be used instead of Query if specified
			SLATE_ARGUMENT(UStruct*, BaseType) // If set, will collect a list of BaseType + SubTypes. Results intersect with Query
			SLATE_ARGUMENT(FOnSelectionChanged, OnSelectionChanged)
			SLATE_ARGUMENT(bool, SearchingEnabled)
		SLATE_END_ARGS()

		UE_API void Construct(const FArguments& InArgs);
	};
}

#undef UE_API
