// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

#include "Styling/SlateTypes.h"

#include "Context/TedsPickerContext.h"

class SBox;
class SWidgetSwitcher;
class SHorizontalBox;

namespace UE::Editor::DataStorage::Picker
{
	class SEverythingPicker : public SCompoundWidget
	{
		SLATE_DECLARE_WIDGET_API(SEverythingPicker, SCompoundWidget, TEDSEVERYTHINGPICKER_API)

	public:
		SLATE_BEGIN_ARGS(SEverythingPicker)
			: _WidthOverride(FOptionalSize())
			, _HeightOverride(FOptionalSize())
			, _MinDesiredWidth(FOptionalSize())
			, _MinDesiredHeight(FOptionalSize())
			, _MaxDesiredWidth(FOptionalSize())
			, _MaxDesiredHeight(FOptionalSize())
			, _ShowTabLabelWhenSingle(true)
			{}
			SLATE_SLOT_ARGUMENT(FPickerContext, Contexts)

			SLATE_ATTRIBUTE(FOptionalSize, WidthOverride)

			SLATE_ATTRIBUTE(FOptionalSize, HeightOverride)

			SLATE_ATTRIBUTE(FOptionalSize, MinDesiredWidth)

			SLATE_ATTRIBUTE(FOptionalSize, MinDesiredHeight)

			SLATE_ATTRIBUTE(FOptionalSize, MaxDesiredWidth)

			SLATE_ATTRIBUTE(FOptionalSize, MaxDesiredHeight)

			// When there is a single tab (i.e. single FPickerContext), whether to still
			//  show the tab label or to make the picker more minimal by just showing the
			//  tab contents. Defaults to true (showing tab).
			SLATE_ARGUMENT(bool, ShowTabLabelWhenSingle)
		SLATE_END_ARGS()

		TEDSEVERYTHINGPICKER_API SEverythingPicker();
		TEDSEVERYTHINGPICKER_API virtual ~SEverythingPicker();

		TEDSEVERYTHINGPICKER_API void Construct(const FArguments& InArgs);

		TEDSEVERYTHINGPICKER_API void AddContext(const FPickerContext::FSlotArguments& SlotArgs);

		static FPickerContext::FSlotArguments Context()
		{
			return FPickerContext::FSlotArguments(MakeUnique<FPickerContext>());
		}

		static FPickerContext::FSlotArguments Context(FPickerContext::FSlotArguments&& InContext)
		{
			return MoveTemp(InContext);
		}

		/** See WidthOverride attribute */
		TEDSEVERYTHINGPICKER_API void SetWidthOverride(TAttribute<FOptionalSize> InWidthOverride);

		/** See HeightOverride attribute */
		TEDSEVERYTHINGPICKER_API void SetHeightOverride(TAttribute<FOptionalSize> InHeightOverride);

		/** See MinDesiredWidth attribute */
		TEDSEVERYTHINGPICKER_API void SetMinDesiredWidth(TAttribute<FOptionalSize> InMinDesiredWidth);

		/** See MinDesiredHeight attribute */
		TEDSEVERYTHINGPICKER_API void SetMinDesiredHeight(TAttribute<FOptionalSize> InMinDesiredHeight);

		/** See MaxDesiredWidth attribute */
		TEDSEVERYTHINGPICKER_API void SetMaxDesiredWidth(TAttribute<FOptionalSize> InMaxDesiredWidth);

		/** See MaxDesiredHeight attribute */
		TEDSEVERYTHINGPICKER_API void SetMaxDesiredHeight(TAttribute<FOptionalSize> InMaxDesiredHeight);

	private:

		ECheckBoxState IsContextTabSelected(int32 ContextId) const;
		void OnActiveContextChanged(ECheckBoxState State, int32 ContextId);

		int32 ActiveContextId = 0;
		TSharedPtr<SBox> ContainerBox;
		TSharedPtr<SHorizontalBox> ContextTabs;
		TSharedPtr<SWidgetSwitcher> ContextTabViews;

		TSlateAttribute<FOptionalSize> WidthOverride;

		TSlateAttribute<FOptionalSize> HeightOverride;

		TSlateAttribute<FOptionalSize> MinDesiredWidth;

		TSlateAttribute<FOptionalSize> MinDesiredHeight;

		TSlateAttribute<FOptionalSize> MaxDesiredWidth;

		TSlateAttribute<FOptionalSize> MaxDesiredHeight;
	};
}