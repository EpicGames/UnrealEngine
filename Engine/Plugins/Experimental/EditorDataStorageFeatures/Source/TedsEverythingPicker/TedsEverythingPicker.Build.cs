// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class TedsEverythingPicker : ModuleRules
{
	public TedsEverythingPicker(ReadOnlyTargetRules Target) : base(Target)
	{
		// Enable truncation warnings in this module
		CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Warning;

		if (Target.bBuildEditor)
		{
			PublicIncludePaths.AddRange(new string[] {});
			PrivateIncludePaths.AddRange(new string[] {});

			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"TypedElementFramework",
				});

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
					"Engine",
					"Slate",
					"SlateCore",
					"TedsQueryStack",
					"TedsTableViewer",
					"TedsTypeInfo",
				});
		}
	}
}