// Copyright Epic Games, Inc. All Rights Reserved.

#include "Deletion/DeletionOperationSystem.h"

#include "Deletion/DeletionOperationInput.h"
#include "Elements/Framework/TypedElementQueryBuilder.h"
#include "TedsOperationColumns.h"

UDeletionOperationSystem::UDeletionOperationSystem()
	: InputTableForce(UE::Editor::DataStorage::InvalidTableHandle)
	, InputTableForceDescr(UE::Editor::DataStorage::InvalidTableHandle)
{
}

UDeletionOperationSystem::~UDeletionOperationSystem()
{
}

void UDeletionOperationSystem::RegisterTables(UE::Editor::DataStorage::ICoreProvider& Storage)
{
	using namespace UE::Editor::DataStorage::Operations;
	OperationTable  = Storage.RegisterTable<FApplyColumn, FDeletionTag>("DeletionOperationTable");

	InputTable = Storage.RegisterTable<FSourceColumn, FDeletionTag>("DeletionOperationInputTable");
	InputTableDescr = Storage.RegisterTable<FSourceColumn, FDeletionTag, FDescriptionColumn>("DeletionOperationInputTable_Descr");
	InputTableForce = Storage.RegisterTable<FSourceColumn, FDeletionTag, FDeletionForceTag>("DeletionOperationInputTable_Force");
	InputTableForceDescr = Storage.RegisterTable<FSourceColumn, FDeletionTag, FDeletionForceTag, FDescriptionColumn>("DeletionOperationInputTable_ForceDescr");
}

bool UDeletionOperationSystem::CreateInputRows(UE::Editor::DataStorage::FRowHandleArray& OutInputRows,
	UE::Editor::DataStorage::FRowHandleArrayView SourceRows, bool bForce, bool bAddDescription) const
{
	using namespace UE::Editor::DataStorage;
	using namespace UE::Editor::DataStorage::Operations;

	OutInputRows.Reset();
	if (SourceRows.IsEmpty())
	{
		return true;
	}
	
	TableHandle Table;
	if (bForce)
	{
		Table = bAddDescription ? InputTableForceDescr : InputTableForce;
	}
	else
	{
		Table = bAddDescription ? InputTableDescr : InputTable;
	}
	
	if (Table == InvalidTableHandle)
	{
		return false;
	}

	OutInputRows.Reserve(SourceRows.Num());
	const RowHandle* SourceRowIt = SourceRows.begin();
	return StoragePtr->BatchAddRow(Table, SourceRows.Num(), [this, &SourceRowIt, &OutInputRows](RowHandle Row)
		{
			StoragePtr->AddColumn(Row, FSourceColumn{ .Value = *SourceRowIt++ });
			OutInputRows.Add(Row);
		});
}

void UDeletionOperationSystem::GetQueryDescription(UE::Editor::DataStorage::FQueryDescription& OutDescription) const
{
	using namespace UE::Editor::DataStorage::Queries;
	using namespace UE::Editor::DataStorage::Operations;

	OutDescription = Select().Where().All<FApplyColumn, FDeletionTag>().Compile();
}
