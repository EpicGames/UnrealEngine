// Copyright Epic Games, Inc. All Rights Reserved.

#include "DragAndDrop/DropOperationSystem.h"

#include "DragAndDrop/DropOperationInput.h"
#include "Elements/Framework/TypedElementQueryBuilder.h"
#include "TedsOperationColumns.h"
#include "TedsQueryNode.h"

UDropOperationSystem::UDropOperationSystem()
{
}

UDropOperationSystem::~UDropOperationSystem()
{
}

void UDropOperationSystem::RegisterTables(UE::Editor::DataStorage::ICoreProvider& Storage)
{
	using namespace UE::Editor::DataStorage::Operations;

	OperationTable  = Storage.RegisterTable<FApplyColumn, FDropTag>("DropOperationTable");
	InputTable      = Storage.RegisterTable<FDropTargetColumn, FSourceColumn>("DropOperationInputTable");
	InputTableDescr = Storage.RegisterTable<FDropTargetColumn, FSourceColumn, FDescriptionColumn>("DropOperationInputTableDescr");
}

bool UDropOperationSystem::CreateInputRows(UE::Editor::DataStorage::FRowHandleArray& OutRows, UE::Editor::DataStorage::FRowHandleArrayView SourceRows,
	UE::Editor::DataStorage::RowHandle TargetRow, bool bAddDescription) const
{
	using namespace UE::Editor::DataStorage;
	using namespace UE::Editor::DataStorage::Operations;
	
	OutRows.Reset();
	if (SourceRows.IsEmpty())
	{
		return true;
	}
	
	TableHandle Table = bAddDescription ? InputTableDescr : InputTable;
	if (Table == InvalidTableHandle)
	{
		return false;
	}
	
	OutRows.Reserve(SourceRows.Num());
	const RowHandle* SourceRowIt = SourceRows.begin();
	return StoragePtr->BatchAddRow(Table, SourceRows.Num(), [this, &OutRows, &SourceRowIt, TargetRow](RowHandle Row)
		{
			StoragePtr->AddColumn(Row, FSourceColumn{ .Value = *SourceRowIt++ });
			StoragePtr->AddColumn(Row, FDropTargetColumn{ .Value = TargetRow });
			OutRows.Add(Row);
		});
}

void UDropOperationSystem::GetQueryDescription(UE::Editor::DataStorage::FQueryDescription& OutDescription) const
{
	using namespace UE::Editor::DataStorage::Queries;
	using namespace UE::Editor::DataStorage::Operations;

	OutDescription = Select().Where().All<FApplyColumn, FDropTag>().Compile();
}
