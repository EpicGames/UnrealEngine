// Copyright Epic Games, Inc. All Rights Reserved
// .
#include "DragAndDrop/TedsDragDropOp.h"

namespace UE::Editor::DataStorage::Widgets
{
	void FTedsDragDropOp::Init(FRowHandleArray InDraggedRows)
	{
		DraggedRows = MoveTemp(InDraggedRows);
	}
	
	TSharedRef<FTedsDragDropOp> FTedsDragDropOp::New(FRowHandleArray InDraggedRows)
	{
		TSharedRef<FTedsDragDropOp> Operation = MakeShareable(new FTedsDragDropOp);
		Operation->Init(MoveTemp(InDraggedRows));
		Operation->Construct();
		return Operation;
	}
}
