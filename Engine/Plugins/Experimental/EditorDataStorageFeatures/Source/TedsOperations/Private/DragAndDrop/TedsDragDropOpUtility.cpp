// Copyright Epic Games, Inc. All Rights Reserved.

#include "DragAndDrop/TedsDragDropOpUtility.h"

#include "DataStorage/Features.h"
#include "DataStorage/MapKey.h"
#include "DragAndDrop/AssetDragDropOp.h"
#include "DragAndDrop/ClassDragDropOp.h"
#include "DragAndDrop/TedsDragDropOp.h"
#include "Elements/Framework/TypedElementRowHandleArray.h"
#include "Elements/Interfaces/TypedElementDataStorageInterface.h"
#include "Input/DragAndDrop.h"

namespace UE::Editor::DataStorage::DragAndDrop
{

bool GetRowsFromData(FRowHandleArray& OutRows, const FDragDropOperation& InData)
{
	if (InData.IsOfType<Widgets::FTedsDragDropOp>())
	{
		OutRows = static_cast<const Widgets::FTedsDragDropOp&>(InData).GetRowArray();
		return true;
	}
	else if (InData.IsOfType<FAssetDragDropOp>())
	{
		if (const ICoreProvider* Storage = GetDataStorageFeature<ICoreProvider>(StorageFeatureName))
		{
			bool bResult = true;
			for (const FAssetData& Element : static_cast<const FAssetDragDropOp&>(InData).GetAssets())
			{
				if (!Element.IsValid())
				{
					bResult = false;
					continue;
				}
				
				RowHandle Row = Storage->LookupMappedRow(/*UE::Editor::AssetData::MappingDomain*/"Asset", FMapKey(Element.GetSoftObjectPath()));
				if (Row == InvalidRowHandle)
				{
					bResult = false;
					continue;
				}
				
				OutRows.Add(Row);
			}
			
			bool bHasFactory = static_cast<const FAssetDragDropOp&>(InData).GetAssetFactory() != nullptr;
			if (bHasFactory)
			{
				// If an asset factory is defined, we "lose" data by ignoring it, so return false.
				return false;
			}
			
			return bResult;
		}
	}
	else if (InData.IsOfType<FClassDragDropOp>())
	{
		if (const ICoreProvider* Storage = GetDataStorageFeature<ICoreProvider>(StorageFeatureName))
		{
			bool bResult = true;
			for (const TWeakObjectPtr<UClass>& ClassPtr : static_cast<const FClassDragDropOp&>(InData).ClassesToDrop)
			{
				const UClass* Class = ClassPtr.Get();
				if (!Class)
				{
					bResult = false;
					continue;
				}
				
				RowHandle Row = Storage->LookupMappedRow(/*UE::Editor::AssetData::MappingDomain*/"Asset", FMapKey(Class->GetPathName()));
				if (Row == InvalidRowHandle)
				{
					bResult = false;
					continue;
				}
				
				OutRows.Add(Row);
			}
			return bResult;
		}
	}
	return false;
}

}
