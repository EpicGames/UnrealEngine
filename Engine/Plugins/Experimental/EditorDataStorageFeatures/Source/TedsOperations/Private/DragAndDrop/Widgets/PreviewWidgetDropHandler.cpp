// Copyright Epic Games, Inc. All Rights Reserved.

#include "DragAndDrop/Widgets/PreviewWidgetDropHandler.h"

#include "Deletion/DeletionOperationInput.h"
#include "Deletion/DeletionOperationSystem.h"
#include "DragAndDrop/DropOperationInput.h"
#include "DragAndDrop/DropOperationSystem.h"
#include "Elements/Framework/TypedElementRegistry.h"
#include "Elements/Interfaces/TypedElementDataStorageInterface.h"
#include "Elements/Interfaces/TypedElementWorldInterface.h"
#include "ISceneOutliner.h"
#include "LevelEditor.h"
#include "Algo/AllOf.h"
#include "Modules/ModuleManager.h"

#define LOCTEXT_NAMESPACE "PreviewWidgetDropHandler"

DEFINE_LOG_CATEGORY_STATIC(LogPreviewWidgetDropHandler, Log, All);

namespace UE::Editor
{
	
namespace PreviewWidgetDropHandler_Private
{

static bool bUseTypedElementHandlesForDeletion = true;
	
static void UpdateOutliner()
{
	// We update the level outliners manually, because they don't update automatically yet during drag&drop.
	const FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>(TEXT("LevelEditor"));
	if (TSharedPtr<ILevelEditor> LevelEditor = LevelEditorModule.GetLevelEditorInstance().Pin())
	{
		for (const TWeakPtr<ISceneOutliner>& SceneOutlinerPtr : LevelEditor->GetAllSceneOutliners())
		{
			if (TSharedPtr<ISceneOutliner> SceneOutliner = SceneOutlinerPtr.Pin())
			{
				SceneOutliner->FullRefresh();
			}
		}
	}
}
	
}
	
FPreviewWidgetDropHandler::FPreviewWidgetDropHandler(TWeakObjectPtr<UTypedElementSelectionSet> SelectionSet)
	: FWidgetDropHandler(MoveTemp(SelectionSet))
{
}

FPreviewWidgetDropHandler::~FPreviewWidgetDropHandler()
{
}

FWidgetDropHandler::FDropResult FPreviewWidgetDropHandler::ResetOperations(DataStorage::ICoreProvider& Storage, UDropOperationSystem& DropSystem)
{
	using namespace UE::Editor::DataStorage;

	if (HasPreviews())
	{
		RemovePreviews(Storage, DropSystem); // @todo: Only previews of operations that change need to be recreated.
	}
	
	FDropResult Result = FWidgetDropHandler::ResetOperations(Storage, DropSystem);

	if (bIsValidDrop)
	{
		CreatePreviews(Storage, DropSystem);
	}
	
	return Result;
}
	
TOptional<FWidgetDropHandler::FDropResult> FPreviewWidgetDropHandler::Update(DataStorage::ICoreProvider& Storage, UDropOperationSystem& DropSystem, const FGeometry& Geometry,
	const FDragDropEvent& InputEvent)
{
	using namespace UE::Editor::DataStorage;

	EUpdateParameters Result = UpdateParameters(Storage, Geometry, InputEvent);
	if (Result == EUpdateParameters::NoChanges)
	{
		return {};
	}

	if (Result == EUpdateParameters::ResetOperations)
	{
		SetRowsFromParameters(Storage, InputRows.GetRows()); // Only update the input rows when we need to.
		return ResetOperations(Storage, DropSystem);
	}

	if (HasPreviews())
	{
		UpdatePreviews(Storage, DropSystem);
	}
	return {};
}
	
void FPreviewWidgetDropHandler::Stop(DataStorage::ICoreProvider& Storage, UDropOperationSystem& DropSystem)
{
	if (HasPreviews())
	{
		RemovePreviews(Storage, DropSystem);
	}
	
	FWidgetDropHandler::Stop(Storage, DropSystem);
}

void FPreviewWidgetDropHandler::Drop(DataStorage::ICoreProvider& Storage, UDropOperationSystem& DropSystem, const FGeometry& Geometry,
		const FDragDropEvent& InputEvent)
{
	if (HasPreviews())
	{
		RemovePreviews(Storage, DropSystem);
	}
	
	FWidgetDropHandler::Drop(Storage, DropSystem, Geometry, InputEvent);
}

static void GetCreatedRows(DataStorage::FRowHandleArray& OutRows, DataStorage::ICoreProvider& Storage, DataStorage::FRowHandleArrayView ResultRows)
{
	using namespace UE::Editor::DataStorage;
	using namespace UE::Editor::DataStorage::Operations;

	OutRows.Reset();
	OutRows.Reserve(ResultRows.Num());
	for (RowHandle InputRow : ResultRows)
	{
		if (const FResultColumn* Column = Storage.GetColumn<FResultColumn>(InputRow))
		{
			OutRows.Append(Column->Value.Created.GetRows());
		}
	}
}
	
void FPreviewWidgetDropHandler::CreatePreviews(DataStorage::ICoreProvider& Storage, UDropOperationSystem& DropSystem)
{
	using namespace UE::Editor::DataStorage;
	using namespace UE::Editor::DataStorage::Operations;
	if (!ensure(bIsValidDrop && PreviewInputRows.IsEmpty()))
	{
		return;
	}

	if (!Storage.FindFactory<UDeletionOperationSystem>())
	{
		return; // We need to be able to revert the previews.
	}	

	// Prepare input rows for a preview creation call.
	for (RowHandle InputRow : InputRows.GetRows())
	{
		Storage.AddColumn<FDropPreviewTag>(InputRow);
	}

	// Create the previews from our input.
	ExecutePreviewDrop(Storage, DropSystem, InputRows.GetRows());

	// Gather the results.
	FRowHandleArray Created;
	GetCreatedRows(Created, Storage, InputRows.GetRows());

	// Clean up the input rows.
	for (RowHandle InputRow : InputRows.GetRows())
	{
		Storage.RemoveColumn<FResultColumn>(InputRow);
		Storage.RemoveColumn<FDropPreviewTag>(InputRow);

		if (PreviewWidgetDropHandler_Private::bUseTypedElementHandlesForDeletion)
		{
			if (const FDropTypedElementResultColumn* Column = Storage.GetColumn<FDropTypedElementResultColumn>(InputRow))
			{
				PreviewElements.Append(Column->Elements);
			}
		}
		Storage.RemoveColumn<FDropTypedElementResultColumn>(InputRow);
	}

	// Create rows that hold our created previews for updates.
	DropSystem.CreateInputRows(PreviewInputRows, Created.GetRows(), InvalidRowHandle, false);	
	
	PreviewWidgetDropHandler_Private::UpdateOutliner();
}

void FPreviewWidgetDropHandler::UpdatePreviews(DataStorage::ICoreProvider& Storage, UDropOperationSystem& DropSystem)
{
	using namespace UE::Editor::DataStorage;
	using namespace UE::Editor::DataStorage::Operations;
	if (!ensure(!PreviewInputRows.IsEmpty()))
	{
		return;
	}
	
	SetRowsFromParameters(Storage, PreviewInputRows.GetRows());
	(void)DropSystem.Apply(PreviewInputRows.GetRows());
}

void FPreviewWidgetDropHandler::RemovePreviews(DataStorage::ICoreProvider& Storage, UDropOperationSystem& DropSystem)
{
	using namespace UE::Editor::DataStorage;
	using namespace UE::Editor::DataStorage::Operations;
	if (!ensure(!PreviewInputRows.IsEmpty()))
	{
		return;
	}

	bool bAllDeleted = true;
	// Currently, TEDS rows are not updated immediately. This means, if the preview is removed the same tick it was created, the created rows will not
	// yet have enough data for the deletion system to remove the placed objects from the world again.
	// Thus, for now, we delete the previews through the TEv1 system.
	if (PreviewWidgetDropHandler_Private::bUseTypedElementHandlesForDeletion)
	{
		TNotNull<UTypedElementRegistry*> Registry = UTypedElementRegistry::GetInstance();

		FTypedElementDeletionOptions Options;
		Options.SetVerifyDeletionCanHappen(false);
		Options.SetWarnAboutReferences(false);
		Options.SetWarnAboutSoftReferences(false);
		for (const FTypedElementHandle& ElementHandle : PreviewElements)
		{
			TTypedElement<ITypedElementWorldInterface> WorldElement = Registry->GetElement<ITypedElementWorldInterface>(ElementHandle);
			bAllDeleted &= WorldElement && WorldElement.DeleteElement(WorldElement.GetOwnerWorld(), SelectionSet.Get(), Options);
		}
	}
	else
	{
		UDeletionOperationSystem* DeletionSystem = Storage.FindFactory<UDeletionOperationSystem>();
		if (!ensureMsgf(DeletionSystem, TEXT("Deletion system not found. Unable to delete previews!")))
		{
			return;
		}
		
		// Make sure the previews gets deleted.
		for (RowHandle PreviewInputRow : PreviewInputRows.GetRows())
		{
			Storage.AddColumn<FDeletionForceTag>(PreviewInputRow);
		}
		
		// Delete our created previews.
		(void)DeletionSystem->Apply(PreviewInputRows.GetRows());

		bAllDeleted = Algo::AllOf(PreviewInputRows.GetRows(), [&Storage](RowHandle Row) { return Storage.HasColumns<FResultColumn>(Row); });
	}

	if (!bAllDeleted)
	{
		UE_LOGF(LogPreviewWidgetDropHandler, Error, "Deletion of previews failed!");
	}

	// Free our preview input rows.
	DropSystem.RemoveInputRows(PreviewInputRows.GetRows());
	PreviewInputRows.Empty();
	PreviewElements.Empty();
	
	PreviewWidgetDropHandler_Private::UpdateOutliner();
}

void FPreviewWidgetDropHandler::GetPreviewRows(DataStorage::FRowHandleArray& OutRows, DataStorage::ICoreProvider& Storage) const
{
	using namespace UE::Editor::DataStorage;
	using namespace UE::Editor::DataStorage::Operations;
	
	OutRows.Reset();
	OutRows.Reserve(PreviewInputRows.Num());
	for (RowHandle PreviewInputRow : PreviewInputRows.GetRows())
	{
		if (const FSourceColumn* Column = Storage.GetColumn<FSourceColumn>(PreviewInputRow))
		{
			OutRows.Add(Column->Value);
		}
	}
}

int32 FPreviewWidgetDropHandler::ExecutePreviewDrop(DataStorage::ICoreProvider& Storage, UDropOperationSystem& DropSystem,
	DataStorage::FRowHandleArrayView Input)
{	
	using namespace UE::Editor::DataStorage;
	
	int32 Result = 0;
	for (RowHandle InputRow : Input)
	{
		// Executing in a batch would be better, but right now we cannot do this due to legacy reasons:
		// The Level Viewport handler needs to execute the FEditorDelegates::OnNewActorsDropped/Placed signals which requires precise knowledge of
		// which dropped UObject has created which AActor. As the required columns for this are deferred, we need to do this without rows.
		Result += ExecutePreviewDropSingle(Storage, DropSystem, InputRow);
	}
	return Result;
}

bool FPreviewWidgetDropHandler::ExecutePreviewDropSingle(DataStorage::ICoreProvider& Storage, UDropOperationSystem& DropSystem,
	DataStorage::RowHandle InputRow)
{
	using namespace UE::Editor::DataStorage;
	return DropSystem.Apply(FRowHandleArrayView(&InputRow, 1, FRowHandleArray::EFlags::IsUnique | FRowHandleArray::EFlags::IsSorted)) != 0;
}
	
}

#undef LOCTEXT_NAMESPACE
