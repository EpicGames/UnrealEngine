// Copyright Epic Games, Inc. All Rights Reserved.

#include "DragAndDrop/Widgets/ViewportDropHandler.h"

#include "ActorFactories/ActorFactory.h"
#include "DragAndDrop/DropOperationInput.h"
#include "Editor/ObjectPositioning.h"
#include "EditorModeManager.h"
#include "EditorViewportClient.h"
#include "Elements/Columns/TypedElementCompatibilityColumns.h"
#include "Elements/Columns/TypedElementHandleColumn.h"
#include "Elements/Framework/TypedElementRegistry.h"
#include "Elements/Interfaces/TypedElementDataStorageInterface.h"
#include "Elements/Interfaces/TypedElementWorldInterface.h"
#include "Framework/Application/SlateApplication.h"
#include "Input/DragAndDrop.h"
#include "Modules/ModuleManager.h"
#include "SEditorViewport.h"
#include "SceneView.h"
#include "Settings/LevelEditorViewportSettings.h"
#include "SnappingUtils.h"

#define LOCTEXT_NAMESPACE "ViewportDropHandler"

namespace UE::Editor
{

namespace ViewportDropHandler_Private
{
	static FIntPoint GetViewportCursorPosition(FEditorViewportClient& ViewportClient, const FGeometry& Geometry, const FDragDropEvent& InputEvent)
	{
		FIntPoint Pos = (Geometry.AbsoluteToLocal(InputEvent.GetScreenSpacePosition()) * Geometry.Scale).IntPoint();

		FIntPoint ViewportOrigin, ViewportSize;
		ViewportClient.GetViewportDimensions(ViewportOrigin, ViewportSize);
		Pos -= ViewportOrigin;
		return Pos;
	}

	// TEv1 workaround until tracing properly supports entities (and TEDS).
	static void AddIgnoredObjects(FCollisionQueryParams& OutParams, DataStorage::ICoreProvider& Storage, DataStorage::FRowHandleArrayView IgnoreRows)
	{
		using namespace UE::Editor::DataStorage;

		UTypedElementRegistry* Registry = UTypedElementRegistry::GetInstance();
		for (RowHandle Row : IgnoreRows)
		{
			if (Registry)
			{
				if (Compatibility::FTypedElementColumn* Column = Storage.GetColumn<Compatibility::FTypedElementColumn>(Row))
				{
					if (TTypedElement<ITypedElementWorldInterface> WorldElement = Registry->GetElement<ITypedElementWorldInterface>(Column->Handle))
					{
						WorldElement.AddIgnoredElementToCollisionQueryParams(OutParams, true);
						continue;
					}
				}
			}

			if (FTypedElementUObjectColumn* Column = Storage.GetColumn<FTypedElementUObjectColumn>(Row))
			{
				if (UObject* Object = Column->Object.Get())
				{
					OutParams.AddIgnoredSourceObject(Object);
				}
			}
		}
	}

	// Optimally this should be provided by some public gizmo functionality instead.
	static FTransform TraceDropTransform(FEditorViewportClient& Client, const FIntPoint& CursorPos, DataStorage::ICoreProvider& Storage, DataStorage::FRowHandleArrayView IgnoreRows)
	{
		FSceneViewFamilyContext ViewFamily(FSceneViewFamily::ConstructionValues(
			Client.Viewport, Client.GetScene(), Client.EngineShowFlags));
		FSceneView* View = Client.CalcSceneView(&ViewFamily);
		FViewportCursorLocation Cursor(View, &Client, CursorPos.X, CursorPos.Y);

		FCollisionQueryParams Params(SCENE_QUERY_STAT(DragDropTrace), true);
		AddIgnoredObjects(Params, Storage, IgnoreRows);

		const Positioning::FObjectPositioningTraceResult TraceResult =
			Positioning::TraceWorldForPositionWithDefault(Cursor, *View, &Params);

		FVector Location = TraceResult.Location;
		FSnappingUtils::SnapPointToGrid(Location, FVector::ZeroVector);

		FVector VertexNormal;
		if (!FSnappingUtils::SnapLocationToNearestVertex(Location, FVector2D(CursorPos), &Client, VertexNormal, false, {}))
		{
			VertexNormal = TraceResult.SurfaceNormal;
		}

		FQuat Rotation = FQuat::Identity;
		if (GetDefault<ULevelEditorViewportSettings>()->SnapToSurface.bEnabled)
		{
			Rotation = FindActorAlignmentRotation(FQuat::Identity, FVector::UpVector, VertexNormal);
		}
		return FTransform(Rotation, Location);
	}

	UTypedElementSelectionSet* GetSelectionSetFromViewportClient(const TSharedPtr<FEditorViewportClient>& ViewportClient)
	{
		if (ViewportClient)
		{
			if (const FEditorModeTools* ModeTools = ViewportClient->GetModeTools())
			{
				return ModeTools->GetEditorSelectionSet();
			}
		}
		return nullptr;
	}
}

FViewportDropHandler::FViewportDropHandler(TSharedPtr<FEditorViewportClient> InViewportClient, FTargetRowGetter InGetDefaultTargetRow)
	: FPreviewWidgetDropHandler(ViewportDropHandler_Private::GetSelectionSetFromViewportClient(InViewportClient))
	, ViewportClient(MoveTemp(InViewportClient))
	, GetDefaultTargetRow(MoveTemp(InGetDefaultTargetRow))
	, LastCursorPos()
	, LastTargetRow(DataStorage::InvalidRowHandle)
	, LastTransform(FTransform::Identity)
{
}

FViewportDropHandler::~FViewportDropHandler()
{
}

FWidgetDropHandler::EUpdateParameters FViewportDropHandler::UpdateParameters(DataStorage::ICoreProvider& Storage, const FGeometry& Geometry,
	const FDragDropEvent& InputEvent)
{
	using namespace UE::Editor::DataStorage;

	FIntPoint CursorPos = ViewportClient ? ViewportDropHandler_Private::GetViewportCursorPosition(*ViewportClient, Geometry, InputEvent) : FIntPoint();	
	if (LastCursorPos != CursorPos && ensure(ViewportClient))
	{
		FRowHandleArray IgnoreRows;
		GetPreviewRows(IgnoreRows, Storage);
		FTransform Transform = ViewportDropHandler_Private::TraceDropTransform(*ViewportClient, CursorPos, Storage, IgnoreRows.GetRows());

		// By default, we just get the target row from our creator / viewport client.
		// @todo: Introduce cases to get target by tracing.
		RowHandle TargetRow = (GetDefaultTargetRow && ViewportClient) ? GetDefaultTargetRow(*ViewportClient) : InvalidRowHandle;
		
		EUpdateParameters Result = LastTargetRow != TargetRow ? EUpdateParameters::ResetOperations : EUpdateParameters::MinorChanges;
		LastCursorPos = CursorPos;
		LastTargetRow = TargetRow;
		LastTransform = Transform;
		return Result;		
	}
	return EUpdateParameters::NoChanges;
}

void FViewportDropHandler::SetRowsFromParameters(DataStorage::ICoreProvider& Storage, DataStorage::FRowHandleArrayView Rows) const
{
	using namespace UE::Editor::DataStorage;
	using namespace UE::Editor::DataStorage::Operations;
	
	for (RowHandle InputRow : Rows)
	{
		Storage.AddColumn(InputRow, FDropTargetColumn{ .Value = LastTargetRow });
		Storage.AddColumn(InputRow, FDropTransformColumn{ .Value = LastTransform });
	}
}

void FViewportDropHandler::Stop(DataStorage::ICoreProvider& Storage, UDropOperationSystem& DropSystem)
{
	FPreviewWidgetDropHandler::Stop(Storage, DropSystem);
	LastCursorPos = FIntPoint();
	LastTargetRow = DataStorage::InvalidRowHandle;
	LastTransform = FTransform::Identity;
}

void FViewportDropHandler::CreatePreviews(DataStorage::ICoreProvider& Storage, UDropOperationSystem& DropSystem)
{
	FPreviewWidgetDropHandler::CreatePreviews(Storage, DropSystem);
	InvalidateViewport();
}

void FViewportDropHandler::UpdatePreviews(DataStorage::ICoreProvider& Storage, UDropOperationSystem& DropSystem)
{
	FPreviewWidgetDropHandler::UpdatePreviews(Storage, DropSystem);
	InvalidateViewport();
}

void FViewportDropHandler::RemovePreviews(DataStorage::ICoreProvider& Storage, UDropOperationSystem& DropSystem)
{
	FPreviewWidgetDropHandler::RemovePreviews(Storage, DropSystem);
	InvalidateViewport();
}

FReply FViewportDropHandler::OnDrop(const FGeometry& Geometry, const FDragDropEvent& InputEvent)
{
	FReply Reply = FPreviewWidgetDropHandler::OnDrop(Geometry, InputEvent);
	if (Reply.IsEventHandled() && ViewportClient)
	{
		if(TSharedPtr<SEditorViewport> EditorViewport = ViewportClient->GetEditorViewportWidget())
		{
			FSlateApplication::Get().SetKeyboardFocus(EditorViewport);
		}
	}
	return Reply;
}

void FViewportDropHandler::InvalidateViewport()
{
	if (ViewportClient && ViewportClient->Viewport)
	{
		ViewportClient->Viewport->InvalidateDisplay();
	}
}

}

#undef LOCTEXT_NAMESPACE
