// Copyright Epic Games, Inc. All Rights Reserved.

#include "DragAndDrop/Widgets/WidgetDropHandler.h"

#include "DataStorage/Features.h"
#include "DragAndDrop/DecoratedDragDropOp.h"
#include "DragAndDrop/DropOperationSystem.h"
#include "DragAndDrop/TedsDragDropOpUtility.h"
#include "Editor.h"
#include "Elements/Framework/EngineElementsLibrary.h"
#include "Elements/Framework/TypedElementSelectionSet.h"
#include "Elements/Interfaces/TypedElementDataStorageInterface.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Input/DragAndDrop.h"
#include "Logging/LogMacros.h"
#include "ScopedTransaction.h"
#include "TedsOperationInput.h"
#include "Widgets/Notifications/SNotificationList.h"

#define LOCTEXT_NAMESPACE "WidgetDropHandler"

DEFINE_LOG_CATEGORY_STATIC(LogWidgetDropHandler, Log, All);

namespace UE::Editor
{
void FWidgetDropHandler::ShowErrorNotifications(const DataStorage::ICoreProvider& Storage, DataStorage::FRowHandleArrayView InputRows)
{
	using namespace UE::Editor::DataStorage;

	FNotificationInfo NotificationInfo(LOCTEXT("ErrorTitle", "Drop Operation Error"));
	NotificationInfo.FadeInDuration = 2.0f;
	NotificationInfo.FadeOutDuration = 2.0f;
	NotificationInfo.ExpireDuration = 5.0f;
	NotificationInfo.bUseSuccessFailIcons = false;
	NotificationInfo.Image = FAppStyle::GetBrush("Icons.WarningWithColor");
	
	for (RowHandle InputRow : InputRows)
	{
		if (!Storage.HasColumns<Operations::FResultColumn>(InputRow))
		{
			if (const FText* Reason = Operations::Utilities::GetDescriptionPtr(Storage, InputRow); Reason && !Reason->IsEmpty())
			{
				UE_LOGF(LogWidgetDropHandler, Warning, "Drop operation failed: %ls", *Reason->ToString());
				
				NotificationInfo.SubText = *Reason;
				if (TSharedPtr<SNotificationItem> Notification = FSlateNotificationManager::Get().AddNotification(NotificationInfo))
				{
					Notification->SetCompletionState(SNotificationItem::CS_Fail);
					Notification->ExpireAndFadeout();
				}
			}
		}
	}
}

FWidgetDropHandler::FWidgetDropHandler(TWeakObjectPtr<UTypedElementSelectionSet> SelectionSet)
	: SelectionSet(MoveTemp(SelectionSet))
	, CurrentOperation(nullptr)
	, bIsValidDrop(false)
{
}

FWidgetDropHandler::~FWidgetDropHandler()
{
}

bool FWidgetDropHandler::OnDragEnter(const FGeometry& Geometry, const FDragDropEvent& InputEvent)
{
	using namespace UE::Editor::DataStorage;
	
	if (TSharedPtr<FDragDropOperation> Operation = InputEvent.GetOperation())
	{
		if (ICoreProvider* Storage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName))
		{
			if (UDropOperationSystem* DropSystem = Storage->FindFactory<UDropOperationSystem>())
			{
				if (TOptional<FDropResult> Result = Start(*Storage, *DropSystem, Geometry, InputEvent, *Operation))
				{
					UpdateOperationDecorator(*Operation, Result);
					return true;
				}
			}
		}
	}
	return false;
}

FReply FWidgetDropHandler::OnDragOver(const FGeometry& Geometry, const FDragDropEvent& InputEvent)
{
	using namespace UE::Editor::DataStorage;

	TSharedPtr<FDragDropOperation> Operation = InputEvent.GetOperation();
	if (Operation && CurrentOperation == Operation.Get())
	{
		if (ICoreProvider* Storage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName))
		{
			if (UDropOperationSystem* DropSystem = Storage->FindFactory<UDropOperationSystem>())
			{
				if (TOptional<FDropResult> Result = Update(*Storage, *DropSystem, Geometry, InputEvent))
				{
					// Only update for changes.
					UpdateOperationDecorator(*Operation, Result);
				}
			}
		}
		return FReply::Handled();
	}
	return FReply::Unhandled();
}

bool FWidgetDropHandler::OnDragLeave(const FDragDropEvent& InputEvent)
{
	using namespace UE::Editor::DataStorage;

	TSharedPtr<FDragDropOperation> Operation = InputEvent.GetOperation();
	if (Operation && CurrentOperation == Operation.Get())
	{
		if (ICoreProvider* Storage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName))
		{
			if (UDropOperationSystem* DropSystem = Storage->FindFactory<UDropOperationSystem>())
			{
				Stop(*Storage, *DropSystem);
				UpdateOperationDecorator(*Operation, {});
			}
		}
		return true;
	}
	return false;
}

FReply FWidgetDropHandler::OnDrop(const FGeometry& Geometry, const FDragDropEvent& InputEvent)
{
	using namespace UE::Editor::DataStorage;

	TSharedPtr<FDragDropOperation> Operation = InputEvent.GetOperation();
	if (Operation && CurrentOperation == Operation.Get())
	{
		ICoreProvider* Storage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName);
		if (Storage && bIsValidDrop)
		{
			if (UDropOperationSystem* DropSystem = Storage->FindFactory<UDropOperationSystem>())
			{
				Drop(*Storage, *DropSystem, Geometry, InputEvent);
				UpdateOperationDecorator(*Operation, {});
			}
		}
		return FReply::Handled();
	}
	return FReply::Unhandled();
}

FDragDropOperation* FWidgetDropHandler::GetCurrentOperation() const
{
	return CurrentOperation;
}

bool FWidgetDropHandler::PrepareInput(DataStorage::ICoreProvider& Storage, UDropOperationSystem& DropSystem, const FDragDropOperation& Operation)
{
	using namespace UE::Editor::DataStorage;
	
	FRowHandleArray SourceRows;
	if (!DragAndDrop::GetRowsFromData(SourceRows, Operation))
	{
		return false; // If we lose data by converting to rows, we don't want to continue.
	}
	
	return PrepareInput(Storage, DropSystem, SourceRows.GetRows());
}

bool FWidgetDropHandler::PrepareInput(DataStorage::ICoreProvider& Storage, UDropOperationSystem& DropSystem,
	DataStorage::FRowHandleArrayView SourceRows)
{
	using namespace UE::Editor::DataStorage;
	checkf(InputRows.IsEmpty(), TEXT("Expected previous operation to be cleared."));

	return DropSystem.CreateInputRows(InputRows, SourceRows, InvalidRowHandle, true) && !InputRows.IsEmpty();
}

FWidgetDropHandler::FDropResult FWidgetDropHandler::ResetOperations(DataStorage::ICoreProvider& Storage, UDropOperationSystem& DropSystem)
{
	using namespace UE::Editor::DataStorage;
	using namespace UE::Editor::DataStorage::Operations;

	// We only accept the drop if all input rows are valid.
	bIsValidDrop = DropSystem.Test(InputRows.GetRows()) == InputRows.Num();

	FText ToolTip;
	for (RowHandle InputRow : InputRows.GetRows())
	{
		// Find an error text when we failed and a success text if we succeeded.
		if (bIsValidDrop != Storage.HasColumns<FTestResultTag>(InputRow))
		{
			continue;
		}
		
		ToolTip = Utilities::GetDescription(Storage, InputRow);
		if (!ToolTip.IsEmpty())
		{
			break;
		}
	}
	
	if (!bIsValidDrop && ToolTip.IsEmpty())
	{
		ToolTip = LOCTEXT("Drop_Unsupported", "Drop operation is unsupported.");
	}
	
	return { bIsValidDrop, MoveTemp(ToolTip) };
}

FWidgetDropHandler::EUpdateParameters FWidgetDropHandler::UpdateParameters(DataStorage::ICoreProvider& Storage, const FGeometry& Geometry,
	const FDragDropEvent& InputEvent)
{
	return EUpdateParameters::NoChanges;
}

void FWidgetDropHandler::SetRowsFromParameters(DataStorage::ICoreProvider& Storage, DataStorage::FRowHandleArrayView Rows) const
{
}

TOptional<FWidgetDropHandler::FDropResult> FWidgetDropHandler::Start(DataStorage::ICoreProvider& Storage, UDropOperationSystem& DropSystem,
                                                          const FGeometry& Geometry, const FDragDropEvent& InputEvent, FDragDropOperation& Operation)
{
	using namespace UE::Editor::DataStorage;

	if (CurrentOperation)
	{
		Stop(Storage, DropSystem);
	}
	
	CurrentOperation = &Operation;
	
	if (!PrepareInput(Storage, DropSystem, Operation))
	{
		Stop(Storage, DropSystem);
		return { };
	}
	(void)UpdateParameters(Storage, Geometry, InputEvent);
	SetRowsFromParameters(Storage, InputRows.GetRows());
	
	// We ignore the drop if there are no operations for it.
	TArray<RowHandle> Operations;
	for (RowHandle InputRow : InputRows.GetRows())
	{
		Operations.Reset();
		DropSystem.GetOperations(Operations, InputRow);
		if (Operations.IsEmpty())
		{
			Stop(Storage, DropSystem);
			return {};
		}
	}
	
	return ResetOperations(Storage, DropSystem);
}

TOptional<FWidgetDropHandler::FDropResult> FWidgetDropHandler::Update(DataStorage::ICoreProvider& Storage, UDropOperationSystem& DropSystem,
	const FGeometry& Geometry, const FDragDropEvent& InputEvent)
{
	using namespace UE::Editor::DataStorage;
	check(CurrentOperation != nullptr);

	EUpdateParameters Result = UpdateParameters(Storage, Geometry, InputEvent);
	if (Result == EUpdateParameters::ResetOperations)
	{
		SetRowsFromParameters(Storage, InputRows.GetRows()); // Only update the input rows when we need to.
		return ResetOperations(Storage, DropSystem);
	}	
	return {};
}
	
void FWidgetDropHandler::Stop(DataStorage::ICoreProvider& Storage, UDropOperationSystem& DropSystem)
{
	using namespace UE::Editor::DataStorage;
	check(CurrentOperation != nullptr);

	DropSystem.RemoveInputRows(InputRows.GetRows());
	
	InputRows.Empty();
	CurrentOperation = nullptr;
	bIsValidDrop = false;	
}

void FWidgetDropHandler::Drop(DataStorage::ICoreProvider& Storage, UDropOperationSystem& DropSystem, const FGeometry& Geometry,
		const FDragDropEvent& InputEvent)
{
	using namespace UE::Editor::DataStorage;
	using namespace UE::Editor::DataStorage::Operations;
	check(bIsValidDrop);

	// Update one last time.
	(void)UpdateParameters(Storage, Geometry, InputEvent);
	SetRowsFromParameters(Storage, InputRows.GetRows());

	// Drop all data.
	ExecuteDrop(Storage, DropSystem, InputRows.GetRows());
	
	// Stop the current operation.
	Stop(Storage, DropSystem);
}

void FWidgetDropHandler::UpdateOperationDecorator(FDragDropOperation& Operation, TOptional<FDropResult> Result) const
{
	Operation.SetDecoratorVisibility(true);
	if (Operation.IsOfType<FDecoratedDragDropOp>())
	{
		static constexpr TCHAR BrushName_Ok[] = TEXT("Graph.ConnectorFeedback.OK");
		static constexpr TCHAR BrushName_Error[] = TEXT("Graph.ConnectorFeedback.Error");
		FDecoratedDragDropOp& DecoratedOp = static_cast<FDecoratedDragDropOp&>(Operation);
		if (Result)
		{
			DecoratedOp.SetToolTip(Result->Tooltip, FAppStyle::GetBrush(Result->bCanDrop ? BrushName_Ok : BrushName_Error));
		}
		else
		{
			DecoratedOp.ResetToDefaultToolTip();
		}
	}
	else if (Result)
	{
		Operation.SetCursorOverride(Result->bCanDrop ? EMouseCursor::Default : EMouseCursor::SlashedCircle);
	}
	else
	{
		Operation.SetCursorOverride({});
	}
}

int32 FWidgetDropHandler::ExecuteDrop(DataStorage::ICoreProvider& Storage, UDropOperationSystem& DropSystem, DataStorage::FRowHandleArrayView Input)
{
	using namespace UE::Editor::DataStorage;
	
	// Bit of a hack to select our newly placed objects.
	// Usually we would like use teds rows for this, but since the synchronization is deferred we cannot access TEv1 elements from our rows yet.
	// We also cannot access the rows next tick, because we might be within a transaction scope which would not capture the selection.
	// To solve this we collect all actors that are placed while the drop process is ongoing.
	TSet<TObjectPtr<AActor>> PlacedActors;	
	FDelegateHandle DelegateHandle;
	if (SelectionSet.IsValid())
	{
		DelegateHandle = GEditor->OnLevelActorAdded().AddLambda([&PlacedActors](AActor* Actor)
			{
				PlacedActors.Add(Actor);
			});
	}
	
	int32 Result = 0;
	FScopedTransaction Transaction(LOCTEXT("Drop_Transaction", "Drop object"));
	for (RowHandle InputRow : Input)
	{
		// Executing in a batch would be better, but right now we cannot do this due to legacy reasons:
		// The Level Viewport handler needs to execute the FEditorDelegates::OnNewActorsDropped/Placed signals which requires precise knowledge of
		// which dropped UObject has created which AActor. As the required columns for this are deferred, we need to do this without rows.
		Result += ExecuteDropSingle(Storage, DropSystem, InputRow);
	}
	
	// Now we select the roots of all captured actors.
	if (UTypedElementSelectionSet* SelectionSetPtr = SelectionSet.Get())
	{
		GEditor->OnLevelActorAdded().Remove(DelegateHandle);
		
		TArray<FTypedElementHandle> RootElements;
		for (const TObjectPtr<AActor>& ActorPtr : PlacedActors)
		{
			if (AActor* Actor = ActorPtr.Get(); IsValid(Actor))
			{
				AActor* Parent = Actor->GetParentActor();
				if (!IsValid(Parent) || !PlacedActors.Contains(Parent))
				{
					constexpr bool bAllowCreate = true;
					if (FTypedElementHandle Element = UEngineElementsLibrary::AcquireEditorActorElementHandle(Actor, bAllowCreate))
					{
						RootElements.Add(Element);
					}
				}
			}
		}
			
		SelectionSetPtr->SetSelection(RootElements, FTypedElementSelectionOptions());
	}
	
	if (Result != Input.Num())
	{
		ShowErrorNotifications(Storage, Input);
	}
	
	return Result;
}

bool FWidgetDropHandler::ExecuteDropSingle(DataStorage::ICoreProvider& Storage, UDropOperationSystem& DropSystem, DataStorage::RowHandle InputRow)
{
	using namespace UE::Editor::DataStorage;
	return DropSystem.Apply(FRowHandleArrayView(&InputRow, 1, FRowHandleArray::EFlags::IsUnique | FRowHandleArray::EFlags::IsSorted)) != 0;
}

}

#undef LOCTEXT_NAMESPACE
