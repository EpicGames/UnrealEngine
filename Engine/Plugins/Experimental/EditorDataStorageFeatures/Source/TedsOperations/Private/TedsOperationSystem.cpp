// Copyright Epic Games, Inc. All Rights Reserved.

#include "TedsOperationSystem.h"

#include "Elements/Framework/TypedElementQueryBuilder.h"
#include "Elements/Interfaces/TypedElementDataStorageInterface.h"
#include "TedsOperationColumns.h"
#include "TedsOperationInput.h"
#include "TedsQueryNode.h"
#include "TedsQueryStackExecutor.h"
#include "TedsRowQueryResultsNode.h"

bool UOperationSystem::FDefaultLess::operator()(const UE::Editor::DataStorage::ICoreProvider& Storage, UE::Editor::DataStorage::RowHandle A,
	UE::Editor::DataStorage::RowHandle B) const
{
	using namespace UE::Editor::DataStorage;
	// First sort by priority column.
	{
		const Operations::FPriorityColumn* ColumnA = Storage.GetColumn<Operations::FPriorityColumn>(A);
		const Operations::FPriorityColumn* ColumnB = Storage.GetColumn<Operations::FPriorityColumn>(B);
		int64 PriorityA = ColumnA ? ColumnA->Value : 0;
		int64 PriorityB = ColumnB ? ColumnB->Value : 0;
		if (PriorityA != PriorityB)
		{
			return PriorityA > PriorityB; // Highest priority should be the first element.
		}
	}
	// Second sort by name column.
	{
		const Operations::FNameColumn* ColumnA = Storage.GetColumn<Operations::FNameColumn>(A);
		const Operations::FNameColumn* ColumnB = Storage.GetColumn<Operations::FNameColumn>(B);
		FName NameA = ColumnA ? ColumnA->Value : FName();
		FName NameB = ColumnB ? ColumnB->Value : FName();
		if (int32 Result = NameA.Compare(NameB); Result != 0)
		{
			return Result < 0;
		}
	}
	// Third sort by row handle.
	return A < B;
}

UOperationSystem::UOperationSystem()
	: StoragePtr(nullptr)
	, OperationTable(UE::Editor::DataStorage::InvalidTableHandle)
	, InputTable(UE::Editor::DataStorage::InvalidTableHandle)
	, InputTableDescr(UE::Editor::DataStorage::InvalidTableHandle)
{
}

UOperationSystem::~UOperationSystem()
{
}

void UOperationSystem::PreRegister(UE::Editor::DataStorage::ICoreProvider& Storage)
{
	StoragePtr = &Storage;
}

void UOperationSystem::RegisterTables(UE::Editor::DataStorage::ICoreProvider& Storage)
{
	using namespace UE::Editor::DataStorage::Operations;
	
	OperationTable  = Storage.RegisterTable<FApplyColumn>("OperationTable");
	InputTable      = Storage.RegisterTable<FSourceColumn>("OperationInputTable");
	InputTableDescr = Storage.RegisterTable<FSourceColumn, FDescriptionColumn>("OperationInputTableDescr");
}

void UOperationSystem::RegisterQueries(UE::Editor::DataStorage::ICoreProvider& Storage)
{
	using namespace UE::Editor::DataStorage;
	using namespace UE::Editor::DataStorage::Queries;
	using namespace UE::Editor::DataStorage::QueryStack;

	FQueryDescription QueryDescription;
	GetQueryDescription(QueryDescription);

	QueryNode   = MakeShared<FQueryNode>(Storage, MoveTemp(QueryDescription));
	ResultsNode = MakeShared<FRowQueryResultsNode>(Storage, QueryNode, FRowQueryResultsNode::ESyncActions::ForceRefreshOnUpdate);
	Executor    = MakeShared<FExplicitUpdateExecutor>("Operations", ResultsNode);
}

UE::Editor::DataStorage::RowHandle UOperationSystem::AddOperation(FName Name,
	TNotNull<UE::Editor::DataStorage::Operations::FApplyCallback> Operation,
	UE::Editor::DataStorage::Operations::FTestCallback Test,
	UE::Editor::DataStorage::Operations::FProbeCallback Probe, int64 Priority)
{
	using namespace UE::Editor::DataStorage;
	using namespace UE::Editor::DataStorage::Operations;

	if (OperationTable == InvalidTableHandle)
	{
		return InvalidRowHandle;
	}

	RowHandle Row = StoragePtr->AddRow(OperationTable);
	if (Row != InvalidRowHandle)
	{
		StoragePtr->AddColumn(Row, FApplyColumn{ .Callback = MoveTemp(Operation) });
		if (Name != NAME_None)
		{
			StoragePtr->AddColumn(Row, FNameColumn{ .Value = MoveTemp(Name) });
		}
		if (Test)
		{
			StoragePtr->AddColumn(Row, FTestColumn{ .Callback = MoveTemp(Test) });
		}
		if (Probe)
		{
			StoragePtr->AddColumn(Row, FProbeColumn{ .Callback = MoveTemp(Probe) });
		}
		if (Priority != DefaultPriority)
		{
			StoragePtr->AddColumn(Row, FPriorityColumn{ .Value = Priority });
		}
	}
	return Row;	
}

void UOperationSystem::GetOperations(TArray<UE::Editor::DataStorage::RowHandle>& OutRows, UE::Editor::DataStorage::RowHandle InputRow,
	const FFilter& Filter, const FLess& Less) const
{
	using namespace UE::Editor::DataStorage;
	
	OutRows.Empty();
	if (!Executor)
	{
		return;
	}
	
	Executor->Update();
	FRowHandleArrayView AllRows = ResultsNode->GetRows();
	if (AllRows.IsEmpty())
	{
		return;
	}

	for (RowHandle Row : AllRows)
	{
		if (Row == InvalidRowHandle)
		{
			continue;
		}

		if (Filter && !Filter(*StoragePtr, Row))
		{
			continue;
		}

		const Operations::FProbeColumn* ProbeColumn = StoragePtr->GetColumn<Operations::FProbeColumn>(Row);
		if (ProbeColumn && ProbeColumn->Callback && !ProbeColumn->Callback(*StoragePtr, InputRow))
		{
			continue;
		}

		OutRows.Add(Row);
	}

	if (Less)
	{
		Algo::Sort(OutRows, [this, &Less](RowHandle A, RowHandle B) { return Less(*StoragePtr, A, B); });
	}
}

int32 UOperationSystem::Test(UE::Editor::DataStorage::FRowHandleArrayView InputRows, const FFilter& Filter, const FLess& Less) const
{
	using namespace UE::Editor::DataStorage;
	using namespace UE::Editor::DataStorage::Operations;

	// @todo: Improve for batch processing.

	int32 SuccessNum = 0;	

	TArray<RowHandle> OperationRows;
	for (RowHandle InputRow : InputRows)
	{
		bool bResult = false;
		OperationRows.Reset();
		StoragePtr->RemoveColumn<FTestResultTag>(InputRow);
		
		GetOperations(OperationRows, InputRow, Filter, Less);
		for (RowHandle OperationRow : OperationRows)
		{
			// No column equals validity.
			const FTestColumn* TestColumn = StoragePtr->GetColumn<FTestColumn>(OperationRow);
			bResult = !TestColumn || !TestColumn->Callback || TestColumn->Callback(*StoragePtr, InputRow);
			if (bResult)
			{
				break;
			}
		}

		if (bResult)
		{
			StoragePtr->AddColumn<FTestResultTag>(InputRow);
			++SuccessNum;
		}
		else
		{
			StoragePtr->RemoveColumn<FTestResultTag>(InputRow);
		}
	}

	return SuccessNum;
}

int32 UOperationSystem::Apply(UE::Editor::DataStorage::FRowHandleArrayView InputRows, const FFilter& Filter, const FLess& Less) const
{
	using namespace UE::Editor::DataStorage;
	using namespace UE::Editor::DataStorage::Operations;
	
	// @todo: Improve for batch processing.

	int32 SuccessNum = 0;
	
	TArray<RowHandle> OperationRows;
	for (RowHandle InputRow : InputRows)
	{
		TOptional<FResult> Result;
		OperationRows.Reset();
		
		GetOperations(OperationRows, InputRow, Filter, Less);
		for (RowHandle OperationRow : OperationRows)
		{
			if (const FApplyColumn* ApplyColumn = StoragePtr->GetColumn<FApplyColumn>(OperationRow))
			{
				Result = ApplyColumn->Callback(*StoragePtr, InputRow);
				if (Result)
				{
					break;
				}
			}
		}

		if (Result)
		{
			StoragePtr->AddColumn(InputRow, FResultColumn{ .Value = MoveTemp(*Result) });
			++SuccessNum;
		}
		else
		{
			StoragePtr->RemoveColumn<FResultColumn>(InputRow);
		}
	}

	return SuccessNum;
}

bool UOperationSystem::CreateInputRows(UE::Editor::DataStorage::FRowHandleArray& OutInputRows, UE::Editor::DataStorage::FRowHandleArrayView SourceRows,
	bool bAddDescription) const
{
	using namespace UE::Editor::DataStorage;
	using namespace UE::Editor::DataStorage::Operations;
	
	OutInputRows.Reset();
	if (SourceRows.IsEmpty())
	{
		return true;
	}
	
	TableHandle Table = bAddDescription ? InputTableDescr : InputTable;
	if (Table == InvalidTableHandle)
	{
		return false;
	}
	
	OutInputRows.Reserve(SourceRows.Num());
	const RowHandle* SourceRowIt = SourceRows.begin();
	return StoragePtr->BatchAddRow(Table, SourceRows.Num(), [this, &SourceRowIt, &OutInputRows](RowHandle Row)
		{
			StoragePtr->AddColumn(Row, FSourceColumn{ .Value = *SourceRowIt++ });
			OutInputRows.Add(Row);
		});
}

void UOperationSystem::RemoveInputRows(UE::Editor::DataStorage::FRowHandleArrayView RowsToRemove) const
{
	StoragePtr->BatchRemoveRows(RowsToRemove);
}

void UOperationSystem::GetQueryDescription(UE::Editor::DataStorage::FQueryDescription& OutDescription) const
{
	using namespace UE::Editor::DataStorage::Queries;
	using namespace UE::Editor::DataStorage::Operations;

	OutDescription = Select().Where().All<FApplyColumn>().Compile();
}
