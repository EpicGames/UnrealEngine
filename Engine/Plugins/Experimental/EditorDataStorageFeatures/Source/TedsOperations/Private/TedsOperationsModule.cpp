// Copyright Epic Games, Inc. All Rights Reserved.

#include "TedsOperationsModule.h"

#include "Modules/ModuleManager.h"

namespace UE::Editor::DataStorage::Operations
{

void FTedsOperationsModule::StartupModule()
{
	IModuleInterface::StartupModule();
}

void FTedsOperationsModule::ShutdownModule()
{
	IModuleInterface::ShutdownModule();
}

}

IMPLEMENT_MODULE(UE::Editor::DataStorage::Operations::FTedsOperationsModule, TedsOperations);
