// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"

namespace UE::Editor::DataStorage::Operations
{
	class FTedsOperationsModule : public IModuleInterface
	{
	public:
		virtual void StartupModule() override;
		virtual void ShutdownModule() override;
	};
}