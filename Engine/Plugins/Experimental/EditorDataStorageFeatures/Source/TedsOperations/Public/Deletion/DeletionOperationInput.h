// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataStorage/CommonTypes.h"

#include "DeletionOperationInput.generated.h"

USTRUCT(meta = (DisplayName = "Force Deletion"))
struct FDeletionOperationForceTag final : public FEditorDataStorageTag
{
	GENERATED_BODY()
};
namespace UE::Editor::DataStorage::Operations
{
	/** Tag to notify the deletion operation that the input should be force deleted, i.e. regardless of restrictions. */
	using FDeletionForceTag = FDeletionOperationForceTag;
}
