// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataStorage/CommonTypes.h"
#include "TedsOperationSystem.h"
#include "TedsOperationInput.h"

#include "DeletionOperationSystem.generated.h"

#define UE_API TEDSOPERATIONS_API

USTRUCT(meta = (DisplayName = "Deletion Operation"))
struct FDeletionOperationTag final : public FEditorDataStorageTag
{
	GENERATED_BODY()
};
namespace UE::Editor::DataStorage::Operations
{
	/** Tag to identify this operation as a drop operation. UDeletionOperationSystem will only consider operation rows with this tag. */
	using FDeletionTag = FDeletionOperationTag;
}

/**
 * Operation system to execute operations that delete something. Only considers operations with the FDeletionTag.
*/
UCLASS(MinimalAPI)
class UDeletionOperationSystem : public UOperationSystem
{
	GENERATED_BODY()
public:
	UE_API UDeletionOperationSystem();
	UE_API virtual ~UDeletionOperationSystem() override;
	
	UE_API virtual void RegisterTables(UE::Editor::DataStorage::ICoreProvider& Storage) override;

	/**
	 * Utility method to create a batch of rows that can be used as input for deletion operations.
	 * The method will create as many rows as source rows were passed in and automatically assign the FSourceColumn and FDeletionTag.
	 * The source rows will be the ones the operation should consider for deletion.
	 * @OutInputRows The array to fill with the input rows that were created.
	 * @SourceRows The array holding the source rows that should be deleted. Automatically assigned to the FSourceColumn.
	 * @bForce Whether the FForceDeletionTag should be added to the input rows. Informs the operation to execute deletion regardless of restrictions.
	 * @bAddDescription Whether the FDescriptionColumn should be added to the input rows. Holds an FText which the operation may write to.
	 * @returns True if the requested number of rows were successfully created, otherwise false.
	 */
	UE_API bool CreateInputRows(UE::Editor::DataStorage::FRowHandleArray& OutInputRows, UE::Editor::DataStorage::FRowHandleArrayView SourceRows,
		bool bForce, bool bAddDescription) const;
	
protected:
	UE_API virtual void GetQueryDescription(UE::Editor::DataStorage::FQueryDescription& OutDescription) const override;

	UE::Editor::DataStorage::TableHandle InputTableForce;
	UE::Editor::DataStorage::TableHandle InputTableForceDescr;
};

#undef UE_API
