// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataStorage/CommonTypes.h"
#include "Elements/Framework/TypedElementHandle.h"
#include "TedsOperationInput.h"

#include "DropOperationInput.generated.h"

USTRUCT(meta = (DisplayName = "Drop Operation Target"))
struct FDropOperationTargetColumn final : public FEditorDataStorageColumn
{
	GENERATED_BODY()

	UE::Editor::DataStorage::RowHandle Value;
};

USTRUCT(meta = (DisplayName = "Drop Operation Name"))
struct FDropOperationNameColumn final : public FEditorDataStorageColumn
{
	GENERATED_BODY()

	FName Value;
};

USTRUCT(meta = (DisplayName = "Drop Operation Transform"))
struct FDropOperationTransformColumn final : public FEditorDataStorageColumn
{
	GENERATED_BODY()

	FTransform Value;
};

USTRUCT(meta = (DisplayName = "Drop Operation Folder"))
struct FDropOperationFolderColumn final : public FEditorDataStorageColumn
{
	GENERATED_BODY()

	FName Path;
};

USTRUCT(meta = (DisplayName = "Drop Operation Preview"))
struct FDropOperationPreviewTag final : public FEditorDataStorageTag
{
	GENERATED_BODY()
};

USTRUCT(meta = (DisplayName = "Drop Operation Typed Element Result"))
struct FDropOperationTypedElementResultColumn final : public FEditorDataStorageColumn
{
	GENERATED_BODY()

	TArray<FTypedElementHandle> Elements;
};

namespace UE::Editor::DataStorage::Operations
{
/** Holds the target row of the drop operation (e.g. a level). */	
using FDropTargetColumn = FDropOperationTargetColumn;
/** Informs the operation to create a preview. */
using FDropPreviewTag = FDropOperationPreviewTag;
/** Holds the name that should be used by the operation. */
using FDropNameColumn = FDropOperationNameColumn;
/** Holds the transform that should be used by the operation. */
using FDropTransformColumn = FDropOperationTransformColumn;
/** Holds the folder path that should be used by the operation. */
using FDropFolderColumn = FDropOperationFolderColumn;
/** Holds the TEv1 handles created by the operation. */
using FDropTypedElementResultColumn = FDropOperationTypedElementResultColumn;

namespace Utilities
{
	inline RowHandle GetDropTargetRow(const ICoreProvider& Storage, RowHandle InputRow)
	{
		const FDropTargetColumn* Column = Storage.GetColumn<FDropTargetColumn>(InputRow);
		return Column ? Column->Value : InvalidRowHandle;
	}
	
	inline FName GetDropName(const ICoreProvider& Storage, RowHandle InputRow)
	{
		const FDropNameColumn* Column = Storage.GetColumn<FDropNameColumn>(InputRow);
		return Column ? Column->Value : NAME_None;
	}
	
	inline FTransform GetDropTransform(const ICoreProvider& Storage, RowHandle InputRow)
	{
		const FDropTransformColumn* Column = Storage.GetColumn<FDropTransformColumn>(InputRow);
		return Column ? Column->Value : FTransform::Identity;
	}
	
	inline FName GetDropFolderPath(const ICoreProvider& Storage, RowHandle InputRow)
	{
		const FDropFolderColumn* Column = Storage.GetColumn<FDropFolderColumn>(InputRow);
		return Column ? Column->Path : NAME_None;
	}

	inline bool HasDropPreviewTag(const ICoreProvider& Storage, RowHandle InputRow)
	{
		return Storage.HasColumns<FDropPreviewTag>(InputRow);
	}
}
}
