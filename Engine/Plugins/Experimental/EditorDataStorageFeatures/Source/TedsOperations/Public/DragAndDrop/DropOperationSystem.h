// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DataStorage/ScopedRowHandle.h"
#include "TedsOperationSystem.h"

#include "DropOperationSystem.generated.h"

#define UE_API TEDSOPERATIONS_API

USTRUCT(meta = (DisplayName = "Drop Operation"))
struct FDropOperationTag final : public FEditorDataStorageTag
{
	GENERATED_BODY()
};
namespace UE::Editor::DataStorage::Operations
{
	/** Tag to identify this operation as a drop operation. UDropOperationSystem will only consider rows with this tag. */
	using FDropTag = FDropOperationTag;
}

/**
 * Operation system to execute operations that drag&drop something on a target. Only considers operations with the FDropTag.
*/
UCLASS(MinimalAPI)
class UDropOperationSystem : public UOperationSystem
{
	GENERATED_BODY()
public:
	UE_API UDropOperationSystem();
	UE_API virtual ~UDropOperationSystem() override;
	
	UE_API virtual void RegisterTables(UE::Editor::DataStorage::ICoreProvider& Storage) override;
	
	/**
	 * Utility method to create a batch of rows that can be used as input for drop operations.
	 * The method will create as many rows as source rows were passed in and automatically assign the FSourceColumn and FDropTag.
	 * The source rows will be the ones the drop operation should consider.
	 * @OutInputRows The array to fill with the input rows that were created.
	 * @SourceRows The array holding the source rows that should be considered by the drop operation. Automatically assigned to the FSourceColumn.
	 * @bAddDescription Whether the FDescriptionColumn should be added to the input rows. Holds an FText which the operation may write to.
	 * @returns True if the requested number of rows were successfully created, otherwise false.
	 */
	UE_API bool CreateInputRows(UE::Editor::DataStorage::FRowHandleArray& OutInputRows, UE::Editor::DataStorage::FRowHandleArrayView SourceRows,
		UE::Editor::DataStorage::RowHandle TargetRow, bool bAddDescription) const;

protected:
	UE_API virtual void GetQueryDescription(UE::Editor::DataStorage::FQueryDescription& OutDescription) const;
};

#undef UE_API
