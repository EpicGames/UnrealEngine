// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DragAndDrop/DecoratedDragDropOp.h"
#include "Elements/Framework/TypedElementRowHandleArray.h"
#include "Input/DragAndDrop.h"

#define UE_API TEDSOPERATIONS_API

namespace UE::Editor::DataStorage::Widgets
{
	// TEDS UI TODO: Instead of using a decorated operation use a default one and allow decorator customization using TEDS UI
	class FTedsDragDropOp : public FDecoratedDragDropOp
	{
	public:
		DRAG_DROP_OPERATOR_TYPE(FTedsDragDropOp, FDecoratedDragDropOp)

		UE_API static TSharedRef<FTedsDragDropOp> New(FRowHandleArray InDraggedRows);

		FRowHandleArrayView    GetRows() const     { return DraggedRows.GetRows(); }
		FRowHandleArray&       GetRowArray()       { return DraggedRows; }
		const FRowHandleArray& GetRowArray() const { return DraggedRows; }
		
	protected:
		void Init(FRowHandleArray InDraggedRows);
		
	protected:
		FRowHandleArray DraggedRows;
	};
}

#undef UE_API