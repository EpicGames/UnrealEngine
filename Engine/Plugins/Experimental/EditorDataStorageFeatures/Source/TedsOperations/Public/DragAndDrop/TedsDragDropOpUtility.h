// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#define UE_API TEDSOPERATIONS_API

class FDragDropOperation;
namespace UE::Editor::DataStorage
{
class FRowHandleArray;
}

namespace UE::Editor::DataStorage::DragAndDrop
{

/** Tries to convert the dropped data into rows. Returns true if all data was successfully converted. */
UE_API bool GetRowsFromData(FRowHandleArray& OutRows, const FDragDropOperation& InData);

}

#undef UE_API
