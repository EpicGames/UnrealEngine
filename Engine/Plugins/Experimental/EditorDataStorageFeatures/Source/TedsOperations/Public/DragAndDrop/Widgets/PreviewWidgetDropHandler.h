// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DragAndDrop/Widgets/WidgetDropHandler.h"

#define UE_API TEDSOPERATIONS_API

struct FTypedElementHandle;

namespace UE::Editor
{

/**
 * Utility type for widgets that accepts drag&drop-events and preview creation on drag-over using teds operations.
 */
class FPreviewWidgetDropHandler : public FWidgetDropHandler
{
public:
	UE_API FPreviewWidgetDropHandler(TWeakObjectPtr<UTypedElementSelectionSet> SelectionSet);
	UE_API virtual ~FPreviewWidgetDropHandler() override;

	bool HasPreviews() const { return !PreviewInputRows.IsEmpty(); }
	
protected:
	UE_API virtual FDropResult ResetOperations(DataStorage::ICoreProvider& Storage, UDropOperationSystem& DropSystem) override;	
	UE_API virtual TOptional<FDropResult> Update(DataStorage::ICoreProvider& Storage, UDropOperationSystem& DropSystem, const FGeometry& Geometry,
		const FDragDropEvent& InputEvent) override;
	UE_API virtual void Stop(DataStorage::ICoreProvider& Storage, UDropOperationSystem& DropSystem) override;
	UE_API virtual void Drop(DataStorage::ICoreProvider& Storage, UDropOperationSystem& DropSystem, const FGeometry& Geometry,
		const FDragDropEvent& InputEvent) override;
	
	UE_API virtual void CreatePreviews(DataStorage::ICoreProvider& Storage, UDropOperationSystem& DropSystem);
	UE_API virtual void UpdatePreviews(DataStorage::ICoreProvider& Storage, UDropOperationSystem& DropSystem);
	UE_API virtual void RemovePreviews(DataStorage::ICoreProvider& Storage, UDropOperationSystem& DropSystem);

	UE_API void GetPreviewRows(DataStorage::FRowHandleArray& OutRows, DataStorage::ICoreProvider& Storage) const;

	UE_API virtual int32 ExecutePreviewDrop(DataStorage::ICoreProvider& Storage, UDropOperationSystem& DropSystem, DataStorage::FRowHandleArrayView InputRows);
	UE_API virtual bool ExecutePreviewDropSingle(DataStorage::ICoreProvider& Storage, UDropOperationSystem& DropSystem, DataStorage::RowHandle InputRow);
	
protected:
	DataStorage::FRowHandleArray PreviewInputRows;

	TArray<FTypedElementHandle> PreviewElements;
};

}

#undef UE_API