// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DragAndDrop/Widgets/PreviewWidgetDropHandler.h"

class FEditorViewportClient;

#define UE_API TEDSOPERATIONS_API

namespace UE::Editor
{

class FViewportDropHandler : public FPreviewWidgetDropHandler
{
public:
	using FTargetRowGetter = TFunction<DataStorage::RowHandle(const FEditorViewportClient&)>;

	/**
	 * Construct a viewport drop handler.
	 * @param InViewportClient The client of the viewport the handler is representing.
	 * @param InGetDefaultTargetRow A function to return a "default" target during drag&drop that is used when not tracing.
	 */
	UE_API FViewportDropHandler(TSharedPtr<FEditorViewportClient> InViewportClient, FTargetRowGetter InGetDefaultTargetRow);
	UE_API virtual ~FViewportDropHandler() override;
	
	UE_API virtual FReply OnDrop(const FGeometry& Geometry, const FDragDropEvent& InputEvent) override;

protected:
	UE_API virtual EUpdateParameters UpdateParameters(DataStorage::ICoreProvider& Storage, const FGeometry& Geometry, const FDragDropEvent& InputEvent) override;
	UE_API virtual void SetRowsFromParameters(DataStorage::ICoreProvider& Storage, DataStorage::FRowHandleArrayView Rows) const override;
	UE_API virtual void Stop(DataStorage::ICoreProvider& Storage, UDropOperationSystem& DropSystem) override;
	
	UE_API virtual void CreatePreviews(DataStorage::ICoreProvider& Storage, UDropOperationSystem& DropSystem) override;
	UE_API virtual void UpdatePreviews(DataStorage::ICoreProvider& Storage, UDropOperationSystem& DropSystem) override;
	UE_API virtual void RemovePreviews(DataStorage::ICoreProvider& Storage, UDropOperationSystem& DropSystem) override;

	UE_API void InvalidateViewport();
	
protected:
	TSharedPtr<FEditorViewportClient> ViewportClient;
	FTargetRowGetter GetDefaultTargetRow; // A function to return a "default" target during drag&drop that is used when not tracing.
	
	FIntPoint LastCursorPos;
	DataStorage::RowHandle LastTargetRow;
	FTransform LastTransform;
};

}

#undef UE_API