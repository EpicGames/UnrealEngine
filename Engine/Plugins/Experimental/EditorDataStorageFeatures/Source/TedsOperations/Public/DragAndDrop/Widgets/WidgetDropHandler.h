// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/Framework/TypedElementRowHandleArray.h"
#include "Elements/Framework/TypedElementRowHandleArrayView.h"
#include "Input/Reply.h"
#include "Misc/Optional.h"

class FDragDropEvent;
class FDragDropOperation;
class UDropOperationSystem;
class UTypedElementSelectionSet;
struct FGeometry;
namespace UE::Editor::DataStorage
{
class ICoreProvider;
}

#define UE_API TEDSOPERATIONS_API

namespace UE::Editor
{

/**
 * Utility type for widgets that accepts drag&drop-events using teds operations.
 */ 
class FWidgetDropHandler
{
public:
	struct FDropResult
	{
		bool bCanDrop = false;
		FText Tooltip = FText();
	};

protected:
	enum class EUpdateParameters
	{
		NoChanges, // Nothing has changed.
		MinorChanges, // Some changes to input, but no need to reset operations.
		ResetOperations, // Input has major changes, reset operations.
	};

public:
	UE_API static void ShowErrorNotifications(const DataStorage::ICoreProvider& Storage, DataStorage::FRowHandleArrayView InputRows);
	
	UE_API FWidgetDropHandler(TWeakObjectPtr<UTypedElementSelectionSet> SelectionSet);
	UE_API virtual ~FWidgetDropHandler();

	UE_API virtual bool OnDragEnter(const FGeometry& Geometry, const FDragDropEvent& InputEvent);
	UE_API virtual FReply OnDragOver(const FGeometry& Geometry, const FDragDropEvent& InputEvent);
	UE_API virtual bool OnDragLeave(const FDragDropEvent& InputEvent);
	UE_API virtual FReply OnDrop(const FGeometry& Geometry, const FDragDropEvent& InputEvent);
	UE_API virtual FDragDropOperation* GetCurrentOperation() const;

protected:
	/** Creates input rows for the given drag&drop input event. */
	UE_API virtual bool PrepareInput(DataStorage::ICoreProvider& Storage, UDropOperationSystem& DropSystem, const FDragDropOperation& Operation);
	/** Creates input rows for the given drag&drop input event. */
	UE_API virtual bool PrepareInput(DataStorage::ICoreProvider& Storage, UDropOperationSystem& DropSystem, DataStorage::FRowHandleArrayView SourceRows);
	/** Finds suitable operations for the given input rows and sets 'bIsValidDrop'. */
	UE_API virtual FDropResult ResetOperations(DataStorage::ICoreProvider& Storage, UDropOperationSystem& DropSystem);
	
	/** Method to update cached parameters for the given input data. */
	UE_API virtual EUpdateParameters UpdateParameters(DataStorage::ICoreProvider& Storage, const FGeometry& Geometry, const FDragDropEvent& InputEvent);
	/** Method to assign cached parameters to the given rows. */
	UE_API virtual void SetRowsFromParameters(DataStorage::ICoreProvider& Storage, DataStorage::FRowHandleArrayView Rows) const;
	
	/** Starts a drag-operation internally. Returns an unset value if the operation is ignored, a result with True if the drop operation would be valid or False if invalid. */
	UE_API virtual TOptional<FDropResult> Start(DataStorage::ICoreProvider& Storage, UDropOperationSystem& DropSystem, const FGeometry& Geometry,
		const FDragDropEvent& InputEvent, FDragDropOperation& Operation);
	/** Updates the current drag-operation internally. Returns an updated result or unset optional if nothing has changed. */
	UE_API virtual TOptional<FDropResult> Update(DataStorage::ICoreProvider& Storage, UDropOperationSystem& DropSystem, const FGeometry& Geometry,
		const FDragDropEvent& InputEvent);
	/** Stops the current drag-operation. */
	UE_API virtual void Stop(DataStorage::ICoreProvider& Storage, UDropOperationSystem& DropSystem);
	/** Stops and executes the drop for the current drag-operation. */
	UE_API virtual void Drop(DataStorage::ICoreProvider& Storage, UDropOperationSystem& DropSystem, const FGeometry& Geometry,
		const FDragDropEvent& InputEvent);

	UE_API virtual void UpdateOperationDecorator(FDragDropOperation& Operation, TOptional<FDropResult> Result) const;

	UE_API virtual int32 ExecuteDrop(DataStorage::ICoreProvider& Storage, UDropOperationSystem& DropSystem, DataStorage::FRowHandleArrayView InputRows);
	UE_API virtual bool ExecuteDropSingle(DataStorage::ICoreProvider& Storage, UDropOperationSystem& DropSystem, DataStorage::RowHandle InputRow);

protected:
	TWeakObjectPtr<UTypedElementSelectionSet> SelectionSet;

	FDragDropOperation* CurrentOperation;
	DataStorage::FRowHandleArray InputRows;
	bool bIsValidDrop;
};

}

#undef UE_API