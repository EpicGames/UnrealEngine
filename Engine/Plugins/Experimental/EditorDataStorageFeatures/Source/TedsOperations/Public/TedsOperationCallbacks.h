// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataStorage/Handles.h"
#include "Misc/Optional.h"
#include "TedsOperationResult.h"
#include "Templates/Function.h"

namespace UE::Editor::DataStorage
{
class ICoreProvider;
}

namespace UE::Editor::DataStorage::Operations
{
	/** Probe call to test superficially if the given input is potentially acceptable for the operation. */
	using FProbeCallback = TFunction<bool(const ICoreProvider&, RowHandle)>;
	/** Test call to query if the operation for the given input can be executed. */
	using FTestCallback  = TFunction<bool(ICoreProvider&, RowHandle)>;
	/** Apply call to execute the operation for the given input. */
	using FApplyCallback = TFunction<TOptional<FResult>(ICoreProvider&, RowHandle)>;
}
