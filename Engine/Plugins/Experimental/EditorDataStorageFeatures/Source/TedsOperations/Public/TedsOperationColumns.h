// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataStorage/CommonTypes.h"
#include "TedsOperationCallbacks.h"

#include "TedsOperationColumns.generated.h"

USTRUCT(meta = (DisplayName = "Operation Name"))
struct FOperationNameColumn final : public FEditorDataStorageColumn
{
	GENERATED_BODY()

	FName Value;
};

USTRUCT(meta = (DisplayName = "Operation Probe"))
struct FOperationProbeColumn final : public FEditorDataStorageColumn
{
	GENERATED_BODY()

	UE::Editor::DataStorage::Operations::FProbeCallback Callback;
};

USTRUCT(meta = (DisplayName = "Operation Test"))
struct FOperationTestColumn final : public FEditorDataStorageColumn
{
	GENERATED_BODY()

	UE::Editor::DataStorage::Operations::FTestCallback Callback;
};

USTRUCT(meta = (DisplayName = "Operation Apply"))
struct FOperationApplyColumn final : public FEditorDataStorageColumn
{
	GENERATED_BODY()

	UE::Editor::DataStorage::Operations::FApplyCallback Callback;
};

USTRUCT(meta = (DisplayName = "Operation Priority"))
struct FOperationPriorityColumn final : public FEditorDataStorageColumn
{
	GENERATED_BODY()

	int64 Value;
};

namespace UE::Editor::DataStorage::Operations
{
	/** Column for the name of the operation. */
	using FNameColumn = FOperationNameColumn;
	/** Probe call column, to test superficially if a given input is potentially acceptable for the operation. */
	using FProbeColumn = FOperationProbeColumn;
	/** Test call column, to query if the operation for a given input can be executed. */
	using FTestColumn = FOperationTestColumn;
	/** Apply call column, to execute the operation for a given input. */
	using FApplyColumn = FOperationApplyColumn;
	/** Priority value of this operation. Used for the default sorting when calling the operation system. */
	using FPriorityColumn = FOperationPriorityColumn;
}
