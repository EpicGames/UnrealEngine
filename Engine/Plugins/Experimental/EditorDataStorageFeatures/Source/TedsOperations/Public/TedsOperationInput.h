// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataStorage/CommonTypes.h"
#include "Elements/Interfaces/TypedElementDataStorageInterface.h"
#include "TedsOperationResult.h"

#include "TedsOperationInput.generated.h"

USTRUCT(meta = (DisplayName = "Operation Input Source"))
struct FOperationSourceColumn final : public FEditorDataStorageColumn
{
	GENERATED_BODY()

	UE::Editor::DataStorage::RowHandle Value;
};

USTRUCT(meta = (DisplayName = "Operation Input Description"))
struct FOperationDescriptionColumn final : public FEditorDataStorageColumn
{
	GENERATED_BODY()

	FText Value;
};

USTRUCT(meta = (DisplayName = "Operation Test Result"))
struct FOperationTestResultTag final : public FEditorDataStorageTag
{
	GENERATED_BODY()
};

USTRUCT(meta = (DisplayName = "Operation Result"))
struct FOperationResultColumn final : public FEditorDataStorageColumn
{
	GENERATED_BODY()

	UE::Editor::DataStorage::Operations::FResult Value;
};

namespace UE::Editor::DataStorage::Operations
{
/** Holds a general purpose source row for the operation. */
using FSourceColumn = FOperationSourceColumn;
/** Holds a description which the operation may write to. */
using FDescriptionColumn = FOperationDescriptionColumn;
/** Denotes that the test operation has been successful. */
using FTestResultTag = FOperationTestResultTag;
/** Denotes that the operation has been sucessful. */
using FResultColumn = FOperationResultColumn;

namespace Utilities
{
/** Returns the value of the source column, if set. */
inline RowHandle GetSourceRow(const ICoreProvider& Storage, RowHandle InputRow)
{
	const FSourceColumn* Column = Storage.GetColumn<FSourceColumn>(InputRow);
	return Column ? Column->Value : InvalidRowHandle;
}

/** Returns a pointer to the description text, if set.*/
inline FText* GetDescriptionPtr(ICoreProvider& Storage, RowHandle InputRow)
{
	FDescriptionColumn* Column = Storage.GetColumn<FDescriptionColumn>(InputRow);
	return Column ? &Column->Value : nullptr;
}

/** Returns a pointer to the description text, if set.*/
inline const FText* GetDescriptionPtr(const ICoreProvider& Storage, RowHandle InputRow)
{
	const FDescriptionColumn* Column = Storage.GetColumn<FDescriptionColumn>(InputRow);
	return Column ? &Column->Value : nullptr;
}

/** Returns the description text as a value. */
inline FText GetDescription(const ICoreProvider& Storage, RowHandle InputRow)
{
	const FText* Description = GetDescriptionPtr(Storage, InputRow);
	return Description ? *Description : FText();
}
}
}


