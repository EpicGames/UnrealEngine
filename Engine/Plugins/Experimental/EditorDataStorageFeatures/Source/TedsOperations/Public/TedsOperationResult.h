// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/Framework/TypedElementRowHandleArray.h"

namespace UE::Editor::DataStorage::Operations
{
	
struct FResult
{
	FRowHandleArray Created;
	FRowHandleArray Changed;
	FRowHandleArray Removed;
};

}
