// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataStorage/Handles.h"
#include "Elements/Interfaces/TypedElementDataStorageFactory.h"
#include "TedsOperationCallbacks.h"
#include "TedsOperationResult.h"

#include "TedsOperationSystem.generated.h"

#define UE_API TEDSOPERATIONS_API

namespace UE::Editor::DataStorage
{
struct FQueryDescription;
}
namespace UE::Editor::DataStorage::QueryStack
{
class IQueryNode;
class IRowNode;
class FRowFilterNode;
class FExplicitUpdateExecutor;
}

/** 
 * System to execute externally customizable logic for a given purpose and input.
 * 
 * By deriving this type, an "OperationSystem" for a specific purpose may be created, to which customizable "Operations" can be registered.
 * The system may then be invoked to "test" or "apply" a registered operation that fits the given input.
 */
UCLASS(MinimalAPI, Abstract)
class UOperationSystem : public UEditorDataStorageFactory
{
	GENERATED_BODY()
public:
	static constexpr int64 DefaultPriority = 0;
	
	using FFilter = TFunction<bool(const UE::Editor::DataStorage::ICoreProvider&, UE::Editor::DataStorage::RowHandle)>;
	using FLess   = TFunction<bool(const UE::Editor::DataStorage::ICoreProvider&, UE::Editor::DataStorage::RowHandle, UE::Editor::DataStorage::RowHandle)>;

	/** Default sorter for invoking an OperationSystem. Sorts by priority column first and name column second. */
	struct FDefaultLess
	{
		UE_API bool operator()(const UE::Editor::DataStorage::ICoreProvider& Storage, UE::Editor::DataStorage::RowHandle A, UE::Editor::DataStorage::RowHandle B) const;
	};

public:
	UE_API UOperationSystem();
	UE_API virtual ~UOperationSystem() override;

	UE_API virtual void PreRegister(UE::Editor::DataStorage::ICoreProvider& Storage) override;
	UE_API virtual void RegisterTables(UE::Editor::DataStorage::ICoreProvider& Storage) override;
	UE_API virtual void RegisterQueries(UE::Editor::DataStorage::ICoreProvider& Storage) override;

	/** Utility method to create a row with columns for the given input operation. */
	UE_API UE::Editor::DataStorage::RowHandle AddOperation(FName Name, TNotNull<UE::Editor::DataStorage::Operations::FApplyCallback> Operation,
		UE::Editor::DataStorage::Operations::FTestCallback Test, UE::Editor::DataStorage::Operations::FProbeCallback Probe,
		int64 Priority = DefaultPriority);

	/** Retrieves all operation rows that accept (probe call) the given input row.
	 * @param OutRows The array to fill with the operation rows.
	 * @param InputRow The row that contains input data that will be passed to the operations.
	 * @param Filter [optional] Predicate that can be used to customarily filter the operation rows. False = Skip, True = Keep.
	 * @param Less [optional] Sorter that can be used to customarily sort the operation rows.
	 */
	UE_API void GetOperations(TArray<UE::Editor::DataStorage::RowHandle>& OutRows, UE::Editor::DataStorage::RowHandle InputRow,
		const FFilter& Filter = {}, const FLess& Less = FDefaultLess()) const;

	/** Invokes the Test callback for each input row for all operations that accept (probe call) it and writes the result of the first success to the input row.
	 * @param InputRows The rows that contain input data that will be passed to the operations.
	 * @param Filter [optional] Predicate that can be used to customarily filter the operation rows. False = Skip, True = Keep.
	 * @param Less [optional] Sorter that can be used to customarily sort the operation rows.
	 * @returns The number of successfully tested input rows.
	 */
	UE_API int32 Test(UE::Editor::DataStorage::FRowHandleArrayView InputRows, const FFilter& Filter = {}, const FLess& Less = FDefaultLess()) const;

	/** Invokes the Apply callback for each input row for all operations that accept (probe call) it and writes the result of the first success to the input row.
	 * @param InputRows The rows that contain input data that will be passed to the operations.
	 * @param Filter [optional] Predicate that can be used to customarily filter the operation rows. False = Skip, True = Keep.
	 * @param Less [optional] Sorter that can be used to customarily sort the operation rows.
	 * @returns The number of successfully applied input rows.
	 */
	UE_API int32 Apply(UE::Editor::DataStorage::FRowHandleArrayView InputRows, const FFilter& Filter = {}, const FLess& Less = FDefaultLess()) const;
	
	/**
	 * Utility method to create a batch of rows that can be used as input for operations.
	 * The method will create as many rows as source rows were passed in and automatically assign the FSourceColumn.
	 * The source rows will be the ones the operation should consider.
	 * @OutInputRows The array to fill with the input rows that were created.
	 * @SourceRows The array holding the source rows that should be considered by the operation. Automatically assigned to the FSourceColumn.
	 * @bAddDescription Whether the FDescriptionColumn should be added to the input rows. Holds an FText which the operation may write to.
	 * @returns True if the requested number of rows were successfully created, otherwise false.
	 */
	UE_API bool CreateInputRows(UE::Editor::DataStorage::FRowHandleArray& OutRows, UE::Editor::DataStorage::FRowHandleArrayView SourceRows,
		bool bAddDescription = false) const;
	
	/** Utility method to remove a batch of rows. */
	UE_API void RemoveInputRows(UE::Editor::DataStorage::FRowHandleArrayView RowsToRemove) const;
	
protected:
	UE_API virtual void GetQueryDescription(UE::Editor::DataStorage::FQueryDescription& OutDescription) const;
	
protected:
	UE::Editor::DataStorage::ICoreProvider*                     StoragePtr;
	UE::Editor::DataStorage::TableHandle                        OperationTable;
	TSharedPtr<UE::Editor::DataStorage::QueryStack::IQueryNode> QueryNode;
	TSharedPtr<UE::Editor::DataStorage::QueryStack::IRowNode>   ResultsNode;
	TSharedPtr<UE::Editor::DataStorage::QueryStack::FExplicitUpdateExecutor> Executor;

protected:
	UE::Editor::DataStorage::TableHandle InputTable;
	UE::Editor::DataStorage::TableHandle InputTableDescr;
};

#undef UE_API
