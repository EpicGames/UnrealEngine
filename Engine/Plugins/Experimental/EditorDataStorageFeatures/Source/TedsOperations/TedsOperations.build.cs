// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

// Experimental test module. please refrain from depending on it until this warning is removed
public class TedsOperations : ModuleRules
{
	public TedsOperations(ReadOnlyTargetRules Target) : base(Target)
	{
		// Enable truncation warnings in this module
		CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Warning;

		ShortName = "TEDSOperations";

		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		if (Target.bBuildEditor)
		{
			PublicIncludePaths.AddRange(new string[] { });
			PrivateIncludePaths.AddRange(new string[] { });

			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
					"SlateCore",
					"TypedElementFramework",
					"UnrealEd",
				});

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"Engine",
					"SceneOutliner",
					"Slate",
					"TedsQueryStack",
					"TypedElementRuntime",
				});

		}
	}
}
