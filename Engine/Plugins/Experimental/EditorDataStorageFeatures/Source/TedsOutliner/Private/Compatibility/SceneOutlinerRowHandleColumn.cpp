// Copyright Epic Games, Inc. All Rights Reserved.

#include "Compatibility/SceneOutlinerRowHandleColumn.h"

#include "SortHelper.h"
#include "TedsOutlinerItem.h"
#include "DataStorage/Features.h"
#include "Elements/Interfaces/TypedElementDataStorageUiInterface.h"
#include "TedsTableViewerColumn.h"
#include "Compatibility/SceneOutlinerTedsBridge.h"

#define LOCTEXT_NAMESPACE "SceneOutlinerRowHandleColumn"

FSceneOutlinerRowHandleColumn::FSceneOutlinerRowHandleColumn(ISceneOutliner& SceneOutliner)
	: WeakSceneOutliner(StaticCastSharedRef<ISceneOutliner>(SceneOutliner.AsShared()))
{
	CreateWidgetConstructor();
}

FSceneOutlinerRowHandleColumn::FSceneOutlinerRowHandleColumn(ISceneOutliner& SceneOutliner, const FGetRowHandle& InGetRowHandle)
	: WeakSceneOutliner(StaticCastSharedRef<ISceneOutliner>(SceneOutliner.AsShared()))
	, GetRowHandle(InGetRowHandle)
{
	CreateWidgetConstructor();
}

void FSceneOutlinerRowHandleColumn::CreateWidgetConstructor()
{
	using namespace UE::Editor::DataStorage;
	auto AssignWidgetToColumn = [this](IUiProvider::FWidgetConstructorPtr WidgetConstructor, TConstArrayView<TWeakObjectPtr<const UScriptStruct>>)
	{
		TableViewerColumn = MakeShared<FTedsTableViewerColumn>(TEXT("Row Handle"), WidgetConstructor);
		return false;
	};
	
	IUiProvider* StorageUi = GetMutableDataStorageFeature<IUiProvider>(UiFeatureName);
	checkf(StorageUi, TEXT("FSceneOutlinerRowHandleColumn created before data storage interfaces were initialized."))
	
	StorageUi->CreateWidgetConstructors(
		StorageUi->FindPurpose(IUiProvider::FPurposeInfo("General", "Cell", "RowHandle").GeneratePurposeID()),
		FMetaDataView(), AssignWidgetToColumn);
}


FName FSceneOutlinerRowHandleColumn::GetID()
{
	static const FName ID("Row Handle");
	return ID;
}

FName FSceneOutlinerRowHandleColumn::GetColumnID()
{
	return GetID();
}

SHeaderRow::FColumn::FArguments FSceneOutlinerRowHandleColumn::ConstructHeaderRowColumn()
{
	return SHeaderRow::Column(GetID())
	.FillWidth(2)
	.HeaderComboVisibility(EHeaderComboVisibility::OnHover);
}

const TSharedRef<SWidget> FSceneOutlinerRowHandleColumn::ConstructRowWidget(FSceneOutlinerTreeItemRef TreeItem, const STableRow<FSceneOutlinerTreeItemPtr>& Row)
{
	using namespace UE::Editor::DataStorage;
	if (!TableViewerColumn)
	{
		return SNew(STextBlock)
				.Text(LOCTEXT("MissingWidgetConstructor",
					"Row Handles cannot be displayed if widget constructor for the General.Cell.RowHandle purpose was not found"));
	}
	
	auto SceneOutliner = WeakSceneOutliner.Pin();
	check(SceneOutliner.IsValid());
	
	RowHandle RowHandle = InvalidRowHandle;

	// Check any custom delegates to get the row handle of the item
	if (GetRowHandle.IsBound())
	{
		RowHandle = GetRowHandle.Execute(TreeItem.Get());
	}
	// Otherwise it could just be a regular TedsOutlinerTreeItem which we know how to handle
	else if (const UE::Editor::Outliner::FTedsOutlinerTreeItem* OutlinerTreeItem = TreeItem->CastTo<UE::Editor::Outliner::FTedsOutlinerTreeItem>())
	{
		RowHandle = OutlinerTreeItem->GetRowHandle();
	}

	ICoreProvider* Storage = TableViewerColumn->GetStorage();

	if (ensureMsgf(Storage, TEXT("Cannot create a widget to display row handles before TEDS is initialized")))
	{
		if (Storage->IsRowAssigned(RowHandle))
		{
			if (TSharedPtr<SWidget> Widget = TableViewerColumn->ConstructRowWidget(RowHandle))
			{
				return Widget.ToSharedRef();
			}
		}
	}
	
	return SNullWidget::NullWidget;
}

void FSceneOutlinerRowHandleColumn::PopulateSearchStrings(const ISceneOutlinerTreeItem& Item, TArray<FString>& OutSearchStrings) const
{
	if (const UE::Editor::Outliner::FTedsOutlinerTreeItem* OutlinerTreeItem = Item.CastTo<UE::Editor::Outliner::FTedsOutlinerTreeItem>())
	{
		OutSearchStrings.Add(LexToString<FString>(OutlinerTreeItem->GetRowHandle()));
	}

}

void FSceneOutlinerRowHandleColumn::SortItems(TArray<FSceneOutlinerTreeItemPtr>& OutItems, const EColumnSortMode::Type SortMode) const
{
	FSceneOutlinerSortHelper<UE::Editor::DataStorage::RowHandle>()
		/** Sort by type first */
		.Primary([this](const ISceneOutlinerTreeItem& Item)
		{
			if (const UE::Editor::Outliner::FTedsOutlinerTreeItem* OutlinerTreeItem = Item.CastTo<UE::Editor::Outliner::FTedsOutlinerTreeItem>())
			{
				return OutlinerTreeItem->GetRowHandle();
			}

			return UE::Editor::DataStorage::InvalidRowHandle;
		}, SortMode)
		.Sort(OutItems);
}

#undef LOCTEXT_NAMESPACE
