// Copyright Epic Games, Inc. All Rights Reserved.

#include "Compatibility/SceneOutlinerTedsBridge.h"

#include "Columns/SceneOutlinerColumns.h"
#include "Columns/TedsActorMobilityColumns.h"
#include "Columns/TedsActorUncachedLightsColumns.h"
#include "Columns/TedsOutlinerColumns.h"
#include "Elements/Columns/TypedElementCompatibilityColumns.h"
#include "Elements/Columns/TypedElementLabelColumns.h"
#include "Elements/Columns/TypedElementMiscColumns.h"
#include "Elements/Columns/TypedElementPackageColumns.h"
#include "Elements/Columns/TypedElementTypeInfoColumns.h"
#include "Elements/Columns/TypedElementVisibilityColumns.h"
#include "Columns/TedsActorSocketColumns.h"
#include "Columns/TedsActorWorldPartitionColumns.h"
#include "Elements/Interfaces/TypedElementDataStorageInterface.h"
#include "Elements/Interfaces/TypedElementDataStorageUiInterface.h"
#include "Layers/Columns/LayersColumns.h"
#include "Modules/ModuleManager.h"
#include "SceneOutlinerPublicTypes.h"
#include "TedsOutlinerHelpers.h"
#include "ActorDesc/TedsActorDescColumns.h"
#include "Widgets/Docking/SDockTab.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SceneOutlinerTedsBridge)

#define LOCTEXT_NAMESPACE "SceneOutlinerTedsBridge"

//
// USceneOutlinerTedsBridgeFactory
// 

void USceneOutlinerTedsBridgeFactory::PreRegister(UE::Editor::DataStorage::ICoreProvider& DataStorage)
{
	using namespace UE::Editor::DataStorage;

	// Map the type column from the TEDS to the default Outliner type column, so we can show type info for objects not in TEDS
	TEDSToOutlinerDefaultColumnMapping.Add(FTypedElementClassTypeInfoColumn::StaticStruct(), FSceneOutlinerBuiltInColumnTypes::ActorInfo());
	TEDSToOutlinerDefaultColumnMapping.Add(FVisibleInEditorColumn::StaticStruct(), FSceneOutlinerBuiltInColumnTypes::Gutter());
	TEDSToOutlinerDefaultColumnMapping.Add(FUObjectIdNameColumn::StaticStruct(), FSceneOutlinerBuiltInColumnTypes::IDName());
	TEDSToOutlinerDefaultColumnMapping.Add(FTedsActorMobilityColumn::StaticStruct(), FSceneOutlinerBuiltInColumnTypes::Mobility());
	TEDSToOutlinerDefaultColumnMapping.Add(FTypedElementLabelColumn::StaticStruct(), FSceneOutlinerBuiltInColumnTypes::Label());
	TEDSToOutlinerDefaultColumnMapping.Add(FLevelColumn::StaticStruct(), FSceneOutlinerBuiltInColumnTypes::Level());
	TEDSToOutlinerDefaultColumnMapping.Add(FTedsPrimaryPackageObjectTag::StaticStruct(), FSceneOutlinerBuiltInColumnTypes::Unsaved());
	TEDSToOutlinerDefaultColumnMapping.Add(FTedsActorSocketColumn::StaticStruct(), FSceneOutlinerBuiltInColumnTypes::Socket());
	TEDSToOutlinerDefaultColumnMapping.Add(FEditorDataStorageActorLayersColumn::StaticStruct(), FSceneOutlinerBuiltInColumnTypes::Layer());
	TEDSToOutlinerDefaultColumnMapping.Add(FWorldPartitionPinnedColumn::StaticStruct(), FSceneOutlinerBuiltInColumnTypes::Pinned());
	TEDSToOutlinerDefaultColumnMapping.Add(FWorldPartitionDataLayerColumn::StaticStruct(), FSceneOutlinerBuiltInColumnTypes::DataLayer());
	TEDSToOutlinerDefaultColumnMapping.Add(FWorldPartitionSubPackageColumn::StaticStruct(), FSceneOutlinerBuiltInColumnTypes::SubPackage());
	TEDSToOutlinerDefaultColumnMapping.Add(FWorldPartitionContentBundleColumn::StaticStruct(), FSceneOutlinerBuiltInColumnTypes::ContentBundle());
	TEDSToOutlinerDefaultColumnMapping.Add(FTedsActorUncachedLightsTag::StaticStruct(), FSceneOutlinerBuiltInColumnTypes::UncachedLights());
}

void USceneOutlinerTedsBridgeFactory::RegisterTables(UE::Editor::DataStorage::ICoreProvider& DataStorage)
{
	DataStorage.RegisterTable<FSceneOutlinerColumn, FTedsOutlinerColumnRefreshEventColumn, FTedsOutlinerSelectionChangeColumn, FTedsOutlinerPendingItemActionsColumn, FTedsOutlinerRowHandleDealiaserColumn>(UE::Editor::Outliner::Helpers::GetTedsOutlinerTableName());
}

void USceneOutlinerTedsBridgeFactory::RegisterWidgetPurposes(UE::Editor::DataStorage::IUiProvider& DataStorageUi) const
{
	UE::Editor::DataStorage::IUiProvider::FPurposeID GeneralRowLabelPurposeID =
		UE::Editor::DataStorage::IUiProvider::FPurposeInfo("General", "RowLabel", NAME_None).GeneratePurposeID();

	UE::Editor::DataStorage::IUiProvider::FPurposeID GeneralHeaderPurposeID =
		UE::Editor::DataStorage::IUiProvider::FPurposeInfo("General", "Header", NAME_None).GeneratePurposeID();

	DataStorageUi.RegisterWidgetPurpose(
		UE::Editor::DataStorage::IUiProvider::FPurposeInfo("SceneOutliner", "Header", NAME_None,
			UE::Editor::DataStorage::IUiProvider::EPurposeType::UniqueByNameAndColumn,
			LOCTEXT("HeaderWidgetPurpose", "Widgets for headers in any Scene Outliner for specific columns or column combinations."),
			GeneralHeaderPurposeID));
	
	DataStorageUi.RegisterWidgetPurpose(
		UE::Editor::DataStorage::IUiProvider::FPurposeInfo("SceneOutliner", "Cell", NAME_None,
			UE::Editor::DataStorage::IUiProvider::EPurposeType::UniqueByNameAndColumn,
			LOCTEXT("CellWidgetPurpose", "Widgets for cells in any Scene Outliner for specific columns or column combinations."),
			DataStorageUi.GetGeneralWidgetPurposeID()));


	DataStorageUi.RegisterWidgetPurpose(
		UE::Editor::DataStorage::IUiProvider::FPurposeInfo("SceneOutliner", "RowLabel", NAME_None,
			UE::Editor::DataStorage::IUiProvider::EPurposeType::UniqueByNameAndColumn,
			LOCTEXT("ItemCellWidgetPurpose", "Widgets for cells in any Scene Outliner that are specific to the Item label column."),
			GeneralRowLabelPurposeID));
}

FName USceneOutlinerTedsBridgeFactory::FindOutlinerColumnFromTedsColumns(TConstArrayView<TWeakObjectPtr<const UScriptStruct>> TEDSColumns) const
{
	// Currently, the algorithm naively looks through the mapping and returns the first match
	for(const TWeakObjectPtr<const UScriptStruct>& Column : TEDSColumns)
	{
		if (const FName* FoundDefaultColumn = TEDSToOutlinerDefaultColumnMapping.Find(Column))
		{
			return *FoundDefaultColumn;
		}
	}
	return NAME_None;
}

void USceneOutlinerTedsBridgeFactory::RemoveTedsColumnMappingToOutlinerColumn(TWeakObjectPtr<const UScriptStruct> TEDSColumnToRemove)
{
	TEDSToOutlinerDefaultColumnMapping.Remove(TEDSColumnToRemove);
}

void USceneOutlinerTedsBridgeFactory::AddTedsColumnMappingToOutlinerColumn(TWeakObjectPtr<const UScriptStruct> TEDSColumnToAdd, const FName& OutlinerColumnId)
{
	TEDSToOutlinerDefaultColumnMapping.Add(TEDSColumnToAdd, OutlinerColumnId);
}

#undef LOCTEXT_NAMESPACE
