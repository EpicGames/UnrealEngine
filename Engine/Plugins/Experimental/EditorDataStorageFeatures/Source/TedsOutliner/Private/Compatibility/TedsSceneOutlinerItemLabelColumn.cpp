// Copyright Epic Games, Inc. All Rights Reserved.

#include "Compatibility/TedsSceneOutlinerItemLabelColumn.h"

#include "ISceneOutlinerMode.h"
#include "SceneOutlinerHelpers.h"

const TSharedRef<SWidget> FTedsSceneOutlinerItemLabelColumn::ConstructRowWidget(FSceneOutlinerTreeItemRef TreeItem, const STableRow<FSceneOutlinerTreeItemPtr>& Row)
{
	ISceneOutliner* Outliner = OwningOutliner.Pin().Get();
	check(Outliner);
	return TreeItem->GenerateLabelWidget(*Outliner, Row);
}

void FTedsSceneOutlinerItemLabelColumn::PopulateSearchStrings( const ISceneOutlinerTreeItem& Item, TArray< FString >& OutSearchStrings ) const
{
	OutSearchStrings.Add(Item.GetDisplayString());
	SceneOutliner::FSceneOutlinerHelpers::PopulateExtraSearchStrings(Item, OutSearchStrings);
}