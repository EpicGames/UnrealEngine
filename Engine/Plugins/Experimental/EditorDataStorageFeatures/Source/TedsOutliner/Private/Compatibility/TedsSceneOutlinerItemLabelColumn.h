// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SceneOutlinerFwd.h"
#include "Widgets/Views/SHeaderRow.h"
#include "Widgets/Views/STableRow.h"
#include "TedsOutlinerColumn.h"

/**
 * This is an override column for the TEDS Outliner/Table Viewer to display item labels for items. The label column requires that we populate 
 * additional search strings for components and construct the Label Column widget via the Tree Item so it can get the appropriate metadata
 * (still constructs a TEDS UI Widget)
 */
class FTedsSceneOutlinerItemLabelColumn : public UE::Editor::Outliner::FTedsOutlinerUiColumn
{
public:
	FTedsSceneOutlinerItemLabelColumn(UE::Editor::Outliner::FTedsOutlinerUiColumnInitParams& InitParams) : FTedsOutlinerUiColumn(InitParams) {}
	
	virtual ~FTedsSceneOutlinerItemLabelColumn() = default;
	
	virtual const TSharedRef<SWidget> ConstructRowWidget( FSceneOutlinerTreeItemRef TreeItem, const STableRow<FSceneOutlinerTreeItemPtr>& Row ) override;

	virtual void PopulateSearchStrings( const ISceneOutlinerTreeItem& Item, TArray< FString >& OutSearchStrings ) const override;
};
