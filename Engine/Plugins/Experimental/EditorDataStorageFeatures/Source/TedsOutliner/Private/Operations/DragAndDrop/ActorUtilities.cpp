// Copyright Epic Games, Inc. All Rights Reserved.

#include "Operations/DragAndDrop/ActorUtilities.h"

#include "Operations/DragAndDrop/AssetUtilities.h"
#include "DragAndDrop/DropOperationInput.h"
#include "Elements/Columns/TypedElementCompatibilityColumns.h"
#include "Elements/Interfaces/TypedElementDataStorageInterface.h"
#include "Engine/Level.h"
#include "Engine/World.h"
#include "GameFramework/Actor.h"

#define LOCTEXT_NAMESPACE "DragAndDrop_ActorUtilities"

namespace UE::Editor::DataStorage::Operations::Utilities
{

namespace ActorUtilities_Private
{
	static bool GetTargetActorOrLevel(FActorLevelPair& OutResult, TNotNull<UObject*> TargetObject, FText* OutError)
	{
		OutResult.Actor = Cast<AActor>(TargetObject);
		if (OutResult.Actor)
		{
			OutResult.Level = OutResult.Actor->GetLevel();
		}
		else
		{
			OutResult.Level = Cast<ULevel>(TargetObject);
		}
	
		if (!OutResult.Level && OutError)
		{
			*OutError = LOCTEXT("TargetInvalid", "Actor cannot be placed on this target.");
		}
		return OutResult.Level != nullptr;
	}
}
	
AActor* GetActorFromRow(const ICoreProvider& Storage, RowHandle Row)
{
	if (!Storage.HasColumns<FTypedElementActorTag>(Row))
	{
		return nullptr;
	}

	const FTypedElementUObjectColumn* Column = Storage.GetColumn<FTypedElementUObjectColumn>(Row);
	if (!Column)
	{
		return nullptr;
	}

	constexpr bool bEvenIfPendingKill = true;
	UObject* Object = Column->Object.Get(bEvenIfPendingKill);
	return CastChecked<AActor>(Object, ECastCheckedType::NullAllowed);
}

ULevel* GetTargetLevel(const ICoreProvider& Storage, RowHandle TargetRow)
{
	if (TargetRow == InvalidRowHandle)
	{
		return nullptr;
	}

	if (const FTypedElementLevelColumn* LevelColumn = Storage.GetColumn<FTypedElementLevelColumn>(TargetRow))
	{
		if (ULevel* Level = LevelColumn->Level.Get())
		{
			return Level;
		}
	}

	if (Storage.HasColumns<FLevelTag>(TargetRow))
	{
		if (const FTypedElementUObjectColumn* ObjectColumn = Storage.GetColumn<FTypedElementUObjectColumn>(TargetRow))
		{
			if (ULevel* Level = Cast<ULevel>(ObjectColumn->Object))
			{
				return Level;
			}
		}
	}

	if (Storage.HasColumns<FWorldTag>(TargetRow))
	{
		if (const FTypedElementWorldColumn* WorldColumn = Storage.GetColumn<FTypedElementWorldColumn>(TargetRow))
		{
			if (UWorld* World = WorldColumn->World.Get())
			{
				return World->PersistentLevel;
			}
		}
	}

	return nullptr;
}

bool GetTargetActorOrLevel(FActorLevelPair& OutResult, const ICoreProvider& Storage, RowHandle InputRow, FText* OutError)
{
	UObject* TargetObject = GetTargetObject(Storage, InputRow, OutError);
	return TargetObject && ActorUtilities_Private::GetTargetActorOrLevel(OutResult, TargetObject, OutError);
}

bool GetTargetActorOrLevelWithAssetValidation(FActorLevelPair& OutResult, const ICoreProvider& Storage, RowHandle InputRow,
	const FAssetData& AssetData, FText* OutError)
{
	UObject* TargetObject = GetTargetObjectWithAssetValidation(Storage, InputRow, AssetData, OutError);
	return TargetObject && ActorUtilities_Private::GetTargetActorOrLevel(OutResult, TargetObject, OutError);
}
	
}

#undef LOCTEXT_NAMESPACE
