// Copyright Epic Games, Inc. All Rights Reserved.

#include "Operations/DragAndDrop/RegisterDropOperations.h"

#include "ActorFolders/ActorFolderColumns.h"
#include "DragAndDrop/DropOperationInput.h"
#include "DragAndDrop/DropOperationSystem.h"
#include "Editor.h"
#include "Elements/Columns/TypedElementCompatibilityColumns.h"
#include "GameFramework/Actor.h"
#include "Operations/DragAndDrop/ActorUtilities.h"
#include "TedsOperationInput.h"

#define LOCTEXT_NAMESPACE "DragAndDrop_ReparentActor"

namespace UE::Editor::DataStorage::Operations
{

namespace ReparentActor_Private
{
static bool Probe(const ICoreProvider& Storage, RowHandle InputRow)
{
	// Just test that the source is an actor.
	RowHandle SourceRow = Utilities::GetSourceRow(Storage, InputRow);
	return Storage.HasColumns<FTypedElementActorTag>(SourceRow);
}

static TOptional<FResult> ReparentActorToFolder(RowHandle SourceRow, AActor* SourceActor,
	const FFolder& TargetFolder, FText* OutDescription, bool bApply)
{
	// Cannot move across levels (different root object).
	const FFolder SourceFolder = SourceActor->GetFolder();
	if (SourceFolder.GetRootObject() != TargetFolder.GetRootObject())
	{
		if (OutDescription)
		{
			*OutDescription = LOCTEXT("DifferentRoot", "Cannot move actor to a folder in a different level.");
		}
		return {};
	}

	// Already in this folder: either no-op, or detach if it still has an outliner parent.
	if (SourceFolder == TargetFolder)
	{
		if (SourceActor->GetSceneOutlinerParent() != nullptr)
		{
			if (!bApply)
			{
				if (OutDescription)
				{
					*OutDescription = (TargetFolder.GetPath() == NAME_None)
						? FText::Format(LOCTEXT("DescriptionMoveToWorldRoot", "Move '{0}' to world root."),
							FText::FromString(SourceActor->GetActorLabel()))
						: FText::Format(LOCTEXT("DescriptionMoveToFolder", "Move '{0}' to '{1}'."),
							FText::FromString(SourceActor->GetActorLabel()),
							FText::FromName(TargetFolder.GetLeafName()));
				}
				return FResult();
			}

			SourceActor->DetachFromActor(FDetachmentTransformRules::KeepWorldTransform);

			FResult Result;
			Result.Changed.Add(SourceRow);
			return Result;
		}

		if (OutDescription)
		{
			*OutDescription = (TargetFolder.GetPath() == NAME_None)
				? LOCTEXT("AlreadyAtWorldRoot", "Actor is already at world root.")
				: LOCTEXT("AlreadyInFolder", "Actor is already in this folder.");
		}
		return {};
	}

	if (!bApply)
	{
		if (OutDescription)
		{
			*OutDescription = (TargetFolder.GetPath() == NAME_None)
				? FText::Format(LOCTEXT("DescriptionMoveToWorldRoot", "Move '{0}' to world root."),
					FText::FromString(SourceActor->GetActorLabel()))
				: FText::Format(LOCTEXT("DescriptionMoveToFolder", "Move '{0}' to '{1}'."),
					FText::FromString(SourceActor->GetActorLabel()),
					FText::FromName(TargetFolder.GetLeafName()));
		}
		return FResult();
	}

	// Detach from parent actor if needed (child actor dragged onto a folder).
	if (SourceActor->GetSceneOutlinerParent() != nullptr)
	{
		SourceActor->DetachFromActor(FDetachmentTransformRules::KeepWorldTransform);
	}

	SourceActor->SetFolderPath(TargetFolder.GetPath());

	FResult Result;
	Result.Changed.Add(SourceRow);
	return Result;
}

static TOptional<FResult> ReparentActor(ICoreProvider& Storage, RowHandle InputRow, bool bApply)
{
	FText* OutDescription = Utilities::GetDescriptionPtr(Storage, InputRow);

	RowHandle SourceRow = Utilities::GetSourceRow(Storage, InputRow);
	AActor* SourceActor = Utilities::GetActorFromRow(Storage, SourceRow);
	if (!SourceActor)
	{
		if (OutDescription)
		{
			*OutDescription = LOCTEXT("SourceInvalid", "Source is not a valid actor.");
		}
		return {};
	}

	const RowHandle TargetRow = Utilities::GetDropTargetRow(Storage, InputRow);

	if (ULevel* TargetLevel = Utilities::GetTargetLevel(Storage, TargetRow))
	{
		if (TargetLevel != SourceActor->GetLevel())
		{
			if (OutDescription)
			{
				*OutDescription = LOCTEXT("SwitchingLevelsUnsupported", "Switching levels is unsupported.");
			}
			return {};
		}
	}

	// Folder target takes precedence: if the drop target exposes a folder, treat as a folder reparent.
	if (const FFolderCompatibilityColumn* FolderColumn = Storage.GetColumn<FFolderCompatibilityColumn>(TargetRow))
	{
		return ReparentActorToFolder(SourceRow, SourceActor, FolderColumn->Folder, OutDescription, bApply);
	}

	// World row: treat as a drop onto the world root folder (clears folder path, detaches any attach parent).
	if (Storage.HasColumns<FWorldTag>(TargetRow))
	{
		const FTypedElementWorldColumn* WorldColumn = Storage.GetColumn<FTypedElementWorldColumn>(TargetRow);
		UWorld* TargetWorld = WorldColumn ? WorldColumn->World.Get() : nullptr;
		if (!TargetWorld)
		{
			if (OutDescription)
			{
				*OutDescription = LOCTEXT("TargetInvalid", "Actor cannot be placed on this target.");
			}
			return {};
		}
		return ReparentActorToFolder(SourceRow, SourceActor, FFolder::GetWorldRootFolder(TargetWorld), OutDescription, bApply);
	}

	Utilities::FActorLevelPair Target;
	if (!Utilities::GetTargetActorOrLevel(Target, Storage, InputRow, OutDescription))
	{
		return {};
	}

	// Only attach-parent reparenting (Target.Actor is another actor) happens here.
	AActor* CurrentParent = SourceActor->GetAttachParentActor();
	const bool bWouldReparent = (Target.Actor != nullptr) && (Target.Actor != CurrentParent);
	if (bWouldReparent && !GEditor->CanParentActors(Target.Actor, SourceActor, OutDescription))
	{
		return {};
	}

	// Dropping on the actor's current attach parent is a no-op; reject so the outliner reports it
	// rather than silently succeeding with a misleading `Move 'X'.` tooltip.
	if (Target.Actor != nullptr && !bWouldReparent)
	{
		if (OutDescription)
		{
			*OutDescription = FText::Format(LOCTEXT("AlreadyAttached", "Actor is already attached to '{0}'."),
				FText::FromString(Target.Actor->GetActorLabel()));
		}
		return {};
	}

	if (!bApply)
	{
		if (OutDescription)
		{
			*OutDescription = bWouldReparent
				? FText::Format(LOCTEXT("DescriptionReparent", "Reparent '{0}' to '{1}'."),
					FText::FromString(SourceActor->GetActorLabel()),
					FText::FromString(Target.Actor->GetActorLabel()))
				: FText::Format(LOCTEXT("DescriptionMove", "Move '{0}'."),
					FText::FromString(SourceActor->GetActorLabel()));
		}
		return FResult();
	}

	if (bWouldReparent)
	{
		if (Target.Actor)
		{
			GEditor->ParentActors(Target.Actor, SourceActor, NAME_None);
		}
		else
		{
			SourceActor->DetachFromActor(FDetachmentTransformRules::KeepWorldTransform);
		}
	}
	
	if (const FDropTransformColumn* Column = Storage.GetColumn<FDropTransformColumn>(InputRow))
	{
		SourceActor->SetActorTransform(Column->Value);
	}

	FResult Result;
	Result.Changed.Add(SourceRow);
	return Result;
}

static bool Test(ICoreProvider& Storage, RowHandle InputRow)
{
	return ReparentActor(Storage, InputRow, false).IsSet();
}

static TOptional<FResult> Apply(ICoreProvider& Storage, RowHandle InputRow)
{
	return ReparentActor(Storage, InputRow, true);
}
}
		
void RegisterReparentActor(ICoreProvider& Storage)
{
	if (UDropOperationSystem* DropOperations = Storage.FindFactory<UDropOperationSystem>())
	{
		DropOperations->AddOperation(
			FName("ReparentActor"),
			ReparentActor_Private::Apply,
			ReparentActor_Private::Test,
			ReparentActor_Private::Probe
		);
	}	
}
    
}

#undef LOCTEXT_NAMESPACE
