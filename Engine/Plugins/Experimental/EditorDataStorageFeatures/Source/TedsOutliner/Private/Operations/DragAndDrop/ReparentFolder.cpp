// Copyright Epic Games, Inc. All Rights Reserved.

#include "Operations/DragAndDrop/RegisterDropOperations.h"

#include "ActorFolders/ActorFolderColumns.h"
#include "DragAndDrop/DropOperationInput.h"
#include "DragAndDrop/DropOperationSystem.h"
#include "EditorActorFolders.h"
#include "Elements/Columns/TypedElementCompatibilityColumns.h"
#include "Elements/Columns/TypedElementFolderColumns.h"
#include "Engine/Level.h"
#include "Engine/World.h"
#include "Operations/DragAndDrop/ActorUtilities.h"
#include "TedsOperationInput.h"
#include "Elements/Columns/TypedElementTypeInfoColumns.h"

#define LOCTEXT_NAMESPACE "DragAndDrop_ReparentFolder"

namespace UE::Editor::DataStorage::Operations
{

namespace ReparentFolder_Private
{

static bool Probe(const ICoreProvider& Storage, RowHandle InputRow)
{
	// Just test that the source is a folder.
	RowHandle SourceRow = Utilities::GetSourceRow(Storage, InputRow);
	return Storage.HasColumns<FFolderTag>(SourceRow);
}

static TOptional<FResult> ReparentFolder(ICoreProvider& Storage, RowHandle InputRow, bool bApply)
{
	FText* OutDescription = Utilities::GetDescriptionPtr(Storage, InputRow);

	RowHandle SourceRow = Utilities::GetSourceRow(Storage, InputRow);
	const FFolderCompatibilityColumn* SourceFolderColumn = Storage.GetColumn<FFolderCompatibilityColumn>(SourceRow);
	if (!SourceFolderColumn)
	{
		if (OutDescription)
		{
			*OutDescription = LOCTEXT("SourceInvalid", "Source is not a valid folder.");
		}
		return {};
	}

	FFolder SourceFolder = SourceFolderColumn->Folder;
	ULevel* SourceLevel = SourceFolder.GetRootObjectAssociatedLevel();
	UWorld* World = SourceLevel ? SourceLevel->GetWorld() : nullptr;
	if (!World)
	{
		if (OutDescription)
		{
			*OutDescription = LOCTEXT("NoWorld", "Could not determine world for folder.");
		}
		return {};
	}

	RowHandle TargetRow = Utilities::GetDropTargetRow(Storage, InputRow);

	FFolder TargetFolder;
	if (const FFolderCompatibilityColumn* TargetFolderColumn = Storage.GetColumn<FFolderCompatibilityColumn>(TargetRow))
	{
		TargetFolder = TargetFolderColumn->Folder;
	}
	// World Root Folder
	else if (Storage.HasColumns<FWorldTag>(TargetRow))
	{
		if (const FTypedElementWorldColumn* WorldColumn = Storage.GetColumn<FTypedElementWorldColumn>(TargetRow))
		{
			if (UWorld* TargetWorld = WorldColumn->World.Get())
			{
				TargetFolder = FFolder::GetWorldRootFolder(TargetWorld);
			}
		}
	}

	if (!TargetFolder.IsValid())
	{
		if (OutDescription)
		{
			*OutDescription = LOCTEXT("TargetUnsupported", "Folders can only be parented to other folders or the world root.");
		}
		return {};
	}

	ULevel* TargetLevel = Utilities::GetTargetLevel(Storage, TargetRow);
	if (!TargetLevel)
	{
		TargetLevel = TargetFolder.GetRootObjectAssociatedLevel();
	}
	ULevel* ResolvedSourceLevel = nullptr;
	if (const FTypedElementLevelColumn* SourceLevelColumn = Storage.GetColumn<FTypedElementLevelColumn>(SourceRow))
	{
		ResolvedSourceLevel = SourceLevelColumn->Level.Get();
	}
	if (!ResolvedSourceLevel)
	{
		ResolvedSourceLevel = SourceLevel;
	}
	if (ResolvedSourceLevel && TargetLevel && ResolvedSourceLevel != TargetLevel)
	{
		if (OutDescription)
		{
			*OutDescription = LOCTEXT("DifferentLevel", "Cannot move a folder to a different level.");
		}
		return {};
	}

	// Validate same root object (cannot move across levels or world partitions)
	if (SourceFolder.GetRootObject() != TargetFolder.GetRootObject())
	{
		if (OutDescription)
		{
			*OutDescription = LOCTEXT("DifferentRoot", "Cannot move a folder to a different root.");
		}
		return {};
	}

	// Validate not dropping a folder into itself or one of its descendants
	if (TargetFolder == SourceFolder)
	{
		if (OutDescription)
		{
			*OutDescription = LOCTEXT("Itself", "Cannot move a folder into itself.");
		}
		return {};
	}
	if (TargetFolder.IsChildOf(SourceFolder))
	{
		if (OutDescription)
		{
			*OutDescription = LOCTEXT("CyclicDrop", "Cannot move a folder into one of its children.");
		}
		return {};
	}

	// Folder is already a direct child of the target
	if (SourceFolder.GetParent() == TargetFolder)
	{
		if (OutDescription)
		{
			*OutDescription = LOCTEXT("AlreadyParented", "Folder is already at this location.");
		}
		return {};
	}

	if (!bApply)
	{
		if (OutDescription)
		{
			if (TargetFolder.GetPath() == NAME_None)
			{
				*OutDescription = FText::Format(
					LOCTEXT("MoveToWorldRoot", "Move '{0}' to world root."),
					FText::FromName(SourceFolder.GetLeafName()));
			}
			else
			{
				*OutDescription = FText::Format(
					LOCTEXT("Description", "Move '{0}' to '{1}'."),
					FText::FromName(SourceFolder.GetLeafName()),
					FText::FromName(TargetFolder.GetLeafName()));
			}
		}
		return FResult(); // Successful test call.
	}

	const FFolder NewPath = FActorFolders::Get().GetFolderName(*World, TargetFolder, SourceFolder.GetLeafName());
	FActorFolders::Get().RenameFolderInWorld(*World, SourceFolder, NewPath);

	FResult Result;
	Result.Changed.Add(SourceRow);
	return Result;
}

static bool Test(ICoreProvider& Storage, RowHandle InputRow)
{
	return ReparentFolder(Storage, InputRow, false).IsSet();
}

static TOptional<FResult> Apply(ICoreProvider& Storage, RowHandle InputRow)
{
	return ReparentFolder(Storage, InputRow, true);
}

} // namespace ReparentFolder_Private

void RegisterReparentFolder(ICoreProvider& Storage)
{
	if (UDropOperationSystem* DropOperations = Storage.FindFactory<UDropOperationSystem>())
	{
		DropOperations->AddOperation(
			FName("ReparentFolder"),
			ReparentFolder_Private::Apply,
			ReparentFolder_Private::Test,
			ReparentFolder_Private::Probe
		);
	}
}

} // namespace UE::Editor::DataStorage::Operations

#undef LOCTEXT_NAMESPACE
