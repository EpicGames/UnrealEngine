// Copyright Epic Games, Inc. All Rights Reserved.

#include "RowMapNode.h"

namespace UE::Editor::Outliner
{
	FRowMapNode::FRowMapNode(TSharedPtr<IRowNode> InParent)
		: Parent(MoveTemp(InParent))
		, CachedParentRevision(Parent->GetRevision())
	{
		RefreshMap();
	}

	DataStorage::FRowHandleArrayView FRowMapNode::GetRows() const
	{
		return Parent->GetRows();
	}

	DataStorage::FRowHandleArray& FRowMapNode::GetMutableRows()
	{
		return Parent->GetMutableRows();
	}

	DataStorage::QueryStack::INode::RevisionId FRowMapNode::GetRevision() const
	{
		return Revision;
	}

	void FRowMapNode::Update(FTimespan AllottedTime)
	{
		if (Parent->GetRevision() != CachedParentRevision)
		{
			RefreshMap();
			CachedParentRevision = Parent->GetRevision();
			Revision++;
		}
	}

	void FRowMapNode::VisitParents(ParentListCallback Callback)
	{
		Callback(Parent);
	}

	int32 FRowMapNode::GetRowIndex(DataStorage::RowHandle Row) const
	{
		if (const int32* Index = RowMap.Find(Row))
		{
			return *Index;
		}

		return INDEX_NONE;
	}

	void FRowMapNode::RefreshMap()
	{
		RowMap.Empty();

		DataStorage::FRowHandleArrayView ParentRows = Parent->GetRows();
		for (int32 Index = 0; Index < ParentRows.Num(); ++Index)
		{
			RowMap.Emplace(ParentRows[Index], Index);
		}
	}
}
