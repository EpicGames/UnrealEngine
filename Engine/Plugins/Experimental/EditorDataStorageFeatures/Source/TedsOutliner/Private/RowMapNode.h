// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "TedsQueryStackInterfaces.h"

namespace UE::Editor::Outliner
{
	/*
	 * A specialized node for the Teds Outliner that stores a map from row handle to index in the parent node.
	 * This is currently required to integrate sorting with the Teds Outliner without a major refactor of the base Outliner code.
	 */
	class FRowMapNode : public DataStorage::QueryStack::IRowNode
	{
	public:
		FRowMapNode(TSharedPtr<IRowNode> InParent);
		
		virtual ~FRowMapNode() override = default;
		virtual DataStorage::FRowHandleArrayView GetRows() const override;
		virtual DataStorage::FRowHandleArray& GetMutableRows() override;
		virtual RevisionId GetRevision() const override;
		virtual void VisitParents(ParentListCallback Callback) override;

		// Look up the index of the given row in the parent node. INDEX_NONE if not found
		int32 GetRowIndex(DataStorage::RowHandle Row) const;
		
	protected:
		virtual void Update(FTimespan AllottedTime) override;
		void RefreshMap();
		
	protected:

		// Map of row handle -> index in the parent array
		TMap<DataStorage::RowHandle, int32> RowMap;
		TSharedPtr<IRowNode> Parent;
		RevisionId CachedParentRevision;
		RevisionId Revision = 0;
	};
}