// Copyright Epic Games, Inc. All Rights Reserved.

#include "Sorting/NaturalSortKey.h"

#include "Containers/StringView.h"
#include "Math/UnrealMathUtility.h"
#include "Misc/Char.h"

namespace UE::Editor::DataStorage
{
	namespace Private
	{
		// A value lower than any printable character so letters/symbols following a digit run still sort after the sentinel,
		// and leading-zero counts written as trailing bytes can never collide with following characters.
		constexpr TCHAR NaturalSortDigitSentinel = static_cast<TCHAR>(0x01);

		// Width of the digit-count fields in TCHARs.
		constexpr int32 DigitLengthBytes = 2;
		constexpr int32 DigitLengthMax = 0xFFFF;

		// Per-digit-run encoding overhead: sentinel (1) + SignificantLen field (DigitLengthBytes) + LeadingZeros field (DigitLengthBytes).
		constexpr int32 DigitRunEncodeOverhead = 1 + DigitLengthBytes * 2;
	}

	FString BuildNaturalSortKey(FStringView Label)
	{
		using namespace Private;

		FString Key;
		Key.Reserve(Label.Len() + DigitRunEncodeOverhead);

		int32 Index = 0;
		while (Index < Label.Len())
		{
			const TCHAR Char = Label[Index];

			if (Char == TEXT('_'))
			{
				++Index;
				continue;
			}

			if (FChar::IsDigit(Char))
			{
				const int32 RunStart = Index;
				while (Index < Label.Len() && FChar::IsDigit(Label[Index]))
				{
					++Index;
				}

				// Strip leading zeros so the encoded length reflects numeric magnitude, not raw digit count. 
				// Keep at least one digit so an all-zero run like "000" still encodes as "0" with two leading zeros counted separately.
				int32 SignificantStart = RunStart;
				while (SignificantStart < Index - 1 && Label[SignificantStart] == TEXT('0'))
				{
					++SignificantStart;
				}

				const int32 SignificantLen = FMath::Min(Index - SignificantStart, DigitLengthMax);
				const int32 LeadingZeros = FMath::Min(SignificantStart - RunStart, DigitLengthMax);

				// NOTE: must use AppendChars (not AppendChar). 
				// FString::AppendChar silently drops zero values (String.cpp.inl: `if (InChar != 0)`), which would corrupt
				// the encoding because the hi-bytes of SignificantLen/LeadingZeros are almost always zero,
				// and LeadingZeros lo-byte is zero whenever there are no leading zeros. 
				// AppendChars copies bytes unconditionally.
				const TCHAR Header[3] =
				{
					NaturalSortDigitSentinel,
					static_cast<TCHAR>((SignificantLen >> 8) & 0xFF),
					static_cast<TCHAR>(SignificantLen & 0xFF),
				};
				Key.AppendChars(Header, UE_ARRAY_COUNT(Header));
				Key.AppendChars(Label.GetData() + SignificantStart, Index - SignificantStart);
				// Leading-zero count as a tiebreaker: when two numbers are numerically
				// equal ("2" and "02"), the one with fewer leading zeros sorts first.
				const TCHAR Trailer[2] =
				{
					static_cast<TCHAR>((LeadingZeros >> 8) & 0xFF),
					static_cast<TCHAR>(LeadingZeros & 0xFF),
				};
				Key.AppendChars(Trailer, UE_ARRAY_COUNT(Trailer));
				continue;
			}

			Key.AppendChar(FChar::ToLower(Char));
			++Index;
		}

		// Case-sensitive suffix: appending the raw label makes byte-wise compare deterministic for labels that case-fold to the same natural-order key. 
		// Without this, Algo::StableSort preserves whatever input order tied rows arrived in, which looks arbitrary to users.
		//
		// Separator is 0x00. It must sort below every byte that can appear in a natural-order continuation,
		// most notably the 0x01 digit-run sentinel, 
		// so that a label like "Actor" still sorts before "Actor2" 
		// (the shorter-natural side's separator beats the longer's digit-run sentinel at the same byte position). 
		// 0x00 is safe despite being a C-string terminator because the key is only ever consumed by CompareNaturalSortKeys (length-aware)
		// and by TSortStringView in the prefix path (length-aware via View.Len()); 
		// FString's own strcmp-based comparators must not be used on keys produced here.
		const TCHAR Separator = static_cast<TCHAR>(0x00);
		Key.AppendChars(&Separator, 1);
		Key.AppendChars(Label.GetData(), Label.Len());

		return Key;
	}

	int32 CompareNaturalSortKeys(FStringView A, FStringView B)
	{
		const int32 MinLen = FMath::Min(A.Len(), B.Len());
		for (int32 Index = 0; Index < MinLen; ++Index)
		{
			if (A[Index] != B[Index])
			{
				return static_cast<int32>(A[Index]) - static_cast<int32>(B[Index]);
			}
		}
		return A.Len() - B.Len();
	}
}
