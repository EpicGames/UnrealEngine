// Copyright Epic Games, Inc. All Rights Reserved.

#include "TedsOutlinerColumn.h"

#include "ActorTreeItem.h"
#include "Modules/ModuleManager.h"
#include "SceneOutlinerModule.h"
#include "TedsOutlinerHelpers.h"
#include "TedsOutlinerImpl.h"
#include "TedsOutlinerItem.h"
#include "TedsTableViewerUtils.h"
#include "TedsTableViewerWidgetColumns.h"
#include "Elements/Columns/TypedElementSlateWidgetColumns.h"
#include "Columns/SceneOutlinerColumns.h"

namespace UE::Editor::Outliner
{
	FTedsOutlinerUiColumnInitParams::FTedsOutlinerUiColumnInitParams(const DataStorage::FMetaData& InMetaData, DataStorage::ICoreProvider& InStorage,
		DataStorage::IUiProvider& InStorageUi, DataStorage::ICompatibilityProvider& InStorageCompatibility)
		: MetaData(InMetaData)
		, Storage(InStorage)
		, StorageUi(InStorageUi)
		, StorageCompatibility(InStorageCompatibility)
	{
	}

	FTedsOutlinerUiColumn::FTedsOutlinerUiColumn(FTedsOutlinerUiColumnInitParams& InitParams)
		: Storage(InitParams.Storage)
		, StorageUi(InitParams.StorageUi)
		, StorageCompatibility(InitParams.StorageCompatibility)
		, MetaData(InitParams.MetaData)
		, NameId(InitParams.NameId)
		, FallbackColumn(InitParams.FallbackColumn)
		, OwningOutliner(InitParams.OwningOutliner)
		, TedsOutlinerImpl(InitParams.TedsOutlinerImpl)
		, Dealiaser(InitParams.Dealiaser)
		, bHybridMode(InitParams.bHybridMode)
	{
		MetaData.AddOrSetMutableData(TEXT("Name"), NameId.ToString());

		using namespace UE::Editor::DataStorage;

		TableViewerColumnImpl = MakeUnique<FTedsTableViewerColumn>(NameId, InitParams.CellWidgetConstructor, InitParams.ColumnTypes,
			InitParams.HeaderWidgetConstructor, MetaData);

		TableViewerColumnImpl->SetIsRowVisibleDelegate(
			FTedsTableViewerColumn::FIsRowVisible::CreateRaw(this, &FTedsOutlinerUiColumn::IsRowVisible)
		);

		// TEDS sorting behaves unexpectedly in hybrid mode, so we only allow fallback sorting there
		if (!bHybridMode)
		{
			TableViewerColumnImpl->SetSortDelegates(InitParams.SortDelegates);
		}
	};
		
	FName FTedsOutlinerUiColumn::GetColumnID()
	{
		return NameId;
	}

	 void FTedsOutlinerUiColumn::Tick(double InCurrentTime, float InDeltaTime)
	{
		TableViewerColumnImpl->Tick();
		if (FallbackColumn)
		{
			FallbackColumn->Tick(InCurrentTime, InDeltaTime);
		}
	}

	bool FTedsOutlinerUiColumn::IsRowVisible(const UE::Editor::DataStorage::RowHandle InRowHandle) const
	{
		TSharedPtr<ISceneOutliner> OutlinerPinned = OwningOutliner.Pin();

		if (!OutlinerPinned)
		{
			return false;
		}
		
		// Try to grab the TEDS Outliner item from the row handle
		FSceneOutlinerTreeItemPtr Item = Helpers::GetTreeItemFromRowHandle(&Storage, OutlinerPinned.ToSharedRef(), InRowHandle);

		// Check if the item is visible in the tree
		return OutlinerPinned->GetTree().IsItemVisible(Item);
	}

	SHeaderRow::FColumn::FArguments FTedsOutlinerUiColumn::ConstructHeaderRowColumn()
	{
		return TableViewerColumnImpl->ConstructHeaderRowColumn();
	}

	// TODO: Sorting is currently handled through the fallback column if it exists because we have no way to sort columns through TEDS
	void FTedsOutlinerUiColumn::SortItems(TArray<FSceneOutlinerTreeItemPtr>& RootItems, const EColumnSortMode::Type SortMode) const
	{
		if (TSharedPtr<FTedsOutlinerImpl> TedsOutlinerImplPin = TedsOutlinerImpl.Pin(); TedsOutlinerImplPin && !bHybridMode)
		{
			TedsOutlinerImplPin->SortItems(RootItems, SortMode);
		}
		else if (FallbackColumn)
		{
			FallbackColumn->SortItems(RootItems, SortMode);
		}
	}

	bool FTedsOutlinerUiColumn::SupportsSorting() const 
	{
		if (!bHybridMode && TableViewerColumnImpl->HasSortInfo())
		{
			return true;
		}
		
		return FallbackColumn ? FallbackColumn->SupportsSorting() : false;
	}

	void FTedsOutlinerUiColumn::OnSortRequested(const EColumnSortPriority::Type SortPriority, const EColumnSortMode::Type InSortMode)
	{
		if (!bHybridMode)
		{
			TableViewerColumnImpl->OnSortCallback(SortPriority, TableViewerColumnImpl->GetColumnName(), InSortMode);
		}
	}

	bool FTedsOutlinerUiColumn::IsSortReady()
	{
		return !bHybridMode && TedsOutlinerImpl.IsValid() ? !TedsOutlinerImpl.Pin()->IsSorting() : true;
	}
	
	const TSharedRef<SWidget> FTedsOutlinerUiColumn::ConstructRowWidget(FSceneOutlinerTreeItemRef TreeItem, const STableRow<FSceneOutlinerTreeItemPtr>& Row)
	{
		using namespace UE::Editor::DataStorage;
		
		RowHandle TargetRowHandle = InvalidRowHandle;

		TSharedPtr<SWidget> RowWidget;

		if (const FTedsOutlinerTreeItem* TedsItem = TreeItem->CastTo<FTedsOutlinerTreeItem>())
		{
			TargetRowHandle = TedsItem->GetRowHandle();
			
		}
		else if (const FActorTreeItem* ActorItem = TreeItem->CastTo<FActorTreeItem>())
		{
			if (AActor* Actor = ActorItem->Actor.Get())
			{
				TargetRowHandle = StorageCompatibility.FindRowWithCompatibleObject(Actor);
			}
		}
		else if (FallbackColumn)
		{
			RowWidget = FallbackColumn->ConstructRowWidget(TreeItem, Row);
		}

		if(Storage.IsRowAssigned(TargetRowHandle))
		{
			RowWidget = TableViewerColumnImpl->ConstructRowWidget(TargetRowHandle,
				[this](ICoreProvider& DataStorage, const RowHandle& WidgetRow)
				{
					DataStorage.AddColumn(WidgetRow, FSceneOutlinerColumn{ .Outliner = OwningOutliner });

					// Parent the cell widget row to the outliner's table viewer row so helpers walking the widget hierarchy can locate FTableViewerTag.
					if (TSharedPtr<FTedsOutlinerImpl> TedsOutlinerImplPin = TedsOutlinerImpl.Pin())
					{
						const FHierarchyHandle WidgetHierarchy = DataStorage.FindHierarchyByName(TableViewerUtils::GetWidgetHierarchyName());
						DataStorage.SetParentRow(WidgetHierarchy, WidgetRow, TedsOutlinerImplPin->GetOutlinerRowHandle());
					}
				});
		}

		if (RowWidget)
		{
			return RowWidget.ToSharedRef();
		}

		return SNullWidget::NullWidget;
	}

	void FTedsOutlinerUiColumn::PopulateSearchStrings( const ISceneOutlinerTreeItem& Item, TArray< FString >& OutSearchStrings ) const
	{
		using namespace UE::Editor::DataStorage;
		
		if (!bHybridMode)
		{
			if(const FTedsOutlinerTreeItem* OutlinerTreeItem = Item.CastTo<FTedsOutlinerTreeItem>())
			{
				const RowHandle RowHandle = OutlinerTreeItem->GetRowHandle();
				TableViewerColumnImpl->OnGetSearchStringCallback(RowHandle, OutSearchStrings);
			}
		}
		else if (FallbackColumn)
		{
			FallbackColumn->PopulateSearchStrings(Item, OutSearchStrings);
		}
	}
}
