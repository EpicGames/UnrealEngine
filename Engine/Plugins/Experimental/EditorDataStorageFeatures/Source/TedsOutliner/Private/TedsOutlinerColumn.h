// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ISceneOutlinerColumn.h"
#include "TedsTableViewerColumn.h"
#include "Columns/TedsOutlinerColumns.h"
#include "DataStorage/Handles.h"
#include "Elements/Interfaces/TypedElementDataStorageUiInterface.h"

namespace UE::Editor::Outliner
{
	class FTedsOutlinerImpl;

	struct FTedsOutlinerUiColumnInitParams
	{
		FTedsOutlinerUiColumnInitParams(const DataStorage::FMetaData& InMetaData, DataStorage::ICoreProvider& InStorage,
			DataStorage::IUiProvider& InStorageUi, DataStorage::ICompatibilityProvider& InStorageCompatibility);
		
		// The metadata used for this column
		DataStorage::FMetaData MetaData;

		// Data storage interfaces
		DataStorage::ICoreProvider& Storage;
		DataStorage::IUiProvider& StorageUi;
		DataStorage::ICompatibilityProvider& StorageCompatibility;

		// Unique ID
		FName NameId;

		// TEDS UI info
		TArray<TWeakObjectPtr<const UScriptStruct>> ColumnTypes;
		DataStorage::IUiProvider::FWidgetConstructorPtr HeaderWidgetConstructor;
		DataStorage::IUiProvider::FWidgetConstructorPtr CellWidgetConstructor;

		// Fallbacks to work with legacy columns
		TSharedPtr<ISceneOutlinerColumn> FallbackColumn;
		FTreeItemIDDealiaser Dealiaser;
		bool bHybridMode = false;

		// Sorting info
		DataStorage::FTedsTableViewerColumn::FSortHandler SortDelegates;

		// Outliner refernces
		TWeakPtr<ISceneOutliner> OwningOutliner;
		TWeakPtr<FTedsOutlinerImpl> TedsOutlinerImpl;
	};

	class FTedsOutlinerUiColumn : public ISceneOutlinerColumn
	{
	public:
		
		FTedsOutlinerUiColumn(FTedsOutlinerUiColumnInitParams& InitParams);
		
		FName GetColumnID() override;

		virtual void Tick(double InCurrentTime, float InDeltaTime) override;
		virtual SHeaderRow::FColumn::FArguments ConstructHeaderRowColumn() override;
		virtual void SortItems(TArray<FSceneOutlinerTreeItemPtr>& RootItems, const EColumnSortMode::Type SortMode) const override;
		virtual bool SupportsSorting() const override;
		virtual void OnSortRequested(const EColumnSortPriority::Type SortPriority, const EColumnSortMode::Type InSortMode) override;
		virtual bool IsSortReady() override;
		const TSharedRef<SWidget> ConstructRowWidget(FSceneOutlinerTreeItemRef TreeItem, const STableRow<FSceneOutlinerTreeItemPtr>& Row) override;
		virtual void PopulateSearchStrings( const ISceneOutlinerTreeItem& Item, TArray< FString >& OutSearchStrings ) const override;
		
		bool IsRowVisible(const DataStorage::RowHandle InRowHandle) const;

	protected:

		// The table viewer implementation that we internally use to create our widgets
		TUniquePtr<DataStorage::FTedsTableViewerColumn> TableViewerColumnImpl;
		
		DataStorage::ICoreProvider& Storage;
		DataStorage::IUiProvider& StorageUi;
		DataStorage::ICompatibilityProvider& StorageCompatibility;
		DataStorage::FMetaData MetaData;
		FName NameId;
		TSharedPtr<ISceneOutlinerColumn> FallbackColumn;
		TWeakPtr<ISceneOutliner> OwningOutliner;
		TWeakPtr<FTedsOutlinerImpl> TedsOutlinerImpl;
		FTreeItemIDDealiaser Dealiaser;
		bool bHybridMode;
	};

}
