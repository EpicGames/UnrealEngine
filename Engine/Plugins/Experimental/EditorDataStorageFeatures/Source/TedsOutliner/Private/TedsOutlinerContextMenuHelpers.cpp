// Copyright Epic Games, Inc. All Rights Reserved.

#include "TedsOutlinerContextMenuHelpers.h"

#include "EditorActorFolders.h"
#include "SceneOutlinerFwd.h"
#include "SSceneOutliner.h"
#include "Selection.h"
#include "TedsOutlinerFolderHelpers.h"
#include "TedsOutlinerItem.h"
#include "TedsOutlinerMode.h"
#include "TedsOutlinerModule.h"
#include "TedsQueryNode.h"
#include "TedsRowFilterNode.h"
#include "TedsRowMergeNode.h"
#include "TedsRowQueryResultsNode.h"
#include "UnrealEdGlobals.h"
#include "ActorFolders/ActorFolderColumns.h"
#include "Context/TedsPickerContextUtil.h"
#include "Editor/UnrealEdEngine.h"
#include "Elements/Columns/TypedElementCompatibilityColumns.h"
#include "Elements/Columns/TypedElementFolderColumns.h"
#include "Elements/Columns/TypedElementLabelColumns.h"
#include "Elements/Columns/TypedElementUIColumns.h"
#include "Elements/Interfaces/TypedElementDataStorageCompatibilityInterface.h"
#include "Elements/Interfaces/TypedElementDataStorageInterface.h"
#include "Columns/TedsOutlinerColumns.h"
#include "Factories/TedsEditorHierarchyFactory.h"
#include "Widgets/SEverythingPicker.h"
#include "Framework/Application/SlateApplication.h"
#include "ScopedTransaction.h"
#include "ActorDesc/TedsActorDescColumns.h"
#include "Columns/TedsActorWorldPartitionColumns.h"
#include "HAL/PlatformApplicationMisc.h"
#include "WorldPartition/WorldPartition.h"
#include "WorldPartition/WorldPartitionActorDescInstance.h"
#include "WorldPartition/ActorDescContainerInstance.h"

#define LOCTEXT_NAMESPACE "TedsOutlinerContextMenuHelpers"

namespace UE::Editor::Outliner::Helpers
{
	using namespace UE::Editor::DataStorage;

	namespace Private
	{
		static RowHandle GetRow(const FSceneOutlinerTreeItemPtr& Item)
		{
			if (Item.IsValid())
			{
				if (const FTedsOutlinerTreeItem* TedsItem = Item->CastTo<FTedsOutlinerTreeItem>())
				{
					return TedsItem->GetRowHandle();
				}
			}
			return InvalidRowHandle;
		}

		struct FTedsSelection
		{
			explicit FTedsSelection(const TArray<FSceneOutlinerTreeItemPtr>& InItems)
				: Items(InItems)
			{
			}

			int32 Num() const
			{
				return Items.Num();
			}

			// Raw item access for loops that don't need TEDS filtering (e.g. root-object computation).
			const TArray<FSceneOutlinerTreeItemPtr>& GetItems() const
			{
				return Items;
			}

			int32 Count(TFunctionRef<bool(RowHandle)> Predicate) const
			{
				int32 NumMatching = 0;
				for (const FSceneOutlinerTreeItemPtr& Item : Items)
				{
					if (const RowHandle Row = GetRow(Item); Row != InvalidRowHandle && Predicate(Row))
					{
						++NumMatching;
					}
				}
				return NumMatching;
			}

			void ForEach(TFunctionRef<bool(RowHandle)> Predicate, TFunctionRef<void(RowHandle, const FSceneOutlinerTreeItemPtr&)> Func) const
			{
				for (const FSceneOutlinerTreeItemPtr& Item : Items)
				{
					if (const RowHandle Row = GetRow(Item); Row != InvalidRowHandle && Predicate(Row))
					{
						Func(Row, Item);
					}
				}
			}

		private:
			const TArray<FSceneOutlinerTreeItemPtr>& Items;
		};

		namespace Folder
		{

			static void AddHierarchySection(
				UToolMenu* InMenu,
				const FTedsSelection& Selection,
				ICoreProvider& Storage,
				TWeakPtr<SSceneOutliner> WeakOutliner,
				TWeakObjectPtr<UWorld> World)
			{
				FToolMenuSection& Section = InMenu->FindOrAddSection("Section");
				Section.Label = LOCTEXT("HierarchySectionName", "Hierarchy");

				auto IsFolderRow = [&Storage](RowHandle Row)
					{
						return Storage.HasColumns<FFolderCompatibilityColumn>(Row);
					};

				const int32 NumFolders = Selection.Count(IsFolderRow);
				const bool bSingleFolderSelected = (Selection.Num() == 1) && (NumFolders == 1);
				const bool bAllAreFolders = (Selection.Num() > 0) && (NumFolders == Selection.Num());

				if (bSingleFolderSelected)
				{
					FSceneOutlinerTreeItemPtr SelectedFolder;
					Selection.ForEach(IsFolderRow, [&SelectedFolder](RowHandle, const FSceneOutlinerTreeItemPtr& Item)
						{
							SelectedFolder = Item;
						});

					const FSlateIcon NewFolderIcon(FAppStyle::GetAppStyleSetName(), "SceneOutliner.NewFolderIcon");

					// Create Sub Folder
					Section.AddMenuEntry(
						"CreateSubFolder",
						LOCTEXT("CreateSubFolder", "Create Sub Folder"),
						FText(),
						NewFolderIcon,
						FUIAction(FExecuteAction::CreateLambda([SelectedFolder, WeakOutliner, World]()
							{
								ICoreProvider* Storage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName);
								if (!Storage)
								{
									return;
								}

								if (!World.IsValid())
								{
									return;
								}

								if (const TSharedPtr<SSceneOutliner> OutlinerPinned = WeakOutliner.Pin())
								{
									Helpers::CreateFolder(*Storage, *OutlinerPinned, *World, SelectedFolder);
								}
							})));

					// Duplicate Hierarchy
					Section.AddMenuEntry(
						"DuplicateFolderHierarchy",
						LOCTEXT("DuplicateFolderHierarchy", "Duplicate Hierarchy"),
						FText(),
						FSlateIcon(),
						FUIAction(
							FExecuteAction::CreateLambda([SelectedFolder, WeakOutliner, World]()
								{
									ICoreProvider* Storage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName);
									if (!Storage)
									{
										return;
									}

									if (!World.IsValid())
									{
										return;
									}

									if (const TSharedPtr<SSceneOutliner> OutlinerPinned = WeakOutliner.Pin())
									{
										const FScopedTransaction Transaction(NSLOCTEXT("UnrealEd", "DuplicateFoldersHierarchy", "Duplicate Folders Hierarchy"));

										GEditor->SelectNone(/*bNoteSelectionChange*/ false, /*bDeselectBSPSurfs*/ true);
										Helpers::SelectDescendantActors(*OutlinerPinned, { SelectedFolder }, /*bSelectImmediateChildrenOnly*/ false);

										GUnrealEd->Exec(World.Get(), TEXT("DUPLICATE"));
									}
								}),
							FCanExecuteAction::CreateLambda([SelectedFolder]()
								{
									return GEditor != nullptr && GUnrealEd != nullptr && SelectedFolder != nullptr;
								})));
				}

				if (bAllAreFolders)
				{
					TArray<FSceneOutlinerTreeItemPtr> SelectedFolderItems;
					Selection.ForEach(IsFolderRow, [&SelectedFolderItems](RowHandle, const FSceneOutlinerTreeItemPtr& Item)
						{
							SelectedFolderItems.Add(Item);
						});

					Section.AddSubMenu(
						"SelectSubMenu",
						LOCTEXT("SelectSubmenu", "Select"),
						LOCTEXT("SelectSubmenu_Tooltip", "Select the contents of the current selection"),
						FNewToolMenuDelegate::CreateLambda([SelectedFolderItems, WeakOutliner](UToolMenu* InSubMenu)
							{
								FToolMenuSection& SubSection = InSubMenu->AddSection("Section");
								// Select Immediate Children
								SubSection.AddMenuEntry(
									"AddChildrenToSelection",
									LOCTEXT("AddChildrenToSelection", "Immediate Children"),
									LOCTEXT("AddChildrenToSelection_ToolTip", "Select all immediate children of the selected folders"),
									FSlateIcon(),
									FExecuteAction::CreateLambda([SelectedFolderItems, WeakOutliner]()
										{
											if (TSharedPtr<SSceneOutliner> OutlinerPinned = WeakOutliner.Pin())
											{
												Helpers::SelectDescendantActors(*OutlinerPinned, SelectedFolderItems, /*bSelectImmediateChildrenOnly*/ true);
											}
										}));

								// Select All Descendants
								SubSection.AddMenuEntry(
									"AddDescendantsToSelection",
									LOCTEXT("AddDescendantsToSelection", "All Descendants"),
									LOCTEXT("AddDescendantsToSelection_ToolTip", "Select all descendants of the selected folders"),
									FSlateIcon(),
									FExecuteAction::CreateLambda([SelectedFolderItems, WeakOutliner]()
										{
											if (TSharedPtr<SSceneOutliner> OutlinerPinned = WeakOutliner.Pin())
											{
												Helpers::SelectDescendantActors(*OutlinerPinned, SelectedFolderItems, /*bSelectImmediateChildrenOnly*/ false);
											}
										}));
							}));
				}
			}

			// Build the "Move To" submenu from TEDS folder rows rather than going through SSceneOutliner::FillFoldersSubMenu.
			static void AddMoveToSection(
				UToolMenu* InMenu,
				const FTedsSelection& Selection,
				ICoreProvider& Storage,
				TWeakPtr<SSceneOutliner> WeakOutliner,
				TWeakObjectPtr<UWorld> World)
			{
				// Determine the common root object across the full selection. Mixed roots produce an invalid root.
				FFolder::FRootObject TargetRoot = FFolder::GetInvalidRootObject();
				{
					TOptional<FFolder::FRootObject> Common;
					bool bMixed = false;
					for (const FSceneOutlinerTreeItemPtr& Item : Selection.GetItems())
					{
						if (!Item.IsValid())
						{
							continue;
						}
						const FFolder::FRootObject ItemRoot = Item->GetRootObject();
						if (!Common.IsSet())
						{
							Common = ItemRoot;
						}
						else if (Common.GetValue() != ItemRoot)
						{
							bMixed = true;
							break;
						}
					}
					if (!bMixed && Common.IsSet())
					{
						TargetRoot = Common.GetValue();
					}
				}

				if (!FFolder::IsRootObjectValid(TargetRoot))
				{
					return;
				}

				TArray<FSceneOutlinerTreeItemPtr> SelectedFolderItems;
				Selection.ForEach(
					[&Storage](RowHandle Row)
				{
					return Storage.HasColumns<FFolderCompatibilityColumn>(Row);
				},
				[&SelectedFolderItems](RowHandle, const FSceneOutlinerTreeItemPtr& Item)
				{
					SelectedFolderItems.Add(Item);
				});

				TArray<FSceneOutlinerTreeItemPtr> SelectedActorItems;
				Selection.ForEach(
					[&Storage](RowHandle Row)
				{
					return Storage.HasColumns<FTypedElementActorTag>(Row);
				},
				[&SelectedActorItems](RowHandle, const FSceneOutlinerTreeItemPtr& Item)
				{
					SelectedActorItems.Add(Item);
				});

				FToolMenuSection& MainSection = InMenu->FindOrAddSection(
					"MainSection",
					LOCTEXT("OutlinerSectionName", "Outliner"),
					FToolMenuInsert(NAME_None, EToolMenuInsertType::Last));
				MainSection.AddSubMenu(
					"MoveActorsTo",
					LOCTEXT("MoveActorsTo", "Move To"),
					LOCTEXT("MoveActorsTo_Tooltip", "Move selection to another folder"),
					FNewToolMenuDelegate::CreateLambda([WeakOutliner, TargetRoot, SelectedFolderItems, SelectedActorItems, World](UToolMenu* InSubMenu)
						{
							using namespace UE::Editor::DataStorage::Picker;
							using namespace UE::Editor::DataStorage::QueryStack;
							using namespace UE::Editor::DataStorage::Queries;

							ICoreProvider* Storage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName);
							if (!Storage)
							{
								return;
							}

							// Collect selected folders so the picker can exclude them and their descendants.
							// Matches SSceneOutliner::GatherInvalidMoveToDestinations.
							TArray<FFolder> ExcludedFolders;
							TArray<FFolder> ExcludedParentFolders;
							bool bSelectionHasRootItem = false;
							for (const FSceneOutlinerTreeItemPtr& Item : SelectedFolderItems)
							{
								if (const FTedsOutlinerTreeItem* TedsItem = Item.IsValid() ? Item->CastTo<FTedsOutlinerTreeItem>() : nullptr)
								{
									const RowHandle Row = TedsItem->GetRowHandle();
									if (const FFolderCompatibilityColumn* FolderColumn = Storage->GetColumn<FFolderCompatibilityColumn>(Row))
									{
										if (!FolderColumn->Folder.IsNone())
										{
											ExcludedFolders.Add(FolderColumn->Folder);

											if (FFolder ParentFolder = FolderColumn->Folder.GetParent(); !ParentFolder.IsNone())
											{
												ExcludedParentFolders.Add(ParentFolder);
											}
											else
											{
												bSelectionHasRootItem = true;
											}
										}
									}
								}
							}
							for (const FSceneOutlinerTreeItemPtr& Item : SelectedActorItems)
							{
								if (const FTedsOutlinerTreeItem* TedsItem = Item.IsValid() ? Item->CastTo<FTedsOutlinerTreeItem>() : nullptr)
								{
									const RowHandle Row = TedsItem->GetRowHandle();
									if (Storage->HasColumns<FTypedElementActorTag>(Row))
									{
										if (const FTypedElementUObjectColumn* ObjectColumn = Storage->GetColumn<FTypedElementUObjectColumn>(Row))
										{
											if (const AActor* Actor = Cast<AActor>(ObjectColumn->Object.Get()))
											{
												if (FFolder ActorFolder = Actor->GetFolder(); !ActorFolder.IsNone())
												{
													ExcludedParentFolders.Add(ActorFolder);
												}
												else
												{
													bSelectionHasRootItem = true;
												}
											}
										}
									}
								}
							}

							// Keep "Create New Folder" as a one-click entry above the picker for feature parity
							const FSlateIcon NewFolderIcon(FAppStyle::GetAppStyleSetName(), "SceneOutliner.NewFolderIcon");
							FToolMenuSection& NewSection = InSubMenu->AddSection("CreateNew");
							NewSection.AddMenuEntry(
								"CreateNew",
								LOCTEXT("CreateNew", "Create New Folder"),
								LOCTEXT("CreateNew_ToolTip", "Move to a new folder"),
								NewFolderIcon,
								FUIAction(FExecuteAction::CreateLambda([WeakOutliner, SelectedFolderItems, World]()
									{
										ICoreProvider* Storage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName);
										if (!Storage)
										{
											return;
										}

										if (!World.IsValid())
										{
											return;
										}

										if (const TSharedPtr<SSceneOutliner> OutlinerPinned = WeakOutliner.Pin())
										{
											Helpers::CreateFolderForSelection(*Storage, *OutlinerPinned, *World, SelectedFolderItems);
										}
									})));

							// Query available Folder options
							TSharedRef<FQueryNode> QueryNode = MakeShared<FQueryNode>(*Storage,
								Select()
								.ReadOnly<FFolderCompatibilityColumn>()
								.Where()
								.All<FFolderTag>()
								.None<FHideRowFromUITag>()
								.Compile());
							TSharedRef<FRowQueryResultsNode> ResultsNode = MakeShared<FRowQueryResultsNode>(*Storage, QueryNode);
							TSharedPtr<FRowFilterNode> FilterNode = MakeShared<FRowFilterNode>(Storage, ResultsNode.ToSharedPtr(),
								[TargetRoot, ExcludedFolders, ExcludedParentFolders](TConstQueryContext<SingleRowInfo> , const FFolderCompatibilityColumn& FolderColumn)
								{
									if (FolderColumn.Folder.IsNone() || FolderColumn.Folder.GetRootObject() != TargetRoot)
									{
										return false;
									}
									for (const FFolder& Excluded : ExcludedFolders)
									{
										if (FolderColumn.Folder == Excluded || FolderColumn.Folder.IsChildOf(Excluded))
										{
											return false;
										}
									}
									for (const FFolder& Excluded : ExcludedParentFolders)
									{
										if (FolderColumn.Folder == Excluded)
										{
											return false;
										}
									}
									return true;
								});

							// For parity, display the World object for a root drop but only when no selected item is already at the root level
							TSharedPtr<IRowNode> MergedNode = FilterNode;
							if (!bSelectionHasRootItem)
							{
								TSharedRef<FQueryNode> WorldQueryNode = MakeShared<FQueryNode>(*Storage,
									Select()
									.ReadOnly<FTypedElementUObjectColumn>()
									.Where()
									.All<FWorldTag>()
									.Compile());
								TSharedRef<FRowQueryResultsNode> WorldResultsNode = MakeShared<FRowQueryResultsNode>(*Storage, WorldQueryNode);
								TSharedPtr<FRowFilterNode> WorldFilterNode = MakeShared<FRowFilterNode>(Storage, WorldResultsNode.ToSharedPtr(),
									[World](TConstQueryContext<SingleRowInfo>, const FTypedElementUObjectColumn& ObjectColumn)
									{
										return ObjectColumn.Object.Get() == World.Get();
									});

								// Merge the World Filter before, so it's pinned on top
								const TSharedPtr<IRowNode> MergeInputs[] = { WorldFilterNode, FilterNode };
								MergedNode = MakeShared<FRowMergeNode>(MergeInputs, FRowMergeNode::EMergeApproach::Append);
							}

							const TSharedRef<SWidget> PickerWidget =
								SNew(SEverythingPicker)
								.MinDesiredWidth(200.f)
								.MaxDesiredHeight(400.f)
								.ShowTabLabelWhenSingle(false)
								+ SEverythingPicker::Context()
								.Label(LOCTEXT("MoveToFoldersTab", "Folders"))
								[
									SNew(SObjectReferenceContextView)
										.QueryStack(MergedNode)
										.HierarchyHandle(Storage->FindHierarchyByName(UTedsEditorHierarchyFactory::EditorObjectHierarchyName))
										.SearchingEnabled(true)
										.Columns({ FTypedElementLabelColumn::StaticStruct() })
										.OnSelectionChanged(FOnSelectionChanged::CreateLambda(
											[WeakOutliner](RowHandle Row, ESelectInfo::Type)
											{
												ICoreProvider* Storage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName);
												if (!Storage)
												{
													return;
												}

												FFolder NewFolder = FFolder::GetInvalidFolder();

												// Allow only Folder and World rows
												if (const FFolderCompatibilityColumn* FolderColumn = Storage->GetColumn<FFolderCompatibilityColumn>(Row))
												{
													NewFolder = FolderColumn->Folder;
												}
												else if (!Storage->HasColumns<FWorldTag>(Row))
												{
													return;
												}

												// We can use the SSceneOutliner::MoveSelectionTo here
												if (TSharedPtr<SSceneOutliner> Outliner = WeakOutliner.Pin())
												{
													Outliner->MoveSelectionTo(NewFolder);
												}
												FSlateApplication::Get().DismissAllMenus();
											}))
								];

							FToolMenuSection& PickerSection = InSubMenu->AddSection("MoveToPickerSection", LOCTEXT("ExistingFolders", "Existing:"));
							PickerSection.AddEntry(FToolMenuEntry::InitWidget("MoveToPicker", PickerWidget, FText::GetEmpty()));
						}));
			}

			static void AddActorEditorContextSection(
				UToolMenu* InMenu,
				const FTedsSelection& Selection,
				ICoreProvider& Storage,
				TWeakPtr<SSceneOutliner> WeakOutliner,
				TWeakObjectPtr<UWorld> World)
			{
				FToolMenuSection& EditorContextSection = InMenu->FindOrAddSection(
					"ActorEditorContextSection",
					LOCTEXT("ActorEditorContextSectionName", "Actor Editor Context"),
					FToolMenuInsert(NAME_None, EToolMenuInsertType::Last));

				auto IsFolderRow = [&Storage](RowHandle Row)
					{
						return Storage.HasColumns<FFolderCompatibilityColumn>(Row);
					};

				const bool bSingleFolderSelected = (Selection.Num() == 1) && (Selection.Count(IsFolderRow) == 1);
				if (bSingleFolderSelected)
				{
					RowHandle SingleFolderRow = InvalidRowHandle;
					Selection.ForEach(IsFolderRow, [&SingleFolderRow](RowHandle Row, const FSceneOutlinerTreeItemPtr&)
						{
							SingleFolderRow = Row;
						});

					EditorContextSection.AddMenuEntry(
						"MakeCurrentFolder",
						LOCTEXT("MakeCurrentFolder", "Make Current Folder"),
						FText(),
						FSlateIcon(),
						FUIAction(
							FExecuteAction::CreateLambda([SingleFolderRow, World]()
								{
									if (!World.IsValid())
									{
										return;
									}

									ICoreProvider* Storage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName);
									if (!Storage)
									{
										return;
									}
									const FFolderCompatibilityColumn* FolderColumn = Storage->GetColumn<FFolderCompatibilityColumn>(SingleFolderRow);
									if (!FolderColumn)
									{
										return;
									}

									const FScopedTransaction Transaction(NSLOCTEXT("UnrealEd", "MakeCurrentActorFolder", "Make Current Actor Folder"));
									FActorFolders::Get().SetActorEditorContextFolder(*World, FolderColumn->Folder);
								}),
							FCanExecuteAction::CreateLambda([SingleFolderRow, World]()
								{
									if (!World.IsValid())
									{
										return false;
									}

									ICoreProvider* Storage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName);
									if (!Storage)
									{
										return false;
									}
									const FFolderCompatibilityColumn* FolderColumn = Storage->GetColumn<FFolderCompatibilityColumn>(SingleFolderRow);
									if (!FolderColumn)
									{
										return false;
									}

									return FActorFolders::Get().GetActorEditorContextFolder(*World) != FolderColumn->Folder;
								})));
				}

				EditorContextSection.AddMenuEntry(
					"ClearCurrentFolder",
					LOCTEXT("ClearCurrentFolder", "Clear Current Folder"),
					FText(),
					FSlateIcon(),
					FUIAction(
						FExecuteAction::CreateLambda([WeakOutliner, World]()
							{
								if (!World.IsValid())
								{
									return;
								}
								ICoreProvider* Storage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName);
								if (!Storage)
								{
									return;
								}
								if (TSharedPtr<SSceneOutliner> Outliner = WeakOutliner.Pin())
								{
									const FScopedTransaction Transaction(NSLOCTEXT("UnrealEd", "ClearCurrentActorFolder", "Clear Current Actor Folder"));
									FActorFolders::Get().SetActorEditorContextFolder(*World, FFolder::GetWorldRootFolder(World.Get()));
								}
							}),
						FCanExecuteAction::CreateLambda([World]()
							{
								if (!World.IsValid())
								{
									return false;
								}
								ICoreProvider* Storage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName);
								if (!Storage)
								{
									return false;
								}

								return !FActorFolders::Get().GetActorEditorContextFolder(*World).IsNone();
							})));
			}
		} // namespace Folder

		namespace ActorDesc
		{
			static const FWorldPartitionActorDescInstance* ResolveActorDescInstance(RowHandle Row)
			{
				using namespace UE::Editor::DataStorage;

				ICoreProvider* Storage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName);
				if (!Storage)
				{
					return nullptr;
				}
				if (const FWorldPartitionHandleColumn* HandleColumn = Storage->GetColumn<FWorldPartitionHandleColumn>(Row))
				{
					return *HandleColumn->Handle;
				}
				return nullptr;
			}

			static void FocusActorBoundsForRow(RowHandle Row)
			{
				if (const FWorldPartitionActorDescInstance* DescInstance = ResolveActorDescInstance(Row))
				{
					const FBox EditorBounds = DescInstance->GetEditorBounds();
					if (EditorBounds.IsValid && GEditor)
					{
						GEditor->MoveViewportCamerasToBox(EditorBounds, /*bActiveViewportOnly=*/ true, 0.5f);
					}
				}
			}

			static void CopyActorFilePathToClipboardForRow(RowHandle Row)
			{
				if (const FWorldPartitionActorDescInstance* DescInstance = ResolveActorDescInstance(Row))
				{
					FString PackageFilename;
					if (FPackageName::TryConvertLongPackageNameToFilename(
						DescInstance->GetActorPackage().ToString(),
						PackageFilename,
						FPackageName::GetAssetPackageExtension()))
					{
						FString Result = FPaths::ConvertRelativePathToFull(PackageFilename);
						FPaths::MakePlatformFilename(Result);
						FPlatformApplicationMisc::ClipboardCopy(*Result);
					}
				}
			}

			// Per-row pin info.
			// populated for rows that carry FWorldPartitionPinnedColumn (i.e. rows that support pinning).
			struct FRowPinInfo
			{
				UWorldPartition* WorldPartition = nullptr;
				FGuid Guid;
				bool bIsPinned = false;
			};

			// Resolve pin info for a row that supports pinning. Presence of FWorldPartitionPinnedColumn is the signal.
			static bool ResolveRowPinInfo(ICoreProvider& Storage, RowHandle Row, FRowPinInfo& Out)
			{
				const FWorldPartitionPinnedColumn* PinnedColumn = Storage.GetColumn<FWorldPartitionPinnedColumn>(Row);
				if (!PinnedColumn)
				{
					return false;
				}

				const FGuidColumn* GuidColumn = Storage.GetColumn<FGuidColumn>(Row);
				const FTypedElementWorldColumn* WorldColumn = Storage.GetColumn<FTypedElementWorldColumn>(Row);
				if (!GuidColumn || !WorldColumn)
				{
					return false;
				}
				UWorldPartition* WorldPartition = WorldColumn->World.IsValid() ? WorldColumn->World->GetWorldPartition() : nullptr;
				if (!WorldPartition)
				{
					return false;
				}

				Out.WorldPartition = WorldPartition;
				Out.Guid = GuidColumn->Guid;
				Out.bIsPinned = PinnedColumn->bIsPinned;
				return true;
			}

			// Mirrors FActorFolderTreeItem::CanChangeChildrenPinnedState: folder row in a partitioned, non-game world whose root is the persistent level.
			static bool IsPartitionedFolderRow(ICoreProvider& Storage, RowHandle Row, UWorld* World)
			{
				if (!World || World->IsGameWorld() || !World->IsPartitionedWorld())
				{
					return false;
				}
				const FFolderCompatibilityColumn* FolderColumn = Storage.GetColumn<FFolderCompatibilityColumn>(Row);
				if (!FolderColumn)
				{
					return false;
				}
				return FFolder::IsRootObjectPersistentLevel(FolderColumn->Folder.GetRootObject());
			}

			// Visit a root row plus every descendant via the EditorObjectHierarchy.
			static void GatherForceLoadGuids(
				RowHandle RootRow,
				ICoreProvider& Storage,
				FHierarchyHandle Hierarchy,
				TMap<UWorldPartition*, TSet<FGuid>>& OutSeen)
			{
				if (RootRow == InvalidRowHandle || !Storage.IsValidHierarchyHandle(Hierarchy))
				{
					return;
				}

				Storage.WalkDepthFirst(Hierarchy, RootRow,
					[&Storage, &OutSeen]
					(const ICoreProvider&, RowHandle, RowHandle TargetRow)
					{
						FRowPinInfo Info;
						if (!ResolveRowPinInfo(Storage, TargetRow, Info))
						{
							return;
						}
						OutSeen.FindOrAdd(Info.WorldPartition).Add(Info.Guid);
					},
					ICoreProvider::ETraversalOrder::PreOrder);
			}

			struct FForceLoadSelectionInfo
			{
				TMap<UWorldPartition*, TArray<FGuid>> GuidsByPartition;
				int32 NumSelectedItems = 0;
				int32 NumSelectedFolders = 0;
				int32 NumPinnedTopLevel = 0;  // top-level Actor/ActorDesc rows already pinned
				bool bAnyPinnable = false;    // any top-level row that is pin-capable
			};

			static FForceLoadSelectionInfo ClassifyForceLoadSelection(
				const TArray<FSceneOutlinerTreeItemPtr>& InItems,
				ICoreProvider& Storage,
				UWorld* World)
			{
				FForceLoadSelectionInfo Info;
				Info.NumSelectedItems = InItems.Num();

				// Top-level pass: counters mirror FActorBrowsingMode::FillDefaultContextBaseMenu's Context lookup.
				for (const FSceneOutlinerTreeItemPtr& Item : InItems)
				{
					if (!Item.IsValid())
					{
						continue;
					}
					const FTedsOutlinerTreeItem* TedsItem = Item->CastTo<FTedsOutlinerTreeItem>();
					if (!TedsItem)
					{
						continue;
					}
					const RowHandle Row = TedsItem->GetRowHandle();

					FRowPinInfo PinInfo;
					if (ResolveRowPinInfo(Storage, Row, PinInfo))
					{
						Info.bAnyPinnable = true;
						if (PinInfo.bIsPinned)
						{
							++Info.NumPinnedTopLevel;
						}
					}
					else if (Storage.HasColumns<FFolderCompatibilityColumn>(Row))
					{
						++Info.NumSelectedFolders;
						if (IsPartitionedFolderRow(Storage, Row, World))
						{
							Info.bAnyPinnable = true;
						}
					}
				}

				// Walk the TEDS EditorObjectHierarchy from each top-level row to collect descendants.
				TMap<UWorldPartition*, TSet<FGuid>> Seen;
				const FHierarchyHandle Hierarchy = Storage.FindHierarchyByName(UTedsEditorHierarchyFactory::EditorObjectHierarchyName);
				for (const FSceneOutlinerTreeItemPtr& Item : InItems)
				{
					if (!Item.IsValid())
					{
						continue;
					}
					const FTedsOutlinerTreeItem* TedsItem = Item->CastTo<FTedsOutlinerTreeItem>();
					if (!TedsItem)
					{
						continue;
					}
					GatherForceLoadGuids(TedsItem->GetRowHandle(), Storage, Hierarchy, Seen);
				}

				for (TPair<UWorldPartition*, TSet<FGuid>>& Pair : Seen)
				{
					Info.GuidsByPartition.Add(Pair.Key, Pair.Value.Array());
				}

				return Info;
			}
		} // namespace ActorDesc
	} // namespace Private

	static void FillFolderContextMenuSection(
		UToolMenu* InMenu,
		const Private::FTedsSelection& Selection,
		ICoreProvider& Storage,
		TWeakPtr<SSceneOutliner> WeakOutliner,
		TWeakObjectPtr<UWorld> World)
	{
		if (Selection.Num() == 0)
		{
			FToolMenuSection& Section = InMenu->FindOrAddSection("Section");
			Section.Label = LOCTEXT("HierarchySectionName", "Hierarchy");

			const FSlateIcon NewFolderIcon(FAppStyle::GetAppStyleSetName(), "SceneOutliner.NewFolderIcon");
			Section.AddMenuEntry(
				"CreateFolder",
				LOCTEXT("CreateFolder", "Create Folder"),
				LOCTEXT("CreateFolderTooltip", "Create Folder"),
				NewFolderIcon,
				FUIAction(FExecuteAction::CreateLambda([WeakOutliner, World]()
				{
					ICoreProvider* Storage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName);
					if (!Storage)
					{
						return;
					}
					if (!World.IsValid())
					{
						return;
					}
					if (const TSharedPtr<SSceneOutliner> OutlinerPinned = WeakOutliner.Pin())
					{
						Helpers::CreateFolder(*Storage, *OutlinerPinned, *World);
					}
				})));
			return;
		}

		Private::Folder::AddHierarchySection(InMenu, Selection, Storage, WeakOutliner, World);
		Private::Folder::AddMoveToSection(InMenu, Selection, Storage, WeakOutliner, World);
		Private::Folder::AddActorEditorContextSection(InMenu, Selection, Storage, WeakOutliner, World);
	}

	static void FillActorDescContextMenuSection(
		UToolMenu* InMenu,
		const Private::FTedsSelection& Selection,
		ICoreProvider& Storage,
		TWeakPtr<SSceneOutliner> WeakOutliner,
		TWeakObjectPtr<UWorld> World)
	{
		if (Selection.Num() == 0)
		{
			return;
		}

		const int32 NumActorDescs = Selection.Count([&Storage](RowHandle Row)
			{
				return Storage.HasColumns<FActorDescTag>(Row);
			});

		// Single-select per-instance entries: gate to one ActorDesc to mirror FActorDescTreeItem::GenerateContextMenu.
		if (Selection.Num() == 1 && NumActorDescs == 1)
		{
			FSceneOutlinerTreeItemPtr SelectedItem = Selection.GetItems()[0];

			FToolMenuSection& Section = InMenu->FindOrAddSection("Section");
			Section.AddMenuEntry(
				"FocusActorBounds",
				LOCTEXT("FocusActorBounds", "Focus Actor Bounds"),
				FText(),
				FSlateIcon(),
				FUIAction(FExecuteAction::CreateLambda([SelectedItem]()
					{
						Private::ActorDesc::FocusActorBoundsForRow(Private::GetRow(SelectedItem));
					})));

			Section.AddMenuEntry(
				"CopyActorFilePath",
				LOCTEXT("CopyActorFilePath", "Copy Actor File Path"),
				LOCTEXT("CopyActorFilePathTooltip", "Copy the file path where this actor is saved"),
				FSlateIcon(FAppStyle::GetAppStyleSetName(), "GenericCommands.Copy"),
				FUIAction(FExecuteAction::CreateLambda([SelectedItem]()
					{
						Private::ActorDesc::CopyActorFilePathToClipboardForRow(Private::GetRow(SelectedItem));
					})));
		}

		// Force Load / Release Force Load: partitioned-world only, mirrors ActorBrowsingMode.cpp:781-811.
		if (World.IsValid() && World->IsPartitionedWorld())
		{
			// Capture the top-level selection so the lambdas can re-walk children at execute/can-execute time.
			const TArray<FSceneOutlinerTreeItemPtr> SelectedItems = Selection.GetItems();

			FToolMenuSection& MainSection = InMenu->FindOrAddSection(
				"MainSection",
				LOCTEXT("OutlinerSectionName", "Outliner"),
				FToolMenuInsert(NAME_None, EToolMenuInsertType::Last));

			MainSection.AddMenuEntry(
				"ForceLoadItems",
				LOCTEXT("ForceLoad", "Force Load"),
				LOCTEXT("ForceLoadTooltip", "Keep the selected items loaded in the editor even when they don't overlap a loaded World Partition region"),
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateLambda([SelectedItems, WeakOutliner, World]()
					{
						ICoreProvider* Storage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName);
						if (!Storage)
						{
							return;
						}
						const Private::ActorDesc::FForceLoadSelectionInfo Info = Private::ActorDesc::ClassifyForceLoadSelection(SelectedItems, *Storage, World.Get());
						if (!GEditor)
						{
							for (const TPair<UWorldPartition*, TArray<FGuid>>& Pair : Info.GuidsByPartition)
							{
								if (Pair.Key && Pair.Value.Num() > 0)
								{
									Pair.Key->PinActors(Pair.Value);
								}
							}
							return;
						}

						// Clear current selection, pin, then re-select every freshly
						// pinned actor as a single batched selection change so the user lands on the loaded set.
						USelection* SelectedActors = GEditor->GetSelectedActors();
						SelectedActors->BeginBatchSelectOperation();
						GEditor->SelectNone(/*bNoteSelectionChange=*/false, /*bDeselectBSPSurfs=*/true);

						for (const TPair<UWorldPartition*, TArray<FGuid>>& Pair : Info.GuidsByPartition)
						{
							if (Pair.Key && Pair.Value.Num() > 0)
							{
								Pair.Key->PinActors(Pair.Value);
							}
						}

						AActor* LastPinnedActor = nullptr;

						// Resolve every pinned guid via FWorldPartitionHandle; works for actors that
						// were already loaded as well as ones PinActors just brought in.
						for (const TPair<UWorldPartition*, TArray<FGuid>>& Pair : Info.GuidsByPartition)
						{
							if (!Pair.Key)
							{
								continue;
							}
							for (const FGuid& Guid : Pair.Value)
							{
								FWorldPartitionHandle Handle(Pair.Key, Guid);
								if (!Handle.IsValid())
								{
									continue;
								}
								if (AActor* Actor = Handle->GetActor())
								{
									GEditor->SelectActor(Actor, /*bInSelected=*/true, /*bNotify=*/false);
									LastPinnedActor = Actor;
								}
							}
						}

						SelectedActors->EndBatchSelectOperation(/*bNotify=*/true);

						// Scroll into view the last Actor pinned
						if (LastPinnedActor)
						{
							if (TSharedPtr<SSceneOutliner> Outliner = WeakOutliner.Pin())
							{
								ICompatibilityProvider* CompatibilityProvider = GetMutableDataStorageFeature<ICompatibilityProvider>(CompatibilityFeatureName);
								if (!CompatibilityProvider)
								{
									return;
								}

								const RowHandle LastPinnedRow = CompatibilityProvider->FindRowWithCompatibleObject(LastPinnedActor);
								const RowHandle OutlinerRow = Storage->LookupMappedRow(MappingDomain, FMapKey(Outliner->GetOutlinerIdentifier()));

								if (const FTedsOutlinerPendingItemActionsColumn* Column = Storage->GetColumn<FTedsOutlinerPendingItemActionsColumn>(OutlinerRow))
								{
									if (Column->OnRegisterPendingItemActions && LastPinnedRow != InvalidRowHandle)
									{
										Column->OnRegisterPendingItemActions->Broadcast(LastPinnedRow, SceneOutliner::ENewItemAction::ScrollIntoView);
									}
								}
							}
						}
					}),
					FCanExecuteAction::CreateLambda([SelectedItems, World]()
					{
						ICoreProvider* Storage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName);
						if (!Storage)
						{
							return false;
						}
						const Private::ActorDesc::FForceLoadSelectionInfo Info = Private::ActorDesc::ClassifyForceLoadSelection(
							SelectedItems, *Storage, World.Get());
						if (!Info.bAnyPinnable)
						{
							return false;
						}
						// Enable when not every selected item is already pinned, or any folder is selected.
						return Info.NumPinnedTopLevel != Info.NumSelectedItems || Info.NumSelectedFolders > 0;
					})));

			MainSection.AddMenuEntry(
				"ReleaseForceLoadItems",
				LOCTEXT("ReleaseForceLoad", "Release Force Load"),
				LOCTEXT("ReleaseForceLoadTooltip", "Allow the World Partition system to load and unload the selected items automatically"),
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateLambda([SelectedItems, World]()
					{
						ICoreProvider* Storage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName);
						if (!Storage)
						{
							return;
						}
						const Private::ActorDesc::FForceLoadSelectionInfo Info = Private::ActorDesc::ClassifyForceLoadSelection(
							SelectedItems, *Storage, World.Get());
						for (const TPair<UWorldPartition*, TArray<FGuid>>& Pair : Info.GuidsByPartition)
						{
							if (Pair.Key && Pair.Value.Num() > 0)
							{
								Pair.Key->UnpinActors(Pair.Value);
							}
						}
					}),
					FCanExecuteAction::CreateLambda([SelectedItems, World]()
					{
						ICoreProvider* Storage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName);
						if (!Storage)
						{
							return false;
						}
						const Private::ActorDesc::FForceLoadSelectionInfo Info = Private::ActorDesc::ClassifyForceLoadSelection(
							SelectedItems, *Storage, World.Get());
						if (!Info.bAnyPinnable)
						{
							return false;
						}
						// Enable when at least one selected item is pinned, or any folder is selected.
						return Info.NumPinnedTopLevel > 0 || Info.NumSelectedFolders > 0;
					})));
		}
	}

	

	void BuildTedsOutlinerContextMenu(UToolMenu* InMenu, TWeakObjectPtr<UWorld> World)
	{
		const UTedsOutlinerMenuContext* TedsContext = InMenu->FindContext<UTedsOutlinerMenuContext>();
		if (!TedsContext || !TedsContext->OwningSceneOutliner)
		{
			return;
		}

		ICoreProvider* Storage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName);
		if (!Storage)
		{
			return;
		}

		SSceneOutliner& Outliner = *TedsContext->OwningSceneOutliner;
		const TArray<FSceneOutlinerTreeItemPtr> Items = Outliner.GetTree().GetSelectedItems();
		const Private::FTedsSelection Selection(Items);
		const TWeakPtr<SSceneOutliner> WeakOutliner = StaticCastSharedRef<SSceneOutliner>(Outliner.AsShared());

		FillFolderContextMenuSection(InMenu, Selection, *Storage, WeakOutliner, World);
		FillActorDescContextMenuSection(InMenu, Selection, *Storage, WeakOutliner, World);
	}

} // namespace UE::Editor::Outliner::Helpers

#undef LOCTEXT_NAMESPACE
