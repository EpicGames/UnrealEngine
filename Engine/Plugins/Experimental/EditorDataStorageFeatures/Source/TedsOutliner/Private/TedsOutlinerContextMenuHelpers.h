// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "UObject/WeakObjectPtrTemplates.h"

class UToolMenu;
class UWorld;

namespace UE::Editor::Outliner::Helpers
{
	// Snapshot the current outliner selection and dispatch to all registered section builders.
	// Call this from ModifyContextMenu / AddDynamicSection in GetSceneOutlinerInitializationOptions.
	void BuildTedsOutlinerContextMenu(UToolMenu* InMenu, TWeakObjectPtr<UWorld> World);
}
