// Copyright Epic Games, Inc. All Rights Reserved.

#include "TedsOutlinerFilter.h"

#include "DataStorage/Features.h"
#include "Elements/Columns/TypedElementCompatibilityColumns.h"
#include "Elements/Columns/TypedElementTypeInfoColumns.h"
#include "Elements/Framework/TypedElementQueryContext.h"
#include "Styling/SlateIconFinder.h"
#include "TedsOutlinerHelpers.h"
#include "TedsOutlinerImpl.h"

#define LOCTEXT_NAMESPACE "TEDSOutlinerFilter"

namespace UE::Editor::Outliner
{
FTedsOutlinerFilter::FTedsOutlinerFilter(const FName& InFilterName, const FText& InFilterDisplayName, const FText& InFilterToolTip, const FName& InFilterIconName, 
	const TSharedPtr<FFilterCategory>& InCategory, const QueryHandle& InFilterQuery, const bool bInteractiveFilter, const bool bActiveByDefault)
	: FTedsFilterBase(InFilterName, InFilterDisplayName, InFilterToolTip, InFilterIconName, InCategory, InFilterQuery, bActiveByDefault)
	, bInteractiveFilter(bInteractiveFilter)
{}

FTedsOutlinerFilter::FTedsOutlinerFilter(const FName& InFilterName, const FText& InFilterDisplayName, const QueryHandle& InFilterQuery,
	const bool bInteractiveFilter, const bool bActiveByDefault)
	: FTedsOutlinerFilter(InFilterName, InFilterDisplayName, FText::FromString(InFilterName.ToString()), FName(), 
		nullptr, InFilterQuery, bInteractiveFilter, bActiveByDefault)
{}

FTedsOutlinerFilter::FTedsOutlinerFilter(const FName& InFilterName, const FText& InFilterDisplayName, const FText& InFilterToolTip, 
	const FName& InFilterIconName, const TSharedPtr<FFilterCategory>& InCategory, const TConstQueryFunction<bool>& InFilterQuery,
	const bool bInteractiveFilter, const bool bActiveByDefault)
	: FTedsFilterBase(InFilterName, InFilterDisplayName, InFilterToolTip, InFilterIconName, InCategory, InFilterQuery, bActiveByDefault)
	, bInteractiveFilter(bInteractiveFilter)
{}

FTedsOutlinerFilter::FTedsOutlinerFilter(const FName& InFilterName, const FText& InFilterDisplayName, const TConstQueryFunction<bool>& InFilterQuery,
	const bool bInteractiveFilter, const bool bActiveByDefault)
	: FTedsOutlinerFilter(InFilterName, InFilterDisplayName, FText::FromString(InFilterName.ToString()), FName(), 
		nullptr, InFilterQuery, bInteractiveFilter, bActiveByDefault)
{}

FTedsOutlinerFilter::FTedsOutlinerFilter(const UClass* InClass, const TSharedPtr<FFilterCategory>& InCategory, const bool bInteractiveFilter, const bool bActiveByDefault)
	: FTedsFilterBase(InClass, InCategory, bActiveByDefault)
	, bInteractiveFilter(bInteractiveFilter)
{}

void FTedsOutlinerFilter::SetSceneOutlinerImpl(const TSharedPtr<FTedsOutlinerImpl>& InTedsOutlinerImpl)
{
	TedsOutlinerImpl = InTedsOutlinerImpl;
	if(IsActiveByDefault())
	{
		ActiveStateChanged(true);
	}
}

void FTedsOutlinerFilter::ActiveStateChanged(bool bActive)
{
	using namespace UE::Editor::DataStorage::Queries;
	
	const TSharedPtr<FTedsOutlinerImpl> TedsOutlinerImplPin = TedsOutlinerImpl.Pin();
	if (ensureMsgf(TedsOutlinerImplPin, TEXT("No TedsOutliner Context was set for the %s filter."), *FilterDisplayName.ToString()))
	{
		if(bActive)
		{
			if (bIsClassFilter)
			{
				// Class filters are separated since we want to OR them with each other
				TedsOutlinerImplPin->AddClassQueryFunction(FilterName, FilterQuery.Get<TConstQueryFunction<bool>>());
			}
			else if (FilterQuery.IsType<QueryHandle>())
			{
				TedsOutlinerImplPin->AddExternalQuery(FilterName, FilterQuery.Get<QueryHandle>());
			}
			else /* if (FilterQuery.IsType<TConstQueryFunction<bool>>()) */
			{
				TedsOutlinerImplPin->AddExternalQueryFunction(FilterName, FilterQuery.Get<TConstQueryFunction<bool>>());
			}
		}
		else
		{
			if (bIsClassFilter)
			{
				TedsOutlinerImplPin->RemoveClassQueryFunction(FilterName);
			}
			else if (FilterQuery.IsType<QueryHandle>())
			{
				TedsOutlinerImplPin->RemoveExternalQuery(FilterName);
			}
			else /* if (FilterQuery.IsType<TQueryFunction<bool>>()) */
			{
				TedsOutlinerImplPin->RemoveExternalQueryFunction(FilterName);
			}
		}
	}
}

bool FTedsOutlinerFilter::PassesFilter(SceneOutliner::FilterBarType InItem) const
{
	if (TSharedPtr<FTedsOutlinerImpl> TedsOutlinerImplPin = TedsOutlinerImpl.Pin())
	{
		// If this item is not compatible with the owning Table Viewer - it does not pass any filter queries
		// If it is compatible, this is simply a dummy filter for the UI while the actual filter is applied through the TEDS query
		if(TedsOutlinerImplPin->IsItemCompatible().IsBound())
		{
			return TedsOutlinerImplPin->IsItemCompatible().Execute(InItem);
		}
	}

	// The filter is applied through a TEDS query and this is just a dummy to activate it, so we can simply return false otherwise
	return false;
}
} // namespace UE::Editor::Outliner

#undef LOCTEXT_NAMESPACE
