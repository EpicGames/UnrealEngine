// Copyright Epic Games, Inc. All Rights Reserved.

#include "TedsOutlinerFolderHelpers.h"

#include "DataStorage/Features.h"
#include "Editor.h"
#include "EditorActorFolders.h"
#include "ISceneOutlinerMode.h"
#include "ScopedTransaction.h"
#include "Elements/Columns/TypedElementCompatibilityColumns.h"
#include "Elements/Interfaces/TypedElementDataStorageInterface.h"
#include "GameFramework/Actor.h"
#include "SSceneOutliner.h"
#include "Selection.h"
#include "Columns/TedsOutlinerColumns.h"
#include "TedsOutlinerHelpers.h"
#include "TedsOutlinerItem.h"
#include "ActorFolders/ActorFolderColumns.h"
#include "ActorFolders/TedsActorFolderUtils.h"
#include "Factories/TedsEditorHierarchyFactory.h"

#define LOCTEXT_NAMESPACE "TedsOutlinerFolderHelpers"

namespace UE::Editor::Outliner::Helpers
{
	namespace Private
	{
		AActor* GetActorFromTreeItem(const FSceneOutlinerTreeItemPtr& Item)
		{
			if (!Item.IsValid())
			{
				return nullptr;
			}
			if (const FTedsOutlinerTreeItem* TedsItem = Item->CastTo<FTedsOutlinerTreeItem>())
			{
				using namespace UE::Editor::DataStorage;
				if (ICoreProvider* Storage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName))
				{
					if (const RowHandle Row = TedsItem->GetRowHandle();
						Storage->HasColumns<FTypedElementActorTag>(Row))
					{
						if (const FTypedElementUObjectColumn* Obj = Storage->GetColumn<FTypedElementUObjectColumn>(Row))
						{
							return Cast<AActor>(Obj->Object.Get());
						}
					}
				}
			}
			return nullptr;
		}

		void CollectDescendantActors(SSceneOutliner& Outliner, const FSceneOutlinerTreeItemPtr& Root, bool bCollectImmediateChildrenOnly, TArray<AActor*>& OutActors)
		{
			using namespace UE::Editor::DataStorage;

			if (!Root.IsValid())
			{
				return;
			}
			const FTedsOutlinerTreeItem* TedsRoot = Root->CastTo<FTedsOutlinerTreeItem>();
			if (!TedsRoot)
			{
				return;
			}
			ICoreProvider* Storage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName);
			if (!Storage)
			{
				return;
			}

			const FHierarchyHandle Hierarchy = Storage->FindHierarchyByName(UTedsEditorHierarchyFactory::EditorObjectHierarchyName);
			if (!Storage->IsValidHierarchyHandle(Hierarchy))
			{
				return;
			}

			const TSharedRef<ISceneOutliner> OutlinerRef = StaticCastSharedRef<ISceneOutliner>(Outliner.AsShared());
			Outliner.SetItemExpansion(Root, true);

			auto Visit = [OutlinerRef, &OutActors, bExpand = !bCollectImmediateChildrenOnly](const ICoreProvider& Storage, RowHandle Row)
			{
				if (Storage.HasColumns<FTypedElementActorTag>(Row))
				{
					if (const FTypedElementUObjectColumn* ObjectColumn = Storage.GetColumn<FTypedElementUObjectColumn>(Row))
					{
						if (AActor* Actor = Cast<AActor>(ObjectColumn->Object.Get()))
						{
							OutActors.Add(Actor);
						}
					}
				}
				// Only expand non-immediate actors because the root was already expanded for them.
				if (bExpand)
				{
					if (FSceneOutlinerTreeItemPtr Item = Helpers::GetTreeItemFromRowHandle(&Storage, OutlinerRef, Row))
					{
						OutlinerRef->SetItemExpansion(Item, true);
					}
				}
			};

			const RowHandle RootRow = TedsRoot->GetRowHandle();
			if (bCollectImmediateChildrenOnly)
			{
				Storage->IterateChildren(Hierarchy, RootRow,
					[&Visit](const ICoreProvider& Storage, RowHandle Child) -> bool
					{
						Visit(Storage, Child);
						return true;
					});
			}
			else
			{
				Storage->WalkDepthFirst(Hierarchy, RootRow,
					[RootRow, &Visit](const ICoreProvider& Storage, RowHandle, RowHandle Target)
					{
						if (Target != RootRow)
						{
							Visit(Storage, Target);
						}
					});
			}
		}

		void RegisterNewFolder(ICoreProvider& Storage, const SSceneOutliner& Outliner, RowHandle NewFolderRow)
		{
			if (!Storage.IsRowAvailable(NewFolderRow))
			{
				return;
			}

			const RowHandle OutlinerRow = Storage.LookupMappedRow(MappingDomain,FMapKey(Outliner.GetOutlinerIdentifier()));

			if (const FTedsOutlinerPendingItemActionsColumn* Column = Storage.GetColumn<FTedsOutlinerPendingItemActionsColumn>(OutlinerRow))
			{
				if (Column->OnRegisterPendingItemActions)
				{
					Column->OnRegisterPendingItemActions->Broadcast(NewFolderRow, SceneOutliner::ENewItemAction::Select | SceneOutliner::ENewItemAction::Rename);
				}
			}
		}
	}
	

	TArray<FFolder> GetFolders(ICoreProvider& Storage, const TArray<FSceneOutlinerTreeItemPtr>& Items)
	{
		TArray<FFolder> Folders;
		for (const FSceneOutlinerTreeItemPtr& Item : Items)
		{
			if (Item.IsValid())
			{
				if (const FTedsOutlinerTreeItem* TedsItem = Item->CastTo<FTedsOutlinerTreeItem>())
				{
					const RowHandle Row = TedsItem->GetRowHandle();
					if (const FFolderCompatibilityColumn* FolderColumn = Storage.GetColumn<FFolderCompatibilityColumn>(Row))
					{
						if (!FolderColumn->Folder.IsNone())
						{
							Folders.Emplace(FolderColumn->Folder);
						}
					}
				}
			}
		}
		return Folders;
	}

	void SelectDescendantActors(SSceneOutliner& Outliner, const TArray<FSceneOutlinerTreeItemPtr>& Roots, bool bSelectImmediateChildrenOnly)
	{
		if (!GEditor)
		{
			return;
		}

		TArray<AActor*> ActorsToSelect;
		for (const FSceneOutlinerTreeItemPtr& Root : Roots)
		{
			Private::CollectDescendantActors(Outliner, Root, bSelectImmediateChildrenOnly, ActorsToSelect);
		}
		if (ActorsToSelect.IsEmpty())
		{
			return;
		}

		GEditor->SelectNone(/*bNoteSelectionChange*/ false, /*bDeselectBSPSurfs*/ true);
		USelection* Selection = GEditor->GetSelectedActors();
		Selection->BeginBatchSelectOperation();
		for (AActor* Actor : ActorsToSelect)
		{
			GEditor->SelectActor(Actor, /*bInSelected*/ true, /*bNotify*/ false);
		}
		Selection->EndBatchSelectOperation(/*bNotify*/ false);
		GEditor->NoteSelectionChange();
	}

	FFolder CreateFolder(ICoreProvider& Storage, const SSceneOutliner& Outliner, UWorld& World, const FSceneOutlinerTreeItemPtr& ParentFolder /* = nullptr */)
	{
		FFolder Folder = FFolder::GetInvalidFolder();

		if (ParentFolder.IsValid())
		{
			if (const FTedsOutlinerTreeItem* TedsItem = ParentFolder->CastTo<FTedsOutlinerTreeItem>())
			{
				const RowHandle Row = TedsItem->GetRowHandle();
				if (const FFolderCompatibilityColumn* FolderColumn = Storage.GetColumn<FFolderCompatibilityColumn>(Row))
				{
					Folder = FolderColumn->Folder;
				}
			}
		}

		const FScopedTransaction Transaction(LOCTEXT("UndoAction_CreateFolder", "Create Folder"));

		FFolder NewFolder = FActorFolders::Get().GetDefaultFolderName(World, Folder);
		FActorFolders::Get().CreateFolder(World, NewFolder);

		Private::RegisterNewFolder(Storage, Outliner, ActorFolders::LookupMappedRow(&Storage, NewFolder));

		return NewFolder;
	}

	FFolder CreateFolderForSelection(ICoreProvider& Storage, const SSceneOutliner& Outliner, UWorld& World, const TArray<FSceneOutlinerTreeItemPtr>& SelectedFolderItems)
	{
		TArray<FFolder> SelectedFolders = GetFolders(Storage, SelectedFolderItems);

		const FScopedTransaction Transaction(LOCTEXT("UndoAction_CreateFolder", "Create Folder"));

		const FFolder NewFolder = FActorFolders::Get().GetDefaultFolderForSelection(World, &SelectedFolders);
		FActorFolders::Get().CreateFolderContainingSelection(World, NewFolder);

		Private::RegisterNewFolder(Storage, Outliner, ActorFolders::LookupMappedRow(&Storage, NewFolder));

		return NewFolder;
	}
}

#undef LOCTEXT_NAMESPACE
