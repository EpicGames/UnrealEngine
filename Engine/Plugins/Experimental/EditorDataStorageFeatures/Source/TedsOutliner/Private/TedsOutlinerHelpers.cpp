// Copyright Epic Games, Inc. All Rights Reserved.

#include "TedsOutlinerHelpers.h"

#include "Columns/TedsOutlinerColumns.h"
#include "Compatibility/SceneOutlinerTedsBridge.h"
#include "DataStorage/Features.h"
#include "Elements/Interfaces/TypedElementDataStorageInterface.h"
#include "Engine/World.h"
#include "GameFramework/Actor.h"
#include "LevelInstance/LevelInstanceInterface.h"
#include "LevelInstance/LevelInstanceSubsystem.h"
#include "Modules/ModuleManager.h"
#include "ISceneOutliner.h"
#include "LevelEditor.h"
#include "SceneOutlinerStandaloneTypes.h"
#include "TedsOutlinerImpl.h"
#include "TedsOutlinerItem.h"
#include "Filters/CustomClassFilterData.h"
#include "TedsTableViewerUtils.h"
#include "Columns/SceneOutlinerColumns.h"
#include "Columns/TedsLevelInstanceColumns.h"
#include "Factories/TedsEditorHierarchyFactory.h"
#include "Elements/Columns/TypedElementCompatibilityColumns.h"
#include "Elements/Columns/TypedElementSlateWidgetColumns.h"
#include "Elements/Framework/TypedElementAttributeBinding.h"
#include "Widgets/Docking/SDockTab.h"
#include "FolderTreeItem.h"
#include "ActorTreeItem.h"
#include "ActorDescTreeItem.h"
#include "ActorDesc/TedsActorDescUtils.h"
#include "ActorFolders/TedsActorFolderUtils.h"

namespace UE::Editor::Outliner::Helpers
{
	FSceneOutlinerTreeItemPtr GetTreeItemFromRowHandle(const DataStorage::ICoreProvider* Storage, TSharedRef<ISceneOutliner> InOutliner, DataStorage::RowHandle InRowHandle)
	{
		// Check if the item is indexed by the row handle for the trivial case
		FSceneOutlinerTreeItemPtr Item = InOutliner->GetTreeItem(InRowHandle);

		if (Item)
		{
			return Item;
		}

		// Otherwise look up any dealiasers that might be bound to this outliner instance
		DataStorage::RowHandle OutlinerRow = Storage->LookupMappedRow(MappingDomain, DataStorage::FMapKey(InOutliner->GetOutlinerIdentifier()));

		if (Storage->IsRowAvailable(OutlinerRow))
		{
			if (const FTedsOutlinerDealiaserColumn* DealiaserColumn = Storage->GetColumn<FTedsOutlinerDealiaserColumn>(OutlinerRow))
			{
				if (DealiaserColumn->Dealiaser.IsBound())
				{
					return InOutliner->GetTreeItem(DealiaserColumn->Dealiaser.Execute(InRowHandle));
				}
			}
		}
		return nullptr;
		
	}
	
	bool RegisterOutlinerDealiaser(DataStorage::ICoreProvider* Storage, TSharedRef<ISceneOutliner> InOutliner, const FTreeItemIDDealiaser& InDealiaser)
	{
		DataStorage::RowHandle OutlinerRow = Storage->LookupMappedRow(MappingDomain, DataStorage::FMapKey(InOutliner->GetOutlinerIdentifier()));

		if (Storage->IsRowAvailable(OutlinerRow))
		{
			Storage->AddColumn(OutlinerRow, FTedsOutlinerDealiaserColumn{.Dealiaser = InDealiaser});
			return true;
		}

		return false;
	}
	
	FName GetTedsOutlinerTableName()
	{
		return TEXT("Editor_TedsOutlinerTable");
	}
	
	void RefreshLevelEditorOutliners()
	{
		FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>("LevelEditor");
		LevelEditorModule.RefreshLevelEditorOutliners();
	}
	
	bool OrderColumnGroup(const TArray<TWeakObjectPtr<const UScriptStruct>>& InColumnGroup,
		const FTedsOutlinerColumnDescription& InColumnDescription, const uint8 InDefaultStartingPriorityIndex,
		TMap<TWeakObjectPtr<const UScriptStruct>, uint8>& OutOrderedColumnMap)
	{
		// Custom implementation of the Kahn Topological Sort to order the columns.
		// Early return if we didn't receive any columns to order
		if (InColumnGroup.IsEmpty())
		{
			return false;
		}
		using FColumnPriorityRelation = FTedsOutlinerColumnParams::FColumnPriorityRelation;
		using EColumnPriorityRelation = FColumnPriorityRelation::EColumnPriorityRelation;

		// Default number of Columns to allocate, in the rare case that the column number is
		// above this amount, it will fall back to heap allocation
		static constexpr uint32 InlineColumnCount = 8;
		const int ColumnCount = InColumnGroup.Num();
		
		TMap<TWeakObjectPtr<const UScriptStruct>, TArray<TWeakObjectPtr<const UScriptStruct>, TInlineAllocator<InlineColumnCount>>,
			TInlineSetAllocator<InlineColumnCount>> ColumnOrderGraph;
		TMap<TWeakObjectPtr<const UScriptStruct>, int, TInlineSetAllocator<InlineColumnCount>> InDegree;
		ColumnOrderGraph.Reserve(ColumnCount);
		InDegree.Reserve(ColumnCount);

		// Initialize Graph
		for (const TWeakObjectPtr<const UScriptStruct>& Column : InColumnGroup)
		{
			ColumnOrderGraph.Add(Column, {});
			InDegree.Add(Column, 0);
		}

		// Add Relations to the Graph
		for (const TWeakObjectPtr<const UScriptStruct>& Column : InColumnGroup)
		{
			if (Column.IsValid())
			{
				if (const FTedsOutlinerColumnParams* Params = InColumnDescription.FindColumnParams(Column))
				{
					const FColumnPriorityRelation& PriorityRelation = Params->PriorityRelation;
					if (PriorityRelation.RelatedColumn.IsValid() && InColumnGroup.Contains(PriorityRelation.RelatedColumn))
					{
						if (PriorityRelation.PriorityRelation == EColumnPriorityRelation::Before)
						{
							ColumnOrderGraph[Column].Add(PriorityRelation.RelatedColumn);
							InDegree[PriorityRelation.RelatedColumn]++;
						}
						else if (PriorityRelation.PriorityRelation == EColumnPriorityRelation::After)
						{
							ColumnOrderGraph[PriorityRelation.RelatedColumn].Add(Column);
							InDegree[Column]++;
						}
					}
				}
			}
		}
		
		TQueue<TWeakObjectPtr<const UScriptStruct>> OrderQueue;
		TArray<TWeakObjectPtr<const UScriptStruct>> FinalGroupOrder;
        FinalGroupOrder.Reserve(ColumnCount);

		for (const TWeakObjectPtr<const UScriptStruct>& Column : InColumnGroup)
		{
			if (InDegree[Column] == 0)
			{
				OrderQueue.Enqueue(Column);
			}
		}

		while (!OrderQueue.IsEmpty())
		{
			TWeakObjectPtr<const UScriptStruct> CurrentColumn;
			OrderQueue.Dequeue(CurrentColumn);
			FinalGroupOrder.Add(CurrentColumn);
		
			for(const TWeakObjectPtr<const UScriptStruct>& FolRightingColumn : ColumnOrderGraph[CurrentColumn])
			{
				InDegree[FolRightingColumn]--;
				if (InDegree[FolRightingColumn] == 0)
				{
					OrderQueue.Enqueue(FolRightingColumn);
				}
			}
		}
		
		// Assign Priority Indexes
		int32 PriorityIndexOffset = 0;
		for (const TWeakObjectPtr<const UScriptStruct>& Column : FinalGroupOrder)
		{
			if (Column.IsValid())
			{
				OutOrderedColumnMap.Add(Column, 
					static_cast<uint8>(FMath::Clamp(InDefaultStartingPriorityIndex + static_cast<uint8>(
						FMath::Clamp(PriorityIndexOffset, 0, 255)), 0, 255)));
				++PriorityIndexOffset;
			}
		}

		// If the column sizes are different, there was a circular dependency that could not be resolved,
		// let the user know that they should reorder their columns.
		if (ensureAlwaysMsgf(FinalGroupOrder.Num() == ColumnCount,
				TEXT("The TEDS Outliner Column order contains a circular priority relation, "
				"the order will be incorrect until fixed.")))
		{
			return true;
		}
		return false;
	}
		
	bool OrderColumns(const TArray<TWeakObjectPtr<const UScriptStruct>>& InColumns,
		const FTedsOutlinerColumnDescription& InColumnDescription, TMap<TWeakObjectPtr<const UScriptStruct>, uint8>& OutOrderedColumnMap)
	{
		using EColumnPriorityGroup = FTedsOutlinerColumnParams::EColumnPriorityGroup;
		
		const int ColumnTypeNum = InColumns.Num();
		TArray<TWeakObjectPtr<const UScriptStruct>> LeftPriorityColumns, MiddlePriorityColumns, RightPriorityColumns;
		LeftPriorityColumns.Reserve(ColumnTypeNum);
		MiddlePriorityColumns.Reserve(ColumnTypeNum);
		RightPriorityColumns.Reserve(ColumnTypeNum);

		// Sort into the different priority groups
		for (const TWeakObjectPtr<const UScriptStruct>& ColumnType : InColumns)
		{
			if (const FTedsOutlinerColumnParams* FoundColumnParams = InColumnDescription.FindColumnParams(ColumnType))
			{
				switch (FoundColumnParams->PriorityGroup)
				{
				case EColumnPriorityGroup::Left:
					LeftPriorityColumns.Add(ColumnType);
					break;
				case EColumnPriorityGroup::Right:
					RightPriorityColumns.Add(ColumnType);
					break;
				case EColumnPriorityGroup::Middle:
				default:
					MiddlePriorityColumns.Add(ColumnType);
				}
			}
			else
			{
				MiddlePriorityColumns.Add(ColumnType);
			}
		}

		constexpr uint8 DefaultLeftPriorityIndex = 0;
		constexpr uint8 DefaultMiddlePriorityIndex = 85;
		constexpr uint8 DefaultRightPriorityIndex = 170;
		
		bool bColumnsOrderedSuccessfully = true;
		
		bColumnsOrderedSuccessfully &= OrderColumnGroup(LeftPriorityColumns, InColumnDescription, DefaultLeftPriorityIndex, OutOrderedColumnMap);
		bColumnsOrderedSuccessfully &= OrderColumnGroup(MiddlePriorityColumns, InColumnDescription, DefaultMiddlePriorityIndex, OutOrderedColumnMap);
		bColumnsOrderedSuccessfully &= OrderColumnGroup(RightPriorityColumns, InColumnDescription, DefaultRightPriorityIndex, OutOrderedColumnMap);
		return bColumnsOrderedSuccessfully;
	}

	TAttribute<FText> GetHighlightTextAttribute(ICoreProvider* DataStorage, RowHandle WidgetRow)
	{
		const RowHandle OutlinerRow = TableViewerUtils::GetTableViewerUiRow(DataStorage, WidgetRow);
		FAttributeBinder Binder(OutlinerRow, DataStorage);
		return Binder.BindData(&FHighlightTextColumn::HighlightText);
	}
	
	FName FindOutlinerColumnFromTedsColumns(const DataStorage::ICoreProvider* Storage, TConstArrayView<TWeakObjectPtr<const UScriptStruct>> TEDSColumns)
	{
		if(const USceneOutlinerTedsBridgeFactory* Factory = Storage->FindFactory<USceneOutlinerTedsBridgeFactory>())
		{
			return Factory->FindOutlinerColumnFromTedsColumns(TEDSColumns);
		}

		return NAME_None;
	}

	void RemoveTedsColumnMappingToOutlinerColumn(DataStorage::ICoreProvider* Storage, TWeakObjectPtr<const UScriptStruct> TEDSColumnToRemove)
	{
		if(USceneOutlinerTedsBridgeFactory* Factory = Storage->FindFactory<USceneOutlinerTedsBridgeFactory>())
		{
			Factory->RemoveTedsColumnMappingToOutlinerColumn(TEDSColumnToRemove);
		}
	}

	void AddTedsColumnMappingToOutlinerColumn(DataStorage::ICoreProvider* Storage, TWeakObjectPtr<const UScriptStruct> TEDSColumnToAdd, const FName& OutlinerColumnId)
	{
		if(USceneOutlinerTedsBridgeFactory* Factory = Storage->FindFactory<USceneOutlinerTedsBridgeFactory>())
		{
			Factory->AddTedsColumnMappingToOutlinerColumn(TEDSColumnToAdd, OutlinerColumnId);
		}
	}
	
    bool CheckValidFilterQueryHandle(const DataStorage::QueryHandle& InQueryHandle)
    {
		using namespace UE::Editor::DataStorage;
    	// Check if the given Query Handle is valid for a filter (Not an Observer Query Handle)
    	ICoreProvider* Storage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName);
    	if (ensureMsgf(Storage, TEXT("TEDS must be initialized before TEDS Filters")))
    	{
    		const EQueryCallbackType QueryHandleCallbackType = Storage->GetQueryDescription(InQueryHandle).Callback.Type;
    		return ensureMsgf(QueryHandleCallbackType == EQueryCallbackType::None || QueryHandleCallbackType == EQueryCallbackType::Processor,
    			TEXT("TEDS Filters cannot accept Observer Query Handles."));
    	}
    	return false;
    }
    
    void ConvertLegacyFiltersToTedsFilters(FSceneOutlinerFilterBarOptions& LegacyFilters, TArray<TSharedPtr<FTedsOutlinerFilter>>& TedsFilters,
    	const TMap<FName, TVariant<QueryHandle, TConstQueryFunction<bool>>>& FilterConversionMap,
    	TMap<FName, TArray<TSharedPtr<FFilterCategory>>>& OutTypeFilterCategoryMap)
	{
		for (const TSharedRef<FCustomClassFilterData>& LegacyClassFilter : LegacyFilters.CustomClassFilters)
		{
			const TArray<TSharedPtr<FFilterCategory>> Categories = LegacyClassFilter->GetCategories();

			TedsFilters.Add(MakeShared<FTedsOutlinerFilter>(LegacyClassFilter->GetClass(), Categories.IsEmpty() ? nullptr : Categories[0]));
			
			if (Categories.Num() > 1)
			{
				TArray<TSharedPtr<FFilterCategory>>& ExtraCategories = OutTypeFilterCategoryMap.FindOrAdd(LegacyClassFilter->GetClass()->GetFName());
				for (int32 i = 1; i < Categories.Num(); ++i)
				{
					ExtraCategories.AddUnique(Categories[i]);
				}
			}
		}
		LegacyFilters.CustomClassFilters.Empty();

		for (const TSharedRef<FFilterBase<SceneOutliner::FilterBarType>>& CustomFilter : LegacyFilters.CustomFilters)
		{
			const FName FilterName = FName(*CustomFilter->GetName());
			if (const TVariant<QueryHandle, TConstQueryFunction<bool>>* FilterFunction = FilterConversionMap.Find(FilterName))
			{
				if (FilterFunction->IsType<QueryHandle>())
				{
					TedsFilters.Add(MakeShared<FTedsOutlinerFilter>(
					FilterName,
					CustomFilter->GetDisplayName(),
					CustomFilter->GetToolTipText(),
					CustomFilter->GetIconName(),
					CustomFilter->GetCategory(),
					FilterFunction->Get<QueryHandle>()));
				}
				else if (FilterFunction->IsType<TConstQueryFunction<bool>>())
				{
					TedsFilters.Add(MakeShared<FTedsOutlinerFilter>(
						FilterName,
						CustomFilter->GetDisplayName(),
						CustomFilter->GetToolTipText(),
						CustomFilter->GetIconName(),
						CustomFilter->GetCategory(),
						FilterFunction->Get<TConstQueryFunction<bool>>()));
				}
			}
		}
		LegacyFilters.CustomFilters.Empty();
	}
	
	TAttribute<FSlateColor> GetLabelWidgetForegroundColor(TNotNull<DataStorage::ICoreProvider*> DataStorage, DataStorage::RowHandle DataRow, 
    	DataStorage::RowHandle WidgetRow)
    {
		TOptional<FLinearColor> ColorOverride;
		if(DataStorage->HasColumns<DataStorage::FPieObjectTag, FTypedElementActorTag>(DataRow))
		{
			if (const FTypedElementUObjectColumn* ObjectColumn = DataStorage->GetColumn<FTypedElementUObjectColumn>(DataRow))
			{
				if (const UObject* Object = ObjectColumn->Object.Get())
				{
					if (!GEditor->ObjectsThatExistInEditorWorld.Get(Object))
					{
						ColorOverride = FLinearColor(0.9f, 0.8f, 0.4f);
					}
				}
			}
		}
		
    	return MakeAttributeLambda([DataStorage, WidgetRow, DataRow, ColorOverride]() -> FSlateColor
    		{
    			if (ColorOverride.IsSet())
    			{
    				return ColorOverride.GetValue();
    			}
    			
    			if (FLevelInstanceEditingColumn* LevelInstanceEditingColumn = DataStorage->GetColumn<FLevelInstanceEditingColumn>(DataRow))
    			{
    				switch (LevelInstanceEditingColumn->EditMode)
    				{
    				case ILevelInstanceEditorModule::ELevelInstanceEditMode::None:
    					break;
    				case ILevelInstanceEditorModule::ELevelInstanceEditMode::Original:
    					return FAppStyle::Get().GetSlateColor("Colors.AccentGreen").GetSpecifiedColor();
    				case ILevelInstanceEditorModule::ELevelInstanceEditMode::Override:
    					return FAppStyle::Get().GetSlateColor("Colors.AccentBlue").GetSpecifiedColor();
    				default:
    					break;
    				}
    			}

    			if (DataStorage->HasColumns<FInLevelInstanceTag>(DataRow))
    			{
					if (const FTypedElementUObjectColumn* ObjectColumn = DataStorage->GetColumn<FTypedElementUObjectColumn>(DataRow))
					{
						if (const AActor* Actor = Cast<AActor>(ObjectColumn->Object.Get()))
						{
							if (!Actor->SupportsSubRootSelection())
							{
								const RowHandle ParentLevelInstanceRow = LevelInstance::FindContainingLevelInstanceRow(DataStorage, DataRow);
								if (ParentLevelInstanceRow != InvalidRowHandle && !DataStorage->HasColumns<FLevelInstanceEditingColumn>(ParentLevelInstanceRow))
								{
									return FSlateColor(FSceneOutlinerCommonLabelData::DarkColor);
								}
							}
						}
					}
    			}

    			if (const FSceneOutlinerColumn* OutlinerColumn = DataStorage->GetColumn<FSceneOutlinerColumn>(WidgetRow))
    			{
    				if (const TSharedPtr<ISceneOutliner> Outliner = OutlinerColumn->Outliner.Pin())
    				{
    					if (const FSceneOutlinerTreeItemPtr TreeItemPtr = Helpers::GetTreeItemFromRowHandle(DataStorage, Outliner.ToSharedRef(), DataRow))
    					{
    						if (TOptional<FLinearColor> ForegroundColor = FSceneOutlinerCommonLabelData::GetForegroundColor(Outliner, *TreeItemPtr); ForegroundColor.IsSet())
    						{
    							return ForegroundColor.GetValue();
    						}
    					}
    				}
    			}

    			return FSlateColor::UseForeground();
    		});
    	
    }

	RowHandle GetRowHandleFromOutlinerItem(ICoreProvider& DataStorage, const ICompatibilityProvider& Compatibility, const RowHandle OutlinerRow, const ISceneOutlinerTreeItem& Item)
	{
		if (const FTedsOutlinerTreeItem* TedsItem = Item.CastTo<FTedsOutlinerTreeItem>())
		{
			return TedsItem->GetRowHandle();
		}

		if (const FFolderTreeItem* FolderItem = Item.CastTo<FFolderTreeItem>())
		{
			if (const FFolder& Folder = FolderItem->GetFolder(); !Folder.IsNone())
			{
				return ActorFolders::LookupMappedRow(&DataStorage, Folder);
			}

			// If it's an empty folder, treat it as the world row.
			if (const FTypedElementWorldColumn* WorldColumn = DataStorage.GetColumn<FTypedElementWorldColumn>(OutlinerRow))
			{
				if (UWorld* World = WorldColumn->World.Get())
				{
					return Compatibility.FindRowWithCompatibleObject(World);
				}
			}
		}

		if (const FActorTreeItem* ActorItem = Item.CastTo<FActorTreeItem>())
		{
			if (AActor* Actor = ActorItem->Actor.Get())
			{
				return Compatibility.FindRowWithCompatibleObject(Actor);
			}
			return InvalidRowHandle;
		}

		if (const FActorDescTreeItem* ActorDescItem = Item.CastTo<FActorDescTreeItem>())
		{
			if (const FWorldPartitionActorDescInstance* ActorDescInstance = *ActorDescItem->ActorDescHandle)
			{
				return DataStorage.LookupMappedRow(UnloadedActor::GetActorDescMappingDomain(), UnloadedActor::GetActorDescMappingKey(ActorDescInstance));
			}
			return InvalidRowHandle;
		}

		return InvalidRowHandle;
	}

	namespace LevelInstance
	{
		RowHandle FindContainingLevelInstanceRow(const ICoreProvider * DataStorage, RowHandle InRow)
		{
			if (!DataStorage || InRow == InvalidRowHandle)
			{
				return InvalidRowHandle;
			}
			if (!DataStorage->HasColumns<FInLevelInstanceTag>(InRow))
			{
				return InvalidRowHandle;
			}
			const FHierarchyHandle Hierarchy = DataStorage->FindHierarchyByName(UTedsEditorHierarchyFactory::EditorObjectHierarchyName);
			if (!DataStorage->IsValidHierarchyHandle(Hierarchy))
			{
				return InvalidRowHandle;
			}
			RowHandle Current = DataStorage->GetParentRow(Hierarchy, InRow);
			while (Current != InvalidRowHandle)
			{
				if (DataStorage->HasColumns<FLevelInstanceTag>(Current))
				{
					return Current;
				}
				Current = DataStorage->GetParentRow(Hierarchy, Current);
			}
			return InvalidRowHandle;
		}

		bool IsInEditingLevelInstanceHierarchy(const ICoreProvider * DataStorage, RowHandle InRow)
		{
			if (!DataStorage || InRow == InvalidRowHandle)
			{
				return false;
			}
			const FHierarchyHandle Hierarchy = DataStorage->FindHierarchyByName(UTedsEditorHierarchyFactory::EditorObjectHierarchyName);
			if (!DataStorage->IsValidHierarchyHandle(Hierarchy))
			{
				return false;
			}
			RowHandle Current = DataStorage->GetParentRow(Hierarchy, InRow);
			while (Current != InvalidRowHandle)
			{
				if (DataStorage->HasColumns<FLevelInstanceEditingColumn>(Current))
				{
					return true;
				}
				Current = DataStorage->GetParentRow(Hierarchy, Current);
			}
			return false;
		}
	}

}
