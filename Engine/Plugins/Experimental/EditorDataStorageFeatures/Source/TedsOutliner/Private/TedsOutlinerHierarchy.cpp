// Copyright Epic Games, Inc. All Rights Reserved.

#include "TedsOutlinerHierarchy.h"

#include "Elements/Framework/TypedElementQueryBuilder.h"
#include "Elements/Interfaces/TypedElementDataStorageInterface.h"
#include "Elements/Interfaces/TypedElementQueryStorageInterfaces.h"
#include "TedsOutlinerImpl.h"
#include "TedsOutlinerItem.h"
#include "TedsOutlinerMode.h"
#include "Elements/Columns/TypedElementCompatibilityColumns.h"
#include "Elements/Columns/TypedElementHiearchyColumns.h"
#include "Elements/Columns/TypedElementLabelColumns.h"
#include "Elements/Columns/TypedElementMiscColumns.h"
#include "Elements/Columns/TypedElementTypeInfoColumns.h"

namespace UE::Editor::Outliner
{
FTedsOutlinerHierarchy::FTedsOutlinerHierarchy(FTedsOutlinerMode* InMode,
	const TSharedRef<FTedsOutlinerImpl>& InTedsOutlinerImpl)
	: ISceneOutlinerHierarchy(InMode)
	, TedsOutlinerImpl(InTedsOutlinerImpl)
{
	HierarchyChangedHandle = TedsOutlinerImpl->OnHierarchyChanged().AddLambda([this](FSceneOutlinerHierarchyChangedData EventData)
	{
		HierarchyChangedEvent.Broadcast(EventData);
	});

	TedsOutlinerImpl->RecompileQueries();
}

FTedsOutlinerHierarchy::~FTedsOutlinerHierarchy()
{
	TedsOutlinerImpl->OnHierarchyChanged().Remove(HierarchyChangedHandle);
}

void FTedsOutlinerHierarchy::CreateItems(TArray<FSceneOutlinerTreeItemPtr>& OutItems) const
{
	TedsOutlinerImpl->CreateItemsFromQuery(OutItems, Mode);
}

void FTedsOutlinerHierarchy::CreateChildren(const FSceneOutlinerTreeItemPtr& Item,
	TArray<FSceneOutlinerTreeItemPtr>& OutChildren) const
{
	TedsOutlinerImpl->CreateChildren(Item, OutChildren);
}

FSceneOutlinerTreeItemPtr FTedsOutlinerHierarchy::FindOrCreateParentItem(const ISceneOutlinerTreeItem& Item,
	const TMap<FSceneOutlinerTreeItemID, FSceneOutlinerTreeItemPtr>& Items, bool bCreate)
{
	using namespace UE::Editor::DataStorage;
	const FTedsOutlinerTreeItem* TedsTreeItem = Item.CastTo<FTedsOutlinerTreeItem>();
	const ICoreProvider* Storage = TedsOutlinerImpl->GetStorage();
	
	// If this item is not a TEDS item, we are not handling it
	if(!TedsTreeItem)
	{
		return nullptr;
	}
	
	const RowHandle ParentRowHandle = TedsOutlinerImpl->GetParentRow(TedsTreeItem->GetRowHandle());

	if(!Storage->IsRowAvailable(ParentRowHandle))
	{
		return nullptr;
	}
	
	if (const FSceneOutlinerTreeItemPtr* ParentItem = Items.Find(ParentRowHandle))
	{
		return *ParentItem;
	}
	else if(bCreate)
	{
		return Mode->CreateItemFor<FTedsOutlinerTreeItem>(FTedsOutlinerTreeItem(ParentRowHandle, TedsOutlinerImpl), TedsOutlinerImpl->ShouldForceShowParentRows());
	}
	
	return nullptr;
}
} // namespace UE::Editor::Outliner