// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ISceneOutlinerHierarchy.h"
#include "Elements/Interfaces/TypedElementDataStorageInterface.h"

namespace UE::Editor::Outliner
{
class FTedsOutlinerMode;
class FTedsOutlinerImpl;

/*
 * Class that keeps track of hierarchy data and creates items using the given TEDS queries.
 * See CreateGenericTEDSOutliner()
 * Inherits from ISceneOutlinerHierarchy, which is responsible for creating the items you want to populate the Outliner
 * with and establishing hierarchical relationships between the items
 */
class FTedsOutlinerHierarchy : public ISceneOutlinerHierarchy
{
public:
	FTedsOutlinerHierarchy(FTedsOutlinerMode* InMode, const TSharedRef<FTedsOutlinerImpl>& InTedsOutlinerImpl);
	virtual ~FTedsOutlinerHierarchy();
	
	/** Create a linearization of all applicable items in the hierarchy */
	virtual void CreateItems(TArray<FSceneOutlinerTreeItemPtr>& OutItems) const override;
	/** Create a linearization of all direct and indirect children of a given item in the hierarchy */
	virtual void CreateChildren(const FSceneOutlinerTreeItemPtr& Item, TArray<FSceneOutlinerTreeItemPtr>& OutChildren) const override;
	/** Find or optionally create a parent item for a given tree item */
	virtual FSceneOutlinerTreeItemPtr FindOrCreateParentItem(const ISceneOutlinerTreeItem& Item, const TMap<FSceneOutlinerTreeItemID,
		FSceneOutlinerTreeItemPtr>& Items, bool bCreate = false) override;

protected:

	// The actual model for the TEDS-Outliner
	TSharedRef<FTedsOutlinerImpl> TedsOutlinerImpl;

	// Delegate called by TedsOutlinerImpl when the hierarchy changes
	FDelegateHandle HierarchyChangedHandle;
};
} // namespace UE::Editor::Outliner
