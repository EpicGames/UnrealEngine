// Copyright Epic Games, Inc. All Rights Reserved.

#include "TedsOutlinerHierarchyInterfaces.h"

#include "TedsOutlinerItem.h"
#include "DataStorage/Features.h"
#include "Elements/Columns/TypedElementMiscColumns.h"
#include "Elements/Framework/TypedElementQueryBuilder.h"

namespace UE::Editor::Outliner
{
	namespace Private
	{
		// Helper method to check if the parent of an item has changed, used with the legacy interface which used sync tags to detect hierarchy changes
		// but weren't fully accurate because sync tags are added on ANY changes to the UObject row
		static bool HasItemParentChanged(DataStorage::IQueryContext& Context, DataStorage::RowHandle Row, DataStorage::RowHandle ParentRowHandle, ISceneOutliner& SceneOutliner)
		{
			const FSceneOutlinerTreeItemPtr Item = SceneOutliner.GetTreeItem(Row, true);

			// If the item doesn't exist, it doesn't make sense to say its parent changed
			if (!Item)
			{
				return false;
			}
										
			const FSceneOutlinerTreeItemPtr ParentItem = Item->GetParent();

			// If the item doesn't have a parent, but ParentRowHandle is valid: The item just got added a parent so we want to dirty it
			if (!ParentItem)
			{
				return Context.IsRowAvailable(ParentRowHandle);
			}
										
			const FTedsOutlinerTreeItem* TedsParentItem = ParentItem->CastTo<FTedsOutlinerTreeItem>();

			if (TedsParentItem)
			{
				// return true if the row handle of the parent item doesn't match what we are given, i.e the parent has changed
				return TedsParentItem->GetRowHandle() != ParentRowHandle;
			}

			return false;
		};
	}

	// @section FTedsOutlinerMultiHierarchyInterface

	void ITedsOutlinerHierarchyDataInterface::ForEachImmediateParent(const DataStorage::ICoreProvider& Storage, DataStorage::RowHandle InRow,
		FParentIterationCallback Callback)
	{
		if (Callback.IsSet())
		{
			RowHandle ParentRow = GetParent(Storage, InRow);
		
			if (Storage.IsRowAvailable(ParentRow))
			{
				Callback(Storage, ParentRow);
			}
		}
	}

	FTedsOutlinerMultiHierarchyInterface::FTedsOutlinerMultiHierarchyInterface(TSharedRef<DataStorage::FHierarchyViewerMultiData> InHierarchyData)
		: HierarchyData(InHierarchyData)
	{
		
	}

	DataStorage::RowHandle FTedsOutlinerMultiHierarchyInterface::GetParent(const DataStorage::ICoreProvider& Storage,
		DataStorage::RowHandle InRow) const
	{
		return HierarchyData->GetParent(Storage, InRow);
	}

	void FTedsOutlinerMultiHierarchyInterface::WalkDepthFirst(const DataStorage::ICoreProvider& Storage, DataStorage::RowHandle InRow,
		DataStorage::ICoreProvider::FHierarchyIterationCallback VisitFn, DataStorage::ICoreProvider::ETraversalOrder TraversalOrder) const
	{
		HierarchyData->WalkDepthFirst(Storage, InRow, VisitFn, TraversalOrder);
	}

	void FTedsOutlinerMultiHierarchyInterface::RegisterQueries(DataStorage::ICoreProvider& Storage,
		const DataStorage::FQueryDescription& OutlinerQueryDescription, TWeakPtr<ISceneOutliner> Outliner, bool bUsingQueryConditionsSyntax)
	{
		// For each hierarchy we are displaying, get the parent change column (if present) and add a processor to detect changes to rows with the change column
		HierarchyData->ForEachHierarchyData([this, &Storage](const TSharedRef<FHierarchyViewerData>& HierarchyViewerData)
		{
			if (const UScriptStruct* ParentChangeTag = Storage.GetParentChangedColumnType(HierarchyViewerData->GetHierarchy()))
			{
				FString ProcessorName = TEXT("FTedsOutlinerMultiHierarchyInterface: Update Parent for ");
				ParentChangeTag->GetFName().AppendString(ProcessorName);
				
				FQueryDescription UpdateParentQueryDescription =
					Select(
					FName(ProcessorName),
					FProcessor(EQueryTickPhase::PostPhysics, Storage.GetQueryTickGroupName(EQueryTickGroups::SyncExternalToDataStorage))
						.SetExecutionMode(EExecutionMode::GameThread),
					[this](IQueryContext& Context, const RowHandle* Rows)
					{
						const int32 RowCount = Context.GetRowCount();
						TConstArrayView<RowHandle> RowsView = MakeArrayView(Rows, RowCount);
							
						for (RowHandle Row : RowsView)
						{
							ParentChangedEvent.Broadcast(Row);
						}
					})
				.Where()
					.All(ParentChangeTag)
				.Compile();
				
				HierarchyChangeQueries.Add(Storage.RegisterQuery(MoveTemp(UpdateParentQueryDescription)));
			}

			return true;
		});
	}

	void FTedsOutlinerMultiHierarchyInterface::UnregisterQueries(DataStorage::ICoreProvider& Storage)
	{
		for (DataStorage::QueryHandle Query : HierarchyChangeQueries)
		{
			Storage.UnregisterQuery(Query);
		}
		HierarchyChangeQueries.Empty();
	}

	void FTedsOutlinerMultiHierarchyInterface::ForEachImmediateParent(const DataStorage::ICoreProvider& Storage, DataStorage::RowHandle InRow,
		FParentIterationCallback Callback)
	{
		if (Callback.IsSet())
		{
			HierarchyData->ForEachHierarchyData([InRow, Callback, &Storage](const TSharedRef<FHierarchyViewerData>& HierarchyViewerData) -> bool
			{
				RowHandle ParentRow = HierarchyViewerData->GetParent(Storage, InRow);
				
				if (Storage.IsRowAvailable(ParentRow))
				{
					return Callback(Storage, ParentRow);
				}

				return true;
			});
		}
	}

	// @section FTedsOutlinerHierarchyData
	
	FTedsOutlinerHierarchyData FTedsOutlinerHierarchyData::GetDefaultHierarchyData()
	{
		const FGetParentRowHandle RowHandleGetter = FGetParentRowHandle::CreateLambda([](const void* InColumnData)
			{
				if(const FTableRowParentColumn* ParentColumn = static_cast<const FTableRowParentColumn *>(InColumnData))
				{
					return ParentColumn->Parent;
				}
    
				return DataStorage::InvalidRowHandle;
			});
    
		const FSetParentRowHandle RowHandleSetter = FSetParentRowHandle::CreateLambda([](void* InColumnData,
			DataStorage::RowHandle InRowHandle)
			{
				if(FTableRowParentColumn* ParentColumn = static_cast<FTableRowParentColumn *>(InColumnData))
				{
					ParentColumn->Parent = InRowHandle;
				}
			});
    		
		return FTedsOutlinerHierarchyData(FTableRowParentColumn::StaticStruct(), RowHandleGetter, RowHandleSetter, FGetChildrenRowsHandles());
	}
	
	// @section FTedsOutlinerLegacyHierarchyInterface

	FTedsOutlinerLegacyHierarchyInterface::FTedsOutlinerLegacyHierarchyInterface(FTedsOutlinerHierarchyData InHierarchyData)
		: HierarchyData(InHierarchyData)
	{
	}

	DataStorage::RowHandle FTedsOutlinerLegacyHierarchyInterface::GetParent(const DataStorage::ICoreProvider& Storage,
		DataStorage::RowHandle InRow) const
	{
		if (HierarchyData.GetParent.IsBound())
		{
			if (const void* HierarchyColumnData = Storage.GetColumnData(InRow, HierarchyData.HierarchyColumn))
			{
				return HierarchyData.GetParent.Execute(HierarchyColumnData);
			}
		}
		
		return DataStorage::InvalidRowHandle;
		
	}

	void FTedsOutlinerLegacyHierarchyInterface::WalkDepthFirst(const DataStorage::ICoreProvider& Storage, DataStorage::RowHandle InRow,
		DataStorage::ICoreProvider::FHierarchyIterationCallback VisitFn, DataStorage::ICoreProvider::ETraversalOrder TraversalOrder) const
	{
		TArray<DataStorage::RowHandle> Children;
		
		if (HierarchyData.GetChildren.IsBound())
		{
			if (const void* HierarchyColumnData = Storage.GetColumnData(InRow, HierarchyData.HierarchyColumn))
			{
				Children = HierarchyData.GetChildren.Execute(const_cast<void*>(HierarchyColumnData));
			}
		}
		// We still have to support the legacy case of grabbing child rows manually since it is still in use by places that don't specify HierarchyData.GetChildren
		// This runs a query to get all rows with FTableRowParentColumn and recurses through the tree to find rows that are children of InRow
		// NOTE: This is extremely expensive and only exists for backward compatibility
		else if (HierarchyData.GetParent.IsBound())
		{
			FRowHandleArray MatchedRowsWithParentColumn;

			// Collect all rows with the parent column
			DirectQueryCallback ChildRowCollector = CreateDirectQueryCallbackBinding(
			[&MatchedRowsWithParentColumn] (const IDirectQueryContext& Context, const RowHandle*)
			{
				MatchedRowsWithParentColumn.Append(Context.GetRowHandles());
			});

			// The legacy case relies on running a query inline, but we don't want to keep supporting that in the new cases so we do a const cast here
			const_cast<ICoreProvider&>(Storage).RunQuery(GetChildrenQuery, ChildRowCollector);

			// Recursively get the children for each row
			TFunction<void(RowHandle)> GetChildrenRecursive = 
				[&Children, &MatchedRowsWithParentColumn, &Storage, &GetChildrenRecursive, InHierarchyData = HierarchyData]
				(RowHandle ExpectedParentRow) -> void
			{
				for(RowHandle PotentialChildRow : MatchedRowsWithParentColumn.GetRows())
				{
					const void* ParentColumnData = Storage.GetColumnData(PotentialChildRow, InHierarchyData.HierarchyColumn);

					if (ensureMsgf(ParentColumnData, TEXT("We should always have the parent column since we only grabbed rows with those ")))
					{
						// Get the parent row handle
						const RowHandle ParentRowHandle = InHierarchyData.GetParent.Execute(ParentColumnData);
			
						// Check if this row is owned by InRow
						if (ParentRowHandle == ExpectedParentRow)
						{
							Children.Add(PotentialChildRow);

							// Recursively look for children of this row now
							GetChildrenRecursive(PotentialChildRow);
						}
					}
				}
			};

			GetChildrenRecursive(InRow);
		}
		
		for (DataStorage::RowHandle ChildRow : Children)
		{
			VisitFn(Storage, InRow, ChildRow);
		}
	}

	void FTedsOutlinerLegacyHierarchyInterface::RegisterQueries(DataStorage::ICoreProvider& Storage, 
		const DataStorage::FQueryDescription& OutlinerQueryDescription, TWeakPtr<ISceneOutliner> Outliner, bool bUsingQueryConditionsSyntax)
	{
		using namespace UE::Editor::DataStorage;
		using namespace UE::Editor::DataStorage::Queries;
		
		// Legacy query that uses the sync tags to detect changes to a row + a util function to determine if that change was actually a hierarchy change
		FQueryDescription UpdateParentQueryDescription =
			Select(
			TEXT("FTedsOutlinerLegacyHierarchyInterface: Update item parent"),
			FProcessor(EQueryTickPhase::PostPhysics, Storage.GetQueryTickGroupName(EQueryTickGroups::SyncExternalToDataStorage))
				.SetExecutionMode(EExecutionMode::GameThread),
			[this, Outliner](IQueryContext& Context, const RowHandle* Rows)
			{
				if (TSharedPtr<ISceneOutliner> OutlinerPin = Outliner.Pin())
				{
					if (const char* ParentColumn = reinterpret_cast<const char*>(Context.GetColumn(HierarchyData.HierarchyColumn)))
					{
						int32 ColumnSize = HierarchyData.HierarchyColumn->GetCppStructOps() 
							? HierarchyData.HierarchyColumn->GetCppStructOps()->GetSize() 
							: HierarchyData.HierarchyColumn->GetStructureSize();
					
						uint32 RowCount = Context.GetRowCount();
					
						for(uint32 RowIndex = 0; RowIndex < RowCount; ++RowIndex, ParentColumn += ColumnSize)
						{
							RowHandle ParentRowHandle = HierarchyData.GetParent.Execute(ParentColumn);
								
							if (Private::HasItemParentChanged(Context, Rows[RowIndex], ParentRowHandle, *OutlinerPin))
							{
								ParentChangedEvent.Broadcast(Rows[RowIndex]);
							}
						}
					}
				}
				
			})
			.ReadOnly(HierarchyData.HierarchyColumn, EOptional::Yes)
		.Compile();

		if (bUsingQueryConditionsSyntax)
		{
			UpdateParentQueryDescription.Conditions.Emplace(TColumn<FTypedElementSyncBackToWorldTag>());
		}
		else
		{
			UpdateParentQueryDescription.ConditionTypes.Add(FQueryDescription::EOperatorType::SimpleAll);
			UpdateParentQueryDescription.ConditionOperators.AddZeroed_GetRef().Type = FTypedElementSyncBackToWorldTag::StaticStruct();
		}
		
		// Add the conditions from FinalQueryDescription to ensure we are only observing the rows the user requested
		MergeQueries(UpdateParentQueryDescription, OutlinerQueryDescription);
		
		HierarchyChangeQuery = Storage.RegisterQuery(MoveTemp(UpdateParentQueryDescription));
		
		if (!HierarchyData.GetChildren.IsBound())
		{
			// Query to get all rows that match our conditions with a parent column (i.e all child rows)
			FQueryDescription ChildHandleQueryDescription;
		
			if (bUsingQueryConditionsSyntax)
			{
				ChildHandleQueryDescription = Select()
					.Where(TColumn(HierarchyData.HierarchyColumn))
					.Compile();
			}
			else
			{
				ChildHandleQueryDescription =
					Select()
					.Where()
						.All(HierarchyData.HierarchyColumn)
					.Compile();
			}
		
			MergeQueries(ChildHandleQueryDescription, OutlinerQueryDescription);
			GetChildrenQuery = Storage.RegisterQuery(MoveTemp(ChildHandleQueryDescription));
		}
	}

	void FTedsOutlinerLegacyHierarchyInterface::UnregisterQueries(DataStorage::ICoreProvider& Storage)
	{
		Storage.UnregisterQuery(HierarchyChangeQuery);
		Storage.UnregisterQuery(GetChildrenQuery);
		HierarchyChangeQuery = DataStorage::InvalidQueryHandle;
		GetChildrenQuery = DataStorage::InvalidQueryHandle;
	}
}
