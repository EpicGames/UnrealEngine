// Copyright Epic Games, Inc. All Rights Reserved.

#include "TedsOutlinerImpl.h"

#include "SSceneOutliner.h"
#include "TedsOutlinerColumn.h"
#include "TedsTableViewerUtils.h"
#include "TedsQueryStackExecutor.h"
#include "DataStorage/Features.h"
#include "Elements/Columns/TypedElementLabelColumns.h"
#include "Elements/Columns/TypedElementMiscColumns.h"
#include "Elements/Columns/TypedElementTypeInfoColumns.h"
#include "Elements/Interfaces/TypedElementDataStorageInterface.h"
#include "Elements/Interfaces/TypedElementDataStorageCompatibilityInterface.h"
#include "Elements/Interfaces/TypedElementDataStorageUiInterface.h"
#include "Elements/Framework/TypedElementQueryBuilder.h"
#include "Columns/SlateDelegateColumns.h"
#include "Compatibility/SceneOutlinerRowHandleColumn.h"
#include "RowMapNode.h"
#include "TedsOutlinerFilter.h"
#include "TedsOutlinerHelpers.h"
#include "TedsOutlinerItem.h"
#include "TedsQueryHandleNode.h"
#include "TedsQueryMergeNode.h"
#include "TedsQueryNode.h"
#include "TedsRowCopyNode.h"
#include "TedsRowFilterNode.h"
#include "TedsRowHandleSortNode.h"
#include "TedsRowMergeNode.h"
#include "TedsRowMonitorNode.h"
#include "TedsRowOrderInversionNode.h"
#include "TedsRowQueryResultsNode.h"
#include "TedsRowSortNode.h"
#include "TedsTableViewerWidgetColumns.h"
#include "Columns/SceneOutlinerColumns.h"
#include "Columns/TedsOutlinerColumns.h"
#include "Compatibility/SceneOutlinerTedsBridge.h"
#include "Compatibility/TedsSceneOutlinerItemLabelColumn.h"
#include "Elements/Columns/TypedElementCompatibilityColumns.h"
#include "Elements/Columns/TypedElementHandleColumn.h"
#include "Elements/Columns/TypedElementOverrideColumns.h"
#include "Elements/Columns/TypedElementPackageColumns.h"
#include "Elements/Columns/TypedElementSelectionColumns.h"
#include "Elements/Columns/TypedElementSlateWidgetColumns.h"
#include "Elements/Columns/TypedElementUIColumns.h"
#include "Filters/FilterBase.h"
#include "Widgets/SWidget.h"

#define LOCTEXT_NAMESPACE "TedsOutliner"

namespace UE::Editor::Outliner
{
namespace Private
{
	static bool bShowTedsColumnFilters = false;
	static constexpr int32 MaxHierarchyWalkIterations = 10000;
} // namespace Private

static FAutoConsoleVariableRef CVarShowTedsColumnFilters(
	TEXT("TEDS.UI.Outliner.ShowTedsColumnFilters"),
	Private::bShowTedsColumnFilters,
	TEXT("Show Teds Column/Tag Filters in the Outliner (requires TEDS Outliner to be enabled and the Outliner to be reopened)"));
	
namespace QueryUtils
{
	static bool CanDisplayRow(DataStorage::IQueryContext& Context, const FSceneOutlinerColumn& TedsOutlinerColumn, DataStorage::RowHandle Row, SSceneOutliner& SceneOutliner)
	{
		/*
		 * Don't display widgets that are created for rows in this table viewer. Widgets are only created for rows that are currently visible, so if we
		 * display the rows for them we are now adding/removing rows to the table viewer based on currently visible rows. But adding rows can cause
		 * scrolling and change the currently visible rows which in turn again adds/removes widget rows. This chain keeps continuing which can cause
		 * flickering/scrolling issues in the table viewer.
		 */
		if (Context.HasColumn<FTypedElementSlateWidgetReferenceColumn>(Row))
		{
			// Check if this widget row belongs to the same table viewer it is being displayed in
			if (const TSharedPtr<ISceneOutliner> TableViewer = TedsOutlinerColumn.Outliner.Pin())
			{
				return &SceneOutliner != TableViewer.Get();
			}
			
		}
		return true;
	}

	
}

namespace LabelWidgetUtils
{
	static IUiProvider::FWidgetConstructorPtr CreateLabelWidgetConstructorInternal(
			const DataStorage::ICoreProvider& Storage,
			DataStorage::IUiProvider& StorageUi,
			const DataStorage::IUiProvider::FPurposeID& Purpose,
			const DataStorage::RowHandle TargetRow)
	{
		// Get all the columns on the given row
		TArray<TWeakObjectPtr<const UScriptStruct>> ColumnTypes;
		Storage.ListColumns(TargetRow, [&ColumnTypes](const UScriptStruct& ColumnType)
		{
			ColumnTypes.Add(&ColumnType);
		});
	
		IUiProvider::FWidgetConstructorPtr WidgetConstructor;
		StorageUi.CreateWidgetConstructors(StorageUi.FindPurpose(Purpose), DataStorage::IUiProvider::EMatchApproach::LongestMatch, ColumnTypes, {},
			[&WidgetConstructor](IUiProvider::FWidgetConstructorPtr InWidgetConstructor, TConstArrayView<TWeakObjectPtr<const UScriptStruct>>)
			{
				WidgetConstructor = MoveTemp(InWidgetConstructor);
				// Either this was the exact match so no need to search further or the longest possible chain didn't match so the next ones will 
				// always be shorter in both cases just return.
				return false;
			});
		return WidgetConstructor;
	}
	
	TSharedPtr<SWidget> CreateLabelWidgetInternal(
		FTypedElementWidgetConstructor& WidgetConstructor,
		DataStorage::ICoreProvider& Storage,
		DataStorage::IUiProvider& StorageUi,
		const DataStorage::RowHandle TargetRow,
		const DataStorage::RowHandle OutlinerWidgetRow,
		ISceneOutlinerTreeItem& TreeItem,
		const STableRow<FSceneOutlinerTreeItemPtr>& RowItem,
		const bool bInteractable)
	{
		using namespace DataStorage::Queries;
		// Query description to pass as metadata to allow the label column to be writable
		static const FQueryDescription MetaDataQueryReadWrite = Select().ReadWrite<FTypedElementLabelColumn>().Where().Compile();
		static const FQueryDescription MetaDataQueryRead = Select().ReadOnly<FTypedElementLabelColumn>().Where().Compile();
	
		RowHandle WidgetRow = Storage.AddRow(Storage.FindTable(TableViewerUtils::GetWidgetTableName()));

		checkf(WidgetRow != InvalidRowHandle, TEXT("Expected valid row handle!"));

		FHierarchyHandle Hierarchy = Storage.FindHierarchyByName(TableViewerUtils::GetWidgetHierarchyName());
		Storage.SetParentRow(Hierarchy, WidgetRow, OutlinerWidgetRow);

		if (FTypedElementRowReferenceColumn* RowReference = Storage.GetColumn<FTypedElementRowReferenceColumn>(WidgetRow))
		{
			RowReference->Row = TargetRow;
		}
		Storage.AddColumn(WidgetRow, FSceneOutlinerColumn{ .Outliner = TreeItem.WeakSceneOutliner });
	
		// Create metadata for the query
		FQueryMetaDataView QueryMetaDataView = bInteractable ? FQueryMetaDataView(MetaDataQueryReadWrite) : FQueryMetaDataView(MetaDataQueryRead);
		TSharedPtr<SWidget> Widget = StorageUi.ConstructWidget(WidgetRow, WidgetConstructor, QueryMetaDataView);
		if (Widget)
		{
			Storage.AddColumn<FExternalWidgetSelectionColumn>(WidgetRow, FExternalWidgetSelectionColumn{
				.IsSelected = MakeShared<FIsSelected>(FIsSelected::CreateSP(&RowItem, &STableRow<FSceneOutlinerTreeItemPtr>::IsSelected))
			});

			if (bInteractable)
			{
				TreeItem.RenameRequestEvent.BindLambda([StoragePtr = &Storage, WidgetRow]()
					{
						if (const FWidgetEnterEditModeColumn* Column = StoragePtr->GetColumn<FWidgetEnterEditModeColumn>(WidgetRow))
						{
							if (Column->OnEnterEditMode)
							{
								Column->OnEnterEditMode->ExecuteIfBound();
							}
						}
					});
			}
		}
		else
		{
			Storage.RemoveRow(WidgetRow);
		}	
		return Widget;
	}
}

FTedsOutlinerColumnParams::FColumnPriorityRelation::FColumnPriorityRelation(const EColumnPriorityRelation InPriorityRelation,
	const TWeakObjectPtr<const UScriptStruct>& InRelatedColumn)
	: PriorityRelation(InPriorityRelation), RelatedColumn(InRelatedColumn)
{}

FTedsOutlinerColumnParams::FTedsOutlinerColumnParams(const ESceneOutlinerColumnVisibility InInitialVisibility,
	const EColumnPriorityGroup InPriorityGroup, const FColumnPriorityRelation& InPriorityRelation)
	: InitialVisibility(InInitialVisibility), PriorityGroup(InPriorityGroup), PriorityRelation(InPriorityRelation)
{}

FTedsOutlinerColumnParams::FTedsOutlinerColumnParams(const EColumnPriorityGroup InPriorityGroup,
	const FColumnPriorityRelation& InPriorityRelation)
		: FTedsOutlinerColumnParams(ESceneOutlinerColumnVisibility::Visible, InPriorityGroup, InPriorityRelation)
{}

FTedsOutlinerColumnDescription::FTedsOutlinerColumnDescription(const TArray<TWeakObjectPtr<const UScriptStruct>>& InColumns,
	const TMap<TWeakObjectPtr<const UScriptStruct>, FTedsOutlinerColumnParams>& InColumnParams,
	const TArray<TSharedPtr<FColumnMetaData>>& InColumnMetaData, const FMetaData& InGenericMetaData)
	: Columns(InColumns)
	, ColumnParams(InColumnParams)
	, GenericMetaData(InGenericMetaData)
{
	ColumnMetaData.Reserve(Columns.Num());

	// Initialize the ColumnMetaData with ReadOnly MetaData for each column if nothing was given. Custom MetaData
	// can override this by passing through the constructor.
	if (InColumnMetaData.IsEmpty())
	{
		for (const TWeakObjectPtr<const UScriptStruct>& Column : InColumns)
		{
			if (Column.IsValid())
			{
				ColumnMetaData.Add(MakeShared<FColumnMetaData>(Column.Get(), FColumnMetaData::EFlags::None));
			}
		}
	}
	else
	{
		ColumnMetaData = InColumnMetaData;
	}
}
	
FTedsOutlinerColumnDescription::FTedsOutlinerColumnDescription(const TArray<TWeakObjectPtr<const UScriptStruct>>& InColumns, const FMetaData& InGenericMetaData)
	: FTedsOutlinerColumnDescription(InColumns, {}, {}, InGenericMetaData)
{}

const TArray<TWeakObjectPtr<const UScriptStruct>>& FTedsOutlinerColumnDescription::GetColumns() const
{
	return Columns;
}

TArray<TWeakObjectPtr<const UScriptStruct>> FTedsOutlinerColumnDescription::GetEffectiveColumns() const
{
	if (ExcludedColumns.IsEmpty())
	{
		return Columns;
	}

	TArray<TWeakObjectPtr<const UScriptStruct>> Result;
	Result.Reserve(Columns.Num());
	for (const TWeakObjectPtr<const UScriptStruct>& Column : Columns)
	{
		if (!ExcludedColumns.Contains(Column))
		{
			Result.Add(Column);
		}
	}
	return Result;
}

const FTedsOutlinerColumnParams* FTedsOutlinerColumnDescription::FindColumnParams(const TWeakObjectPtr<const UScriptStruct>& InColumn) const
{
	return ColumnParams.Find(InColumn);
}

FTedsOutlinerColumnParams& FTedsOutlinerColumnDescription::FindOrAddColumnParams(const TWeakObjectPtr<const UScriptStruct>& InColumn)
{
	return ColumnParams.FindOrAdd(InColumn);
}

const TArray<TSharedPtr<FColumnMetaData>>& FTedsOutlinerColumnDescription::GetColumnMetaData() const
{
	return ColumnMetaData;
}

FMetaData FTedsOutlinerColumnDescription::GetGenericMetaData() const
{
	return GenericMetaData;
}

bool FTedsOutlinerColumnDescription::ExcludeColumns(TConstArrayView<TWeakObjectPtr<const UScriptStruct>> InColumns)
{
	bool bChanged = false;
	for (const TWeakObjectPtr<const UScriptStruct>& Column : InColumns)
	{
		bool bAlreadyExcluded = false;
		ExcludedColumns.Add(Column, &bAlreadyExcluded);
		bChanged |= !bAlreadyExcluded;
	}
	return bChanged;
}

bool FTedsOutlinerColumnDescription::IncludeColumns(TConstArrayView<TWeakObjectPtr<const UScriptStruct>> InColumns)
{
	bool bChanged = false;
	for (const TWeakObjectPtr<const UScriptStruct>& Column : InColumns)
	{
		bChanged |= (ExcludedColumns.Remove(Column) > 0);
	}
	return bChanged;
}

const TSet<TWeakObjectPtr<const UScriptStruct>>& FTedsOutlinerColumnDescription::GetExcludedColumns() const
{
	return ExcludedColumns;
}

FTedsOutlinerParams::FTedsOutlinerParams(SSceneOutliner* InSceneOutliner)
	: SceneOutliner(InSceneOutliner)
	, QueryDescription()
	, bShowRowHandleColumn(true)
	, bForceShowParents(true)
	, bUseDefaultObservers(true)
	, HierarchyData(MakeShared<FTedsOutlinerLegacyHierarchyInterface>(FTedsOutlinerHierarchyData::GetDefaultHierarchyData()))
	, SelectionSet(nullptr)
{
	CellWidgetPurpose =
		DataStorage::IUiProvider::FPurposeID(DataStorage::IUiProvider::FPurposeInfo(
			"SceneOutliner", "Cell", NAME_None).GeneratePurposeID());
	
	HeaderWidgetPurpose =
		DataStorage::IUiProvider::FPurposeID(DataStorage::IUiProvider::FPurposeInfo(
			"SceneOutliner", "Header", NAME_None).GeneratePurposeID());

	LabelWidgetPurpose =
		DataStorage::IUiProvider::FPurposeID(DataStorage::IUiProvider::FPurposeInfo(
			"SceneOutliner", "RowLabel", NAME_None).GeneratePurposeID());
}
	
FTedsOutlinerImpl::FTedsOutlinerImpl(const FTedsOutlinerParams& InParams, ISceneOutlinerMode* InMode, bool bInHybridMode)
	: CreationParams(InParams)
	, CellWidgetPurpose(InParams.CellWidgetPurpose)
	, HeaderWidgetPurpose(InParams.HeaderWidgetPurpose)
	, LabelWidgetPurpose(InParams.LabelWidgetPurpose)
	, InitialQueryDescription(InParams.QueryDescription)
	, HierarchyData(InParams.HierarchyData)
	, SelectionSet(InParams.SelectionSet)
	, bForceShowParents(InParams.bForceShowParents)
	, SceneOutlinerMode(InMode)
	, SceneOutliner(InParams.SceneOutliner)
	, bHybridMode(bInHybridMode)
	, ExpansionStateBridge(InParams.ExpansionStateBridge)
{
	// Initialize the TEDS constructs
	using namespace UE::Editor::DataStorage;
	Storage = GetMutableDataStorageFeature<DataStorage::ICoreProvider>(StorageFeatureName);
	StorageUi = GetMutableDataStorageFeature<DataStorage::IUiProvider>(UiFeatureName);
	StorageCompatibility = GetMutableDataStorageFeature<DataStorage::ICompatibilityProvider>(CompatibilityFeatureName);
	
	bUsingQueryConditionsSyntax = InitialQueryDescription.Conditions && !InitialQueryDescription.Conditions->IsEmpty();
}

void FTedsOutlinerImpl::CreateFilterQueries()
{
	using namespace UE::Editor::DataStorage::Queries;

	if (Private::bShowTedsColumnFilters)
	{
		// Create separate categories for columns and tags
		TSharedRef<FFilterCategory> TedsColumnFilterCategory = MakeShared<FFilterCategory>(LOCTEXT("TedsColumnFilters", "TEDS Columns"), LOCTEXT("TedsColumnFiltersTooltip", "Filter by TEDS columns"));
		TSharedRef<FFilterCategory> TedsTagFilterCategory = MakeShared<FFilterCategory>(LOCTEXT("TedsTagFilters", "TEDS Tags"), LOCTEXT("TedsTagFiltersTooltip", "Filter by TEDS Tags"));

		const UStruct* TedsColumn = DataStorage::FColumn::StaticStruct();
		const UStruct* TedsTag = DataStorage::FTag::StaticStruct();

		// Grab all UStruct types to see if they derive from FColumn or FTag
		ForEachObjectOfClass(UScriptStruct::StaticClass(), [&](UObject* Obj)
		{
			if (const UScriptStruct* Struct = Cast<const UScriptStruct>(Obj))
			{
				if (Struct->IsChildOf(TedsColumn) || Struct->IsChildOf(TedsTag))
				{
					// Create a query description to filter for this tag/column
					QueryHandle FilterQuery;

					if (bUsingQueryConditionsSyntax)
					{
						FilterQuery = Storage->RegisterQuery(
							Select()
							.Where(TColumn(Struct))
							.Compile());
					}
					else
					{
						FilterQuery = Storage->RegisterQuery(
							Select()
							.Where()
								.All(Struct)
							.Compile());
					}

					// Create the filter
					TSharedRef<FTedsOutlinerFilter> TedsFilter = MakeShared<FTedsOutlinerFilter>(Struct->GetFName(), Struct->GetDisplayNameText(), 
						Struct->GetDisplayNameText(), FName(), Struct->IsChildOf(TedsColumn) ? TedsColumnFilterCategory : TedsTagFilterCategory, 
						FilterQuery, true);
					TedsFilter->SetSceneOutlinerImpl(AsShared());
					SceneOutliner->AddFilterToFilterBar(TedsFilter);
				}
			}
		});
	}

	const TSharedRef<FFilterCategory> CustomClassFiltersCategory = MakeShared<FFilterCategory>(LOCTEXT("CustomTypeFilters", "Custom Type Filters"), LOCTEXT("CustomTypeFiltersTooltip", "Filter by custom class types"));
	const TSharedRef<FFilterCategory> CustomFiltersCategory = MakeShared<FFilterCategory>(LOCTEXT("TedsFilters", "TEDS Custom Filters"), LOCTEXT("TedsFiltersTooltip", "Filter by custom TEDS queries"));
		
	for(TSharedPtr<FTedsOutlinerFilter>& Filter : CreationParams.Filters)
	{
		if (Filter)
		{
			if (!Filter->GetCategory())
			{
				if(Filter->IsClassFilter())
				{
					Filter->SetCategory(CustomClassFiltersCategory);
				}
				else
				{
					Filter->SetCategory(CustomFiltersCategory);
				}
			}

			Filter->SetSceneOutlinerImpl(AsShared());
			if (Filter->IsInteractiveFilter())
			{
				if (Filter->IsClassFilter())
				{
					const TArray<TSharedPtr<FFilterCategory>>* ExtraCategories =
							CreationParams.FilterCategoryMap.Find(Filter->GetIdentifier());
					SceneOutliner->AddTypeFilterToFilterBar(Filter.ToSharedRef(), ExtraCategories ? *ExtraCategories : TArray<TSharedPtr<FFilterCategory>>());
				}
				else
				{
					SceneOutliner->AddFilterToFilterBar(Filter.ToSharedRef());
				}
			}
			else
			{
				Filter->ActiveStateChanged(true);
			}
		}
	}
}

void FTedsOutlinerImpl::Init()
{
	if (HierarchyData)
	{
		HierarchyData->OnHierarchyChangedEvent().AddSPLambda(this, [this](RowHandle TargetRow)
		{
			FSceneOutlinerHierarchyChangedData EventData;
			EventData.Type = FSceneOutlinerHierarchyChangedData::Moved;
			EventData.ItemIDs.Add(TargetRow);
			HierarchyChangedEvent.Broadcast(EventData);
		});
	}
	
	RegisterTedsOutliner();
	CreateFilterQueries();
	RecompileQueries();

	// Tick post TEDS update to make sure all processors have run and the data is correct
	Storage->OnUpdateCompleted().AddRaw(this, &FTedsOutlinerImpl::PostDataStorageUpdate);

	// FTedsOutlinerImpl::Init is called before the Outliner UI has been fully init, so we generate the columns in the next tick
	TickerHandle = FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateLambda([this](float)
	{
		RegenerateColumns(CreationParams.ColumnDescription);
		// Since Sort runs before the Teds Columns are added, we want to resort since we override the Item Label column
		SceneOutliner->RequestSort();
		return false;
	}));
}

void FTedsOutlinerImpl::FullRefresh() const
{
	FSceneOutlinerHierarchyChangedData EventData;
	EventData.Type = FSceneOutlinerHierarchyChangedData::FullRefresh;
	HierarchyChangedEvent.Broadcast(EventData);
}

void FTedsOutlinerImpl::OnMonitoredRowsAdded(DataStorage::FRowHandleArrayView InRows)
{
	// Refresh queries so that the added rows can pass through the filters if applied
	RefreshQueries();

	const bool bHasNoActiveFilters = FilterNodes.IsEmpty() && ClassFilterNodes.IsEmpty();
	for (DataStorage::RowHandle RowToAdd : InRows)
	{
		// Don't display the new row unless it is in the filtered row list, but there is no need to check if there are no filters active
		if (bHasNoActiveFilters || FinalCombinedRowNode->GetRows().Contains(RowToAdd))
		{
			// If the row we are adding is selected, we are probably synced to some external selection and the Outliner needs to update selection
			if (Storage->HasColumns<FTypedElementSelectionColumn>(RowToAdd))
			{
				bSelectionDirty = true;
			}
			FSceneOutlinerHierarchyChangedData EventData;
			EventData.Type = FSceneOutlinerHierarchyChangedData::Added;
			EventData.Items.Add(SceneOutlinerMode->CreateItemFor<FTedsOutlinerTreeItem>(FTedsOutlinerTreeItem(RowToAdd, AsShared())));
			HierarchyChangedEvent.Broadcast(EventData);
		}
	}
}

void FTedsOutlinerImpl::OnMonitoredRowsRemoved(DataStorage::FRowHandleArrayView InRows)
{
	for (DataStorage::RowHandle RowToRemove : InRows)
	{
		FSceneOutlinerHierarchyChangedData EventData;
		EventData.Type = FSceneOutlinerHierarchyChangedData::Removed;
		EventData.ItemIDs.Add(RowToRemove);
		HierarchyChangedEvent.Broadcast(EventData);
	}
}

FTedsOutlinerImpl::~FTedsOutlinerImpl()
{
	if (Storage)
	{
		if (HierarchyData)
		{
			HierarchyData->UnregisterQueries(*Storage);
		}

		// Clean up any widget rows that were parented to the Outliner
		const FHierarchyHandle WidgetHierarchy = Storage->FindHierarchyByName(TableViewerUtils::GetWidgetHierarchyName());
		if (Storage->IsValidHierarchyHandle(WidgetHierarchy))
		{
			FRowHandleArray RowsToRemove;
			Storage->WalkDepthFirst(WidgetHierarchy, OutlinerRowHandle,
				[&RowsToRemove](const ICoreProvider&, RowHandle, RowHandle Target)
				{
					RowsToRemove.Add(Target);
					return true;
				});
			Storage->BatchRemoveRows(RowsToRemove.GetRows());
		}
		else
		{
			// If no hierarchy was found, then manually clean up the handle as usually the Walk will handle it
			Storage->RemoveRow(OutlinerRowHandle);
		}

		Storage->OnUpdateCompleted().RemoveAll(this);
	}

	SceneOutliner->OnTextFilterChanged().Remove(OnTextFilteredChangedHandle);

	UnregisterQueries();
	FTSTicker::RemoveTicker(TickerHandle);
}

FTedsOutlinerImpl::FIsItemCompatible& FTedsOutlinerImpl::IsItemCompatible()
{
	return IsItemCompatibleWithTeds;
}

void FTedsOutlinerImpl::SetSelection(const TArray<DataStorage::RowHandle>& InSelectedRows)
{
	if (!SelectionSet)
	{
		return;
	}

	// Clear the current selection first using the UTypedElementSelectionSet
	ClearSelection();

	// Select each row using the TypedElementSelectionInterface via UTypedElementSelectionSet
	FTypedElementSelectionOptions SelectionOptions;
	SelectionOptions.SetAllowHidden(true); // Have to allow selecting things that we made hidden
	for(DataStorage::RowHandle Row : InSelectedRows)
	{
		FTypedElementHandle ElementHandle;
		// Get the TypedElementHandle from storage for this row
		if (DataStorage::Compatibility::FTypedElementColumn* Column = Storage->GetColumn<DataStorage::Compatibility::FTypedElementColumn>(Row))
		{
			ElementHandle = Column->Handle;
		}
		
		if (ElementHandle)
		{
			// Use the UTypedElementSelectionSet's selection interface to select this element
			SelectionSet->SelectElement(ElementHandle, SelectionOptions);
		}
	}
}
	
TSharedRef<SWidget> FTedsOutlinerImpl::CreateLabelWidget(
	DataStorage::ICoreProvider& Storage,
	DataStorage::IUiProvider& StorageUi,
	const DataStorage::IUiProvider::FPurposeID& Purpose,
	DataStorage::RowHandle TargetRow,
	const DataStorage::RowHandle OutlinerWidgetRow,
	ISceneOutlinerTreeItem& TreeItem,
	const STableRow<FSceneOutlinerTreeItemPtr>& RowItem,
	bool bInteractable)
{
	IUiProvider::FWidgetConstructorPtr WidgetConstructor = LabelWidgetUtils::CreateLabelWidgetConstructorInternal(Storage, StorageUi, Purpose, TargetRow);
	if (!WidgetConstructor)
	{
		return SNullWidget::NullWidget;
	}

	TSharedPtr<SWidget> Widget = LabelWidgetUtils::CreateLabelWidgetInternal(*WidgetConstructor, Storage, StorageUi, TargetRow, OutlinerWidgetRow, TreeItem, RowItem, bInteractable);
	if (!Widget)
	{
		return SNullWidget::NullWidget;
	}
	
	return SNew(SBox)
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Center)
		[
			Widget.ToSharedRef()
		];
}

TSharedRef<SWidget>FTedsOutlinerImpl::CreateLabelWidgetForItem(DataStorage::RowHandle TargetRow, ISceneOutlinerTreeItem& TreeItem, const STableRow<FSceneOutlinerTreeItemPtr>& RowItem, bool bIsInteractable) const
{
	return CreateLabelWidget(*Storage, *StorageUi, LabelWidgetPurpose, TargetRow, GetOutlinerRowHandle(), TreeItem, RowItem, bIsInteractable);
}

void FTedsOutlinerImpl::AddExternalQuery(const FName& QueryName, const QueryHandle& InQueryHandle)
{
	if (ensureMsgf(InQueryHandle != InvalidQueryHandle, TEXT("An Invalid Query Handle cannot be used for a TEDS Filter")))
	{
		ExternalQueries.Emplace(QueryName, MakeShared<DataStorage::QueryStack::FQueryHandleNode>(InQueryHandle));
	}
}

void FTedsOutlinerImpl::RemoveExternalQuery(const FName& QueryName)
{
	ExternalQueries.Remove(QueryName);
}

void FTedsOutlinerImpl::AppendExternalQueries(FQueryDescription& OutQuery)
{
	for(const TPair<FName, TSharedPtr<QueryStack::IQueryNode>>& ExternalQuery : ExternalQueries)
	{
		DataStorage::Queries::MergeQueries(OutQuery, Storage->GetQueryDescription(ExternalQuery.Value->GetQuery()));
	}
}

void FTedsOutlinerImpl::AddExternalQueryFunction(const FName& QueryName, const DataStorage::Queries::TConstQueryFunction<bool>& InQueryFunction)
{
	// Store the external query functions as functions instead of RowNodes since the row node to filter on has not been initialized yet
	ExternalQueryFunctions.Emplace(QueryName, InQueryFunction);
}

void FTedsOutlinerImpl::RemoveExternalQueryFunction(const FName& QueryName)
{
	ExternalQueryFunctions.Remove(QueryName);
}

void FTedsOutlinerImpl::AddClassQueryFunction(const FName& ClassName, const TConstQueryFunction<bool>& InClassQueryFunction)
{
	ClassFilters.Emplace(ClassName, InClassQueryFunction);
}

void FTedsOutlinerImpl::RemoveClassQueryFunction(const FName& ClassName)
{
	ClassFilters.Remove(ClassName);
}

bool FTedsOutlinerImpl::CanDisplayRow(DataStorage::RowHandle ItemRowHandle) const
{
	/*
	 * Don't display widgets that are created for rows in this table viewer. Widgets are only created for rows that are currently visible, so if we
	 * display the rows for them we are now adding/removing rows to the table viewer based on currently visible rows. But adding rows can cause
	 * scrolling and change the currently visible rows which in turn again adds/removes widget rows. This chain keeps continuing which can cause
	 * flickering/scrolling issues in the table viewer.
	 */
	if (Storage->HasColumns<FTypedElementSlateWidgetReferenceColumn>(ItemRowHandle))
	{
		// Check if this widget row belongs to the same table viewer it is being displayed in
		if (const FSceneOutlinerColumn* TedsOutlinerColumn = Storage->GetColumn<FSceneOutlinerColumn>(ItemRowHandle))
		{
			if (const TSharedPtr<ISceneOutliner> TableViewer = TedsOutlinerColumn->Outliner.Pin())
		{
				return SceneOutliner != TableViewer.Get();
			}
		}
	}
	return true;
}

void FTedsOutlinerImpl::RegisterTedsOutliner()
{
	DataStorage::TableHandle OutlinerTable = Storage->FindTable(UE::Editor::Outliner::Helpers::GetTedsOutlinerTableName());

	if (OutlinerTable != DataStorage::InvalidTableHandle)
	{
		OutlinerRowHandle = Storage->AddRow(OutlinerTable);

		if (FSceneOutlinerColumn* TedsOutlinerColumn = Storage->GetColumn<FSceneOutlinerColumn>(OutlinerRowHandle))
		{
			TedsOutlinerColumn->Outliner = StaticCastSharedRef<ISceneOutliner>(SceneOutliner->AsShared());
		}

		Storage->AddColumn<FTableViewerTag>(OutlinerRowHandle);
		Storage->AddColumn(OutlinerRowHandle, FHighlightTextColumn{});

		OnTextFilteredChangedHandle = SceneOutliner->OnTextFilterChanged().AddLambda([this]()
			{
				if (FHighlightTextColumn* HighlightTextColumn = Storage->GetColumn<FHighlightTextColumn>(OutlinerRowHandle))
				{
					HighlightTextColumn->HighlightText = SceneOutliner->GetFilterHighlightText().Get();
				}
			});

		if (FTedsOutlinerColumnRefreshEventColumn* ColumnQueryColumn = Storage->GetColumn<FTedsOutlinerColumnRefreshEventColumn>(OutlinerRowHandle))
		{
			ColumnQueryColumn->OnRefreshColumns = MakeShared<UE::Editor::Outliner::FOnRefreshColumns>();
			ColumnQueryColumn->OnRefreshColumns->BindSP(this, &FTedsOutlinerImpl::RegenerateColumns);
		}

		if (FTedsOutlinerRowHandleDealiaserColumn* RowHandleDealiaserColumn = Storage->GetColumn<FTedsOutlinerRowHandleDealiaserColumn>(OutlinerRowHandle))
		{
			RowHandleDealiaserColumn->GetRowHandle = FGetRowHandleFromOutlinerItem::CreateSP(this, &FTedsOutlinerImpl::GetRowHandleFromOutlinerItem);
		}

		FName Identifier = SceneOutliner->GetOutlinerIdentifier();

		if (!Identifier.IsNone())
		{
			// Any old outliner instances could still have dangling references if external systems are holding onto them - which means it might only
			// be destructed after this new instance is init and MapRow ensures on duplicate keys. At this point it is safe to remove the old mapping
			// since all the data will be fine because the old row exists till the destructor is called and we want any systems using the mapping
			// to start looking at the new instance anyways
			Storage->RemoveRowMapping(MappingDomain, DataStorage::FMapKey(SceneOutliner->GetOutlinerIdentifier()));
			Storage->MapRow(MappingDomain, DataStorage::FMapKey(SceneOutliner->GetOutlinerIdentifier()), OutlinerRowHandle);
		}
	}
}

void FTedsOutlinerImpl::ClearColumns() const
{
	for (FName ColumnName : AddedColumns)
	{
		SceneOutliner->RemoveColumn(ColumnName);
	}
}

void FTedsOutlinerImpl::RegenerateColumns(const Outliner::FTedsOutlinerColumnDescription& ColumnDescription)
{
	ClearColumns();

	FTreeItemIDDealiaser Dealiaser;

	if (FTedsOutlinerDealiaserColumn* DealiaserColumn = Storage->GetColumn<FTedsOutlinerDealiaserColumn>(OutlinerRowHandle))
	{
		Dealiaser = DealiaserColumn->Dealiaser;
	}

	using MatchApproach = DataStorage::IUiProvider::EMatchApproach;
	
	DataStorage::FColumnsMetaDataView MetaDataView(ColumnDescription.GetColumnMetaData());
	
	TArray<TWeakObjectPtr<const UScriptStruct>> ColumnTypes =
		TableViewerUtils::CreateVerifiedColumnTypeArray(ColumnDescription.GetEffectiveColumns());

	AddedColumns.Reset(ColumnTypes.Num());

	// Sort by before/after before passing here
	TMap<TWeakObjectPtr<const UScriptStruct>, uint8> OrderedColumnMap;
	OrderedColumnMap.Reserve(ColumnTypes.Num());
	Helpers::OrderColumns(ColumnTypes, ColumnDescription, OrderedColumnMap);
	
	const RowHandle CellPurposeRow = StorageUi->FindPurpose(CellWidgetPurpose);
	const RowHandle HeaderPurposeRow = StorageUi->FindPurpose(HeaderWidgetPurpose);

	// Label Column Construction
	if (!bHybridMode)
	{
		auto LabelColumnConstructor = [this, ColumnDescription, MetaDataView, HeaderPurposeRow, Dealiaser](
			IUiProvider::FWidgetConstructorPtr CellConstructor, TConstArrayView<TWeakObjectPtr<const UScriptStruct>> ColumnTypes)
			{
				const FName LabelColumnName = FSceneOutlinerBuiltInColumnTypes::Label();
				// Explicitly set the priority Index to greater than high-priority icon columns since the Label Column
				// is manually added to the ColumnMap right now and cannot use dynamic priority grouping
				constexpr uint8 LabelColumnPriority = 10;
	
				// Grab a reference to the label column and remove it from the actual outliner
				const TMap<FName, TSharedPtr<ISceneOutlinerColumn>>& Columns = SceneOutliner->GetColumns();
				const TSharedPtr<ISceneOutlinerColumn> FallbackColumnPtr = Columns.FindRef(LabelColumnName);
				SceneOutliner->RemoveColumn(LabelColumnName);
	
				const FText DisplayName = CellConstructor.Get()->CreateWidgetDisplayNameText(Storage);
				AddedColumns.Add(LabelColumnName);
				SceneOutliner->AddColumn(LabelColumnName,
					FSceneOutlinerColumnInfo(
						ESceneOutlinerColumnVisibility::Visible, 
						LabelColumnPriority,
						FCreateSceneOutlinerColumn::CreateLambda(
							[this, ColumnDescription, MetaDataView, LabelColumnName, ColumnTypes, CellConstructor, FallbackColumnPtr, HeaderPurposeRow, Dealiaser](ISceneOutliner&)
							{
								FTedsOutlinerUiColumnInitParams Params(FMetaData(), *Storage, *StorageUi, *StorageCompatibility);
								const FSceneOutlinerColumn* TedsOutlinerColumn = Storage->GetColumn<FSceneOutlinerColumn>(OutlinerRowHandle);
								if  (!TedsOutlinerColumn)
								{
									return MakeShared<FTedsSceneOutlinerItemLabelColumn>(Params);
								}
									
								IUiProvider::FWidgetConstructorPtr HeaderConstructor = 
									TableViewerUtils::CreateHeaderWidgetConstructor(*StorageUi, MetaDataView, ColumnTypes, HeaderPurposeRow);
	
								FTedsTableViewerColumn::FSortHandler SortDelegates
								{
									.OnSortHandler = FTedsTableViewerColumn::FOnSort::CreateSP(this, &FTedsOutlinerImpl::OnSort),
									.IsSortingHandler = FTedsTableViewerColumn::FIsSorting::CreateSP(this, &FTedsOutlinerImpl::IsSorting)
								};

								FMetaData TedsOutlinerUiColumnMetaData = ColumnDescription.GetGenericMetaData();
								TedsOutlinerUiColumnMetaData.AddImmutableData(TEXT("OutlinerIdentifier"), SceneOutliner->GetOutlinerIdentifier().ToString());
									
								Params.MetaData = TedsOutlinerUiColumnMetaData;
								Params.NameId = LabelColumnName;
								Params.ColumnTypes = TArray<TWeakObjectPtr<const UScriptStruct>>(ColumnTypes.GetData(), ColumnTypes.Num());
								Params.HeaderWidgetConstructor = MoveTemp(HeaderConstructor);
								Params.CellWidgetConstructor = CellConstructor;
								Params.FallbackColumn = FallbackColumnPtr;
								Params.OwningOutliner = TedsOutlinerColumn->Outliner;
								Params.TedsOutlinerImpl = AsWeak();
								Params.SortDelegates = MoveTemp(SortDelegates);
								Params.Dealiaser = Dealiaser;
								Params.bHybridMode = false;
								
								return MakeShared<FTedsSceneOutlinerItemLabelColumn>(Params);
	
							}),
						false,
						TOptional<float>(),
						DisplayName
					)
				);
				return true;
			};

		// The Outliner Label matches to the label column && the type info column, so we pass in both
		TArray<TWeakObjectPtr<const UScriptStruct>> MatchedLabelColumns = {
			FTypedElementLabelColumn::StaticStruct(),
			FTypedElementClassTypeInfoColumn::StaticStruct() };
		StorageUi->CreateWidgetConstructors(CellPurposeRow, MatchApproach::LongestMatch, MatchedLabelColumns, 
			MetaDataView, LabelColumnConstructor);
	}

	int32 IndexOffset = 0;
	auto ColumnConstructor = [this, ColumnDescription, MetaDataView, &IndexOffset, &OrderedColumnMap, HeaderPurposeRow, Dealiaser](
		IUiProvider::FWidgetConstructorPtr CellConstructor, TConstArrayView<TWeakObjectPtr<const UScriptStruct>> ColumnTypes)
		{
			/* If we have a fallback column for this query, remove it, take over its priority and 
			 * replace it with the TEDS column. But also allow the TEDS-Outliner column to fall back to it for
			 * data not in TEDS yet.
			 */
			const FName FallbackColumnName = Helpers::FindOutlinerColumnFromTedsColumns(Storage, ColumnTypes);
			// A priority should always be found, but use the IndexOffset as a default
			uint8 ColumnPriority = static_cast<uint8>(FMath::Clamp(IndexOffset, 0, 255));
			ESceneOutlinerColumnVisibility ColumnVisibility = ESceneOutlinerColumnVisibility::Visible;
			if (const FSceneOutlinerColumnInfo* FallbackColumnInfo = SceneOutliner->GetSharedData().ColumnMap.Find(FallbackColumnName))
			{
				ColumnVisibility = FallbackColumnInfo->Visibility;
				ColumnPriority = FallbackColumnInfo->PriorityIndex;
			}
			else
			{
				for (const TWeakObjectPtr<const UScriptStruct>& ColumnType : ColumnTypes)
				{
					if (const FTedsOutlinerColumnParams* FoundColumnParams = ColumnDescription.FindColumnParams(ColumnType))
					{
						ColumnVisibility = FoundColumnParams->InitialVisibility;
					}
					if (const uint8* FoundColumnPriority = OrderedColumnMap.Find(ColumnType))
					{
						// We want to use the same column if possible so break as soon as we find the Priority
						// (meaning Params would have been defined if desired for this column above, if not, none
						// were given for the matching column we found for the priority)
						ColumnPriority = *FoundColumnPriority;
						break;
					}
				}
			}

			// Grab a reference to the fallback column and remove it from the actual outliner
			const TMap<FName, TSharedPtr<ISceneOutlinerColumn>>& Columns = SceneOutliner->GetColumns();
			const TSharedPtr<ISceneOutlinerColumn> FallbackColumnPtr = Columns.FindRef(FallbackColumnName);
			SceneOutliner->RemoveColumn(FallbackColumnName);

			FName NameId = TableViewerUtils::FindLongestMatchingName(ColumnTypes, IndexOffset);
			const FText DisplayName = CellConstructor.Get()->CreateWidgetDisplayNameText(Storage, GetOutlinerRowHandle());
			AddedColumns.Add(NameId);
			SceneOutliner->AddColumn(NameId,
				FSceneOutlinerColumnInfo(
					ColumnVisibility, 
					ColumnPriority,
					FCreateSceneOutlinerColumn::CreateLambda(
						[this, ColumnDescription, MetaDataView, NameId, &ColumnTypes, CellConstructor, FallbackColumnPtr, HeaderPurposeRow, Dealiaser](ISceneOutliner&)
						{
							FTedsOutlinerUiColumnInitParams Params(FMetaData(), *Storage, *StorageUi, *StorageCompatibility);
							const FSceneOutlinerColumn* TedsOutlinerColumn = Storage->GetColumn<FSceneOutlinerColumn>(OutlinerRowHandle);
							if  (!TedsOutlinerColumn)
							{
								return MakeShared<FTedsOutlinerUiColumn>(Params);
							}
								
							IUiProvider::FWidgetConstructorPtr HeaderConstructor = 
								UE::Editor::DataStorage::TableViewerUtils::CreateHeaderWidgetConstructor(*StorageUi, MetaDataView, ColumnTypes, HeaderPurposeRow);

							DataStorage::FTedsTableViewerColumn::FSortHandler SortDelegates
							{
								.OnSortHandler = DataStorage::FTedsTableViewerColumn::FOnSort::CreateSP(this, &FTedsOutlinerImpl::OnSort),
								.IsSortingHandler = DataStorage::FTedsTableViewerColumn::FIsSorting::CreateSP(this, &FTedsOutlinerImpl::IsSorting)
							};

							FMetaData TedsOutlinerUiColumnMetaData = ColumnDescription.GetGenericMetaData();
							TedsOutlinerUiColumnMetaData.AddImmutableData(TEXT("OutlinerIdentifier"), SceneOutliner->GetOutlinerIdentifier().ToString());
								
							Params.MetaData = TedsOutlinerUiColumnMetaData;
							Params.NameId = NameId;
							Params.ColumnTypes = TArray<TWeakObjectPtr<const UScriptStruct>>(ColumnTypes.GetData(), ColumnTypes.Num());
							Params.HeaderWidgetConstructor = MoveTemp(HeaderConstructor);
							Params.CellWidgetConstructor = CellConstructor;
							Params.FallbackColumn = FallbackColumnPtr;
							Params.OwningOutliner = TedsOutlinerColumn->Outliner;
							Params.TedsOutlinerImpl = AsWeak();
							Params.SortDelegates = MoveTemp(SortDelegates);
							Params.Dealiaser = Dealiaser;
							Params.bHybridMode = bHybridMode;
							
							return MakeShared<FTedsOutlinerUiColumn>(Params);

						}),
					true,
					TOptional<float>(),
					DisplayName
				)
			);
			++IndexOffset;
			return true;
		};

	StorageUi->CreateWidgetConstructors(CellPurposeRow, MatchApproach::LongestMatch, ColumnTypes, 
		MetaDataView, ColumnConstructor);
}

RowHandle FTedsOutlinerImpl::GetRowHandleFromOutlinerItem(const ISceneOutlinerTreeItem& Item) const
{
	if (Storage && StorageCompatibility)
	{
		return Helpers::GetRowHandleFromOutlinerItem(*Storage, *StorageCompatibility, OutlinerRowHandle, Item);
	}
	return InvalidRowHandle;
}

void FTedsOutlinerImpl::OnSelectionChangedHandler(const UTypedElementSelectionSet* InSelectionSet)
{
	bSelectionDirty = true;
}

void FTedsOutlinerImpl::SetPendingSelectionType(ESelectInfo::Type InSelectionType)
{
	PendingSelectionType = InSelectionType;
}

DataStorage::RowHandle FTedsOutlinerImpl::GetOutlinerRowHandle() const
{
	return OutlinerRowHandle;
}

void FTedsOutlinerImpl::IncludeExcludeColumns(const TConstArrayView<TWeakObjectPtr<const UScriptStruct>> IncludedColumns, const TConstArrayView<TWeakObjectPtr<const UScriptStruct>> ExcludedColumns)
{
	const bool bIncludedColumns = CreationParams.ColumnDescription.IncludeColumns(IncludedColumns);
	const bool bExcludedColumns = CreationParams.ColumnDescription.ExcludeColumns(ExcludedColumns);
	if (bIncludedColumns || bExcludedColumns)
	{
		RegenerateColumns(CreationParams.ColumnDescription);
	}
}

int32 FTedsOutlinerImpl::GetTotalRowCount() const
{
	if (FinalCombinedRowNode)
	{
		return FinalCombinedRowNode->GetRows().Num() + ForceShownParentRows.Num();
	}
	return 0;
}

const FTedsOutlinerExpansionStateBridge& FTedsOutlinerImpl::GetExpansionStateBridge() const
{
	return ExpansionStateBridge;
}

void FTedsOutlinerImpl::CreateItemsFromQuery(TArray<FSceneOutlinerTreeItemPtr>& OutItems, ISceneOutlinerMode* InMode)
{
	using namespace UE::Editor::DataStorage::Queries;

	// We refresh the row collector node on demand, since this function is only called on a FullRefresh() and the node is setup to not do anything on
	// update. Addition/Removal in between refreshes is handled by the row monitor node
	// Technically the RowMonitorNode also handles keeping this list up to date using observers, but the monitor node is optional and it's safer
	// to do a full clean refresh at this point anyways
	
	RefreshQueries();

	// Compute force-shown parents upfront from the full FinalCombinedRowNode, before any text search touches the tree. 
	// Doing it here (rather than lazily in GetParentRow) ensures the set is stable and unaffected by which items the text filter later hides.
	ForceShownParentRows.Empty();
	if (bForceShowParents && HierarchyData)
	{
		InitialQueryResultsSortedNodeExecutor->Update();
		const DataStorage::FRowHandleArrayView InitialRows = InitialQueryResultsSortedNode->GetRows();
		const DataStorage::FRowHandleArrayView FinalRows = RowHandleSortNode->GetRows(); // sorted — Contains is O(log n)

		for (RowHandle Row : FinalCombinedRowNode->GetRows())
		{
			RowHandle Parent = HierarchyData->GetParent(*Storage, Row);
			int32 MaxIterations = Private::MaxHierarchyWalkIterations;
			while (Storage->IsRowAvailable(Parent))
			{
				--MaxIterations;
				if (!ensureMsgf(MaxIterations > 0,
					TEXT("Reached max iterations walking parents for ForceShownParentRows, row %llu"), Row))
				{
					break;
				}
				if (!FinalRows.Contains(Parent) && InitialRows.Contains(Parent))
				{
					ForceShownParentRows.Add(Parent);
				}
				Parent = HierarchyData->GetParent(*Storage, Parent);
			}
		}

		ForceShownParentRows.MakeUnique();
	}

	for (RowHandle Row : FinalCombinedRowNode->GetRows())
	{
		if (FSceneOutlinerTreeItemPtr TreeItem = InMode->CreateItemFor<FTedsOutlinerTreeItem>(FTedsOutlinerTreeItem(Row, AsShared()), false))
		{
			OutItems.Add(TreeItem);
		}
	}

	// We need to update selection since the whole outliner was re-populated
	bSelectionDirty = true;
}

void FTedsOutlinerImpl::CreateChildren(const FSceneOutlinerTreeItemPtr& Item, TArray<FSceneOutlinerTreeItemPtr>& OutChildren) const
{
	/* TEDS-Outliner TODO: This can probably be improved or optimized in the future
	 * 
	 * TEDS currently only supports one way lookup for parents, so to get the children
	 * for a given row we currently have to go through every row (that matches our populate query) with a parent column to check if the parent
	 * is our row.
	 * This has to be done recursively to grab our children, grandchildren and so on...
	 */

	// If there's no hierarchy data, there is no need to create children
	if (!HierarchyData)
	{
		return;
	}

	using namespace UE::Editor::DataStorage::Queries;
	
	const FTedsOutlinerTreeItem* TedsTreeItem = Item->CastTo<FTedsOutlinerTreeItem>();

	// If this item is not a TEDS item, we are not handling it
	if (!TedsTreeItem)
	{
		return;
	}
		
	RowHandle ItemRowHandle = TedsTreeItem->GetRowHandle();

	if(!Storage->IsRowAssigned(ItemRowHandle))
	{
		return;
	}

	FRowHandleArray ChildItems;
	
	HierarchyData->WalkDepthFirst(*Storage, ItemRowHandle, [ItemRowHandle, &ChildItems](const ICoreProvider& Context, RowHandle Owner, RowHandle Target)
	{
		if (Target != ItemRowHandle)
		{
			ChildItems.Add(Target);
		}
	});
	
	// If the row doesn't exist in our query stack results, it doesn't match the query and should not be displayed
	FRowHandleArrayView Rows(RowHandleSortNode->GetRows());

	// Actually create the items for the child entities 
	for (RowHandle ChildItemRowHandle : ChildItems.GetRows())
	{
		if (!Rows.Contains(ChildItemRowHandle))
		{
			continue;
		}
		
		if (FSceneOutlinerTreeItemPtr ChildActorItem = SceneOutlinerMode->CreateItemFor<FTedsOutlinerTreeItem>(FTedsOutlinerTreeItem(ChildItemRowHandle, AsShared())))
		{
			OutChildren.Add(ChildActorItem);
		}
	}
}

DataStorage::RowHandle FTedsOutlinerImpl::GetParentRow(DataStorage::RowHandle InRowHandle)
{
	// No parent if there is no hierarchy data specified
	if (!HierarchyData)
	{
		return DataStorage::InvalidRowHandle;
	}
	
	DataStorage::FRowHandleArrayView Rows;

	if (ShouldForceShowParentRows())
	{
		// We want to only include parents that match the initial query but not any additional filters. This essentially acts as 
		// a way to force show parents even if they don't match the current filter but their child does
	
		// Update this on demand, if the results are already sorted no work is done
		InitialQueryResultsSortedNodeExecutor->Update();
	
		Rows = InitialQueryResultsSortedNode->GetRows();
	}
	else
	{
		// If we aren't force showing parents, make sure the parent also matches the final query with all additional filters included
		Rows = RowHandleSortNode->GetRows();
	}
	
	RowHandle ParentRowHandle = InRowHandle;

	// Sanity check in case there are loops in the hierarchy
	int32 MaxIterations = Private::MaxHierarchyWalkIterations;
	do
	{
		--MaxIterations;
		if (!ensureMsgf(MaxIterations > 0, TEXT("Reached max iterations when looking for parent of row %llu"), InRowHandle))
		{
			ParentRowHandle = InvalidRowHandle;
			break;
		}
		
		ParentRowHandle = HierarchyData->GetParent(*Storage, ParentRowHandle);
	}
	// Walk up the parent chain until we find a parent that is present our query stack results or we reached the root of this hierarchy
	while (Storage->IsRowAvailable(ParentRowHandle) && !Rows.Contains(ParentRowHandle));
	
	if (!Storage->IsRowAvailable(ParentRowHandle))
	{
		return DataStorage::InvalidRowHandle;
	}

	return ParentRowHandle;
}

bool FTedsOutlinerImpl::ShouldForceShowParentRows() const
{
	return bForceShowParents;
}

void FTedsOutlinerImpl::RefreshQueries() const
{
	InitialRowCollectorNode->Refresh();
	CombinedRowCollectorNode->Refresh();
	QueryStackExecutor->Update();
}

void FTedsOutlinerImpl::RecompileQueries()
{
	using namespace UE::Editor::DataStorage::Queries;
	using namespace UE::Editor::DataStorage::QueryStack;
	
	UnregisterQueries();

	// Get a list of all the query nodes (initial query + query description filters) to combine them into one composite query
	TArray<TSharedPtr<IQueryNode>> QueryNodes;

	TSharedPtr<IQueryNode> InitialQueryNode = MakeShared<FQueryNode>(*Storage, InitialQueryDescription);
	
	QueryNodes.Add(InitialQueryNode);
	for (const TPair<FName, TSharedPtr<IQueryNode>>& ExternalQuery : ExternalQueries)
	{
		QueryNodes.Add(ExternalQuery.Value);
	}

	TSharedPtr<IQueryNode> CombinedQueryNode = MakeShared<FQueryMergeNode>(*Storage, QueryNodes);

	// If we have a monitor node, it handles updating the collector node automatically. Otherwise, we need it to update on tick
	FRowQueryResultsNode::ESyncActions SyncActions =
		CreationParams.bUseDefaultObservers ? FRowQueryResultsNode::ESyncActions::None : FRowQueryResultsNode::ESyncActions::ForceRefreshOnUpdate;

	// Node to collect all rows matching the initial query (stored so OR filters can filter on the original query)
	InitialRowCollectorNode = MakeShared<FRowQueryResultsNode>(*Storage, InitialQueryNode, SyncActions);
	
	// We store a copy of the collected rows into an array sorted by row handles for faster lookup
	TSharedRef<FRowCopyNode> InitialCopyNode = MakeShared<FRowCopyNode>(InitialRowCollectorNode);
	InitialQueryResultsSortedNode = MakeShared<FRowHandleSortNode>(InitialCopyNode);
	
	static int InstanceCounter = 0;
	
	// InstanceCounter is incremented later on
	InitialQueryResultsSortedNodeExecutor = 
		MakeUnique<QueryStack::FExplicitUpdateExecutor>(FName("TedsOutlinerInitialResults", InstanceCounter++), InitialQueryResultsSortedNode);


	// Node to collect all rows matching the composite query
	CombinedRowCollectorNode = MakeShared<FRowQueryResultsNode>(*Storage, CombinedQueryNode, SyncActions);

	FilterNodes.Empty();
	TSharedPtr<IRowNode> PrimaryFilterNode = CombinedRowCollectorNode;
	// Combines all active external filters using AND
	for (const TPair<FName, TConstQueryFunction<bool>>& ExternalQueryFunction : ExternalQueryFunctions)
	{
		TSharedPtr<FRowFilterNode> FilterNode = MakeShared<FRowFilterNode>(Storage, PrimaryFilterNode, ExternalQueryFunction.Value);
		FilterNodes.Add(FilterNode);
		PrimaryFilterNode = FilterNode;
	}

	ClassFilterNodes.Empty();
	// Combines all active class filters using OR
	if (!ClassFilters.IsEmpty())
	{
		for (const TPair<FName, TConstQueryFunction<bool>>& ClassFilter : ClassFilters)
		{
			ClassFilterNodes.Add(MakeShared<FRowFilterNode>(Storage, InitialRowCollectorNode, ClassFilter.Value));
		}
		
		TArray<TSharedPtr<IRowNode>> ClassBaseFilterNodes(ClassFilterNodes);
		CombinedClassFilterRowNode = MakeShared<FRowMergeNode>(ClassBaseFilterNodes, FRowMergeNode::EMergeApproach::Unique);

		FinalCombinedRowNode = MakeShared<FRowMergeNode>(MakeConstArrayView({PrimaryFilterNode, CombinedClassFilterRowNode}), FRowMergeNode::EMergeApproach::Repeating);
	}
	else
	{
		FinalCombinedRowNode = PrimaryFilterNode;
	}

	// The "actual" collector node passed into other misc nodes is the monitor node if it exists, the collector node otherwise
	TSharedPtr<IRowNode> ActualCollectorNode = FinalCombinedRowNode;
	
	// Node to monitor row addition/removal
	if (CreationParams.bUseDefaultObservers)
	{
		RowMonitorNode = MakeShared<FRowMonitorNode>(*Storage, FinalCombinedRowNode, CombinedQueryNode);
		RowMonitorNode->OnMonitoredRowsAdded().AddRaw(this, &FTedsOutlinerImpl::OnMonitoredRowsAdded);
		RowMonitorNode->OnMonitoredRowsRemoved().AddRaw(this, &FTedsOutlinerImpl::OnMonitoredRowsRemoved);
		ActualCollectorNode = RowMonitorNode;
	}
	
	// We store a copy of the collected rows into an array sorted by row handles for faster lookup
	TSharedRef<FRowCopyNode> CopyNode = MakeShared<FRowCopyNode>(ActualCollectorNode);
	RowHandleSortNode = MakeShared<FRowHandleSortNode>(CopyNode);

	// Nodes to actually sort the rows by the column the user requested
	// Persist the sort order if the sorting nodes are being reset
	RowPrimarySortingNode = MakeShared<FRowSortNode>(*Storage, RowHandleSortNode, RowPrimarySortingNode ? RowPrimarySortingNode->GetColumnSorter() : nullptr);
	RowPrimaryInversionNode = MakeShared<FRowOrderInversionNode>(RowPrimarySortingNode, RowPrimaryInversionNode ? RowPrimaryInversionNode->IsEnabled() : false);

	// A custom node to store row handle -> sort index mapping.
	RowSortMapNode = MakeShared<FRowMapNode>(RowPrimaryInversionNode);
	QueryStackExecutor = MakeUnique<QueryStack::FExplicitUpdateExecutor>(FName("TedsOutliner", InstanceCounter++), RowSortMapNode);

	// Our final query to collect rows to populate the Outliner - currently the same as the initial query the user provided
	FQueryDescription FinalQueryDescription(InitialQueryDescription);

	// Add the filters the user has active to the query
	AppendExternalQueries(FinalQueryDescription);

	// Queries to track parent info, only required if we have hierarchy data
	if (HierarchyData)
	{
		HierarchyData->UnregisterQueries(*Storage);
		HierarchyData->RegisterQueries(*Storage, FinalQueryDescription, StaticCastSharedRef<ISceneOutliner>(SceneOutliner->AsShared()), bUsingQueryConditionsSyntax);
	}

	if (SelectionSet)
	{
		if (!SelectionSetChangedDelegateHandle.IsValid() && SelectionSet != nullptr)
		{
			SelectionSetChangedDelegateHandle = SelectionSet->OnChanged().AddSP(this, &FTedsOutlinerImpl::OnSelectionChangedHandler);
		}
	}

	// Query to track when the label of a row we are observing changes, to re-filter/re-search for the item
	FQueryDescription LabelUpdateQueryDescription = 
		Select(
			TEXT("Re-Filter Teds Outliner Item on label change"),
			FProcessor(EQueryTickPhase::PostPhysics, Storage->GetQueryTickGroupName(EQueryTickGroups::SyncExternalToDataStorage))
				.SetExecutionMode(EExecutionMode::GameThread),
			[this](IQueryContext& Context, RowHandle Row, const FTypedElementLabelColumn& LabelColumn)
			{
				RowsPendingLabelUpdate.Add(Row);
			}
		)
		.Compile();
	
	if (bUsingQueryConditionsSyntax)
	{
		LabelUpdateQueryDescription.Conditions.Emplace(TColumn<FTypedElementSyncBackToWorldTag>() || TColumn<FTypedElementSyncFromWorldTag>());
	}
	else
	{
		LabelUpdateQueryDescription.ConditionTypes.Add(FQueryDescription::EOperatorType::SimpleAny);
		LabelUpdateQueryDescription.ConditionOperators.AddZeroed_GetRef().Type = FTypedElementSyncBackToWorldTag::StaticStruct();
		
		LabelUpdateQueryDescription.ConditionTypes.Add(FQueryDescription::EOperatorType::SimpleAny);
		LabelUpdateQueryDescription.ConditionOperators.AddZeroed_GetRef().Type = FTypedElementSyncFromWorldTag::StaticStruct();
	}
		
	// Add the conditions from FinalQueryDescription to ensure we are the rows the user requested
	MergeQueries(LabelUpdateQueryDescription, FinalQueryDescription);

	LabelUpdateQuery = Storage->RegisterQuery(MoveTemp(LabelUpdateQueryDescription));
	
	if (CreationParams.AddObserverQuery.IsSet())
	{
		// Additional use specified queries to add/remove nodes based on custom conditions
		FQueryDescription AdditionalAddObserverQueryDescription(CreationParams.AddObserverQuery.GetValue());
	
		AdditionalAddObserverQueryDescription.Callback.Function = [this](const FQueryDescription&, const IQueryContext& Context)
			{
				FRowHandleArray Rows(FRowHandleArrayView(Context.GetRowHandles(), FRowHandleArrayView::EFlags::IsUnique));
			
				// Weak ptr in case the Teds Outliner is destroyed before the next tick
				TWeakPtr<FTedsOutlinerImpl> WeakThis = AsWeak();
			
				// We need to defer the add by one frame so the query stack has a chance to update, otherwise the row won't be in the query stack results
				// and will get treated as not passing the filters and not get added
				FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateLambda([WeakThis, Rows](float)
				{
					if (TSharedPtr<FTedsOutlinerImpl> TedsOutlinerImpl = WeakThis.Pin())
					{
						TedsOutlinerImpl->OnMonitoredRowsAdded(Rows.GetRows());
					}
					return false;
				}));
			};
	
		AdditionalAddObserverQuery = Storage->RegisterQuery(MoveTemp(AdditionalAddObserverQueryDescription));
	}

	if (CreationParams.RemoveObserverQuery.IsSet())
	{
		FQueryDescription AdditionalRemoveObserverQueryDescription(CreationParams.RemoveObserverQuery.GetValue());
	
		AdditionalRemoveObserverQueryDescription.Callback.Function = [this](const FQueryDescription&, const IQueryContext& Context)
			{
				OnMonitoredRowsRemoved(FRowHandleArrayView(Context.GetRowHandles(), FRowHandleArrayView::EFlags::IsUnique));
			};
	
		AdditionalRemoveObserverQuery = Storage->RegisterQuery(MoveTemp(AdditionalRemoveObserverQueryDescription));
	}
}

void FTedsOutlinerImpl::UnregisterQueries()
{
	if (SelectionSetChangedDelegateHandle.IsValid())
	{
		if (SelectionSet != nullptr)
		{
			SelectionSet->OnChanged().Remove(SelectionSetChangedDelegateHandle);
		}
		SelectionSetChangedDelegateHandle.Reset();
	}
	if (Storage)
	{
		Storage->UnregisterQuery(SelectedRowsQuery);
		Storage->UnregisterQuery(SelectionAddedQuery);
		Storage->UnregisterQuery(SelectionRemovedQuery);
		Storage->UnregisterQuery(LabelUpdateQuery);
		Storage->UnregisterQuery(AdditionalAddObserverQuery);
		Storage->UnregisterQuery(AdditionalRemoveObserverQuery);

		SelectedRowsQuery = DataStorage::InvalidQueryHandle;
		SelectionAddedQuery = DataStorage::InvalidQueryHandle;
		SelectionRemovedQuery = DataStorage::InvalidQueryHandle;
		LabelUpdateQuery = DataStorage::InvalidQueryHandle;
		AdditionalAddObserverQuery = DataStorage::InvalidQueryHandle;
		AdditionalRemoveObserverQuery = DataStorage::InvalidQueryHandle;

	}
}

void FTedsOutlinerImpl::ClearSelection() const
{
	if (!SelectionSet)
	{
		return;
	}

	// Use the UTypedElementSelectionSet to clear all selections
	// This will properly use the TypedElementSelectionInterface for each element
	FTypedElementSelectionOptions SelectionOptions;
	SelectionSet->ClearSelection(SelectionOptions);
}

void FTedsOutlinerImpl::PostDataStorageUpdate()
{
	// Update the label for any rows that might need it
	for (DataStorage::RowHandle Row : RowsPendingLabelUpdate)
	{
		// Search is currently handled externally and not through the query stack, so if a node doesn't match the search filters it should still
		// exist in the query stack. If it does not exist in the query stack, there's no need to check it against the search filter or add it to the
		// Outliner
		if (RowSortMapNode->GetRowIndex(Row) != INDEX_NONE)
		{
			// If the item already exists, it only needs an update if it passed a filter previously and does not now (or vice versa)
			if (FSceneOutlinerTreeItemPtr ExistingItem = SceneOutliner->GetTreeItem(Row))
			{
				bool bCachedFilteredFlag = ExistingItem->Flags.bIsFilteredOut;

				// This implicitly calls into the data storage to get the label of the row and check against the search query
				ExistingItem->Flags.bIsFilteredOut = !SceneOutliner->PassesAllFilters(ExistingItem);

				if (bCachedFilteredFlag != ExistingItem->Flags.bIsFilteredOut)
				{
					SceneOutliner->OnItemLabelChanged(ExistingItem, false);
				}
			}
			// If the item doesn't exist, create a dummy item to see if it would match the current search/filter queries and should be actually added
			else if (FSceneOutlinerTreeItemPtr PotentialItem = SceneOutlinerMode->CreateItemFor<FTedsOutlinerTreeItem>(
					FTedsOutlinerTreeItem(Row, AsShared()), true))
			{
				SceneOutliner->OnItemLabelChanged(PotentialItem, false);
			}
		}
	}
	
	RowsPendingLabelUpdate.Empty();
}

void FTedsOutlinerImpl::Tick()
{
	if (bSelectionDirty)
	{
		OnTedsOutlinerSelectionChanged.Broadcast(PendingSelectionType);
		PendingSelectionType = ESelectInfo::Direct;
		bSelectionDirty = false;
	}

	// The Sort node is the child most node in all cases, so updating it causes all the nodes in the stack to update
	QueryStackExecutor->Update();
}

bool FTedsOutlinerImpl::IsSorting() const
{
	return RowPrimarySortingNode ? RowPrimarySortingNode->IsSorting() : false;
}

void FTedsOutlinerImpl::OnSort(FName ColumnName, TSharedPtr<const DataStorage::FColumnSorterInterface> ColumnSorter, EColumnSortMode::Type Direction,
	EColumnSortPriority::Type Priority)
{
	if (RowPrimarySortingNode)
	{
		if (Direction == EColumnSortMode::Type::Ascending)
		{
			RowPrimarySortingNode->SetColumnSorter(ColumnSorter);
			RowPrimaryInversionNode->Enable(false);
		}
		else
		{
			RowPrimaryInversionNode->Enable(true);
		}
	}
}

void FTedsOutlinerImpl::SortItems(TArray<FSceneOutlinerTreeItemPtr>& Items, EColumnSortMode::Type Direction) const
{
	Items.Sort([this](FSceneOutlinerTreeItemPtr A, FSceneOutlinerTreeItemPtr B)
	{
		FTedsOutlinerTreeItem* TedsA = A->CastTo<FTedsOutlinerTreeItem>();
		FTedsOutlinerTreeItem* TedsB = B->CastTo<FTedsOutlinerTreeItem>();

		if (!TedsA)
		{
			return false;
		}

		if (!TedsB)
		{
			return true;
		}
			
		return RowSortMapNode->GetRowIndex(TedsA->GetRowHandle()) < RowSortMapNode->GetRowIndex(TedsB->GetRowHandle());
	});
}

DataStorage::ICoreProvider* FTedsOutlinerImpl::GetStorage() const
{
	return Storage;
}

DataStorage::IUiProvider* FTedsOutlinerImpl::GetStorageUI() const
{
	return StorageUi;
}

DataStorage::ICompatibilityProvider* FTedsOutlinerImpl::GetStorageCompatibility() const
{
	return StorageCompatibility;
}

UTypedElementSelectionSet* FTedsOutlinerImpl::GetSelectionSet() const
{
	return SelectionSet;
}

FOnTedsOutlinerSelectionChanged& FTedsOutlinerImpl::OnSelectionChanged()
{
	return OnTedsOutlinerSelectionChanged;
}

ISceneOutlinerHierarchy::FHierarchyChangedEvent& FTedsOutlinerImpl::OnHierarchyChanged()
{
	return HierarchyChangedEvent;
}

} // namespace UE::Editor::Outliner

#undef LOCTEXT_NAMESPACE
