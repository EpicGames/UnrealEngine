// Copyright Epic Games, Inc. All Rights Reserved.

#include "TedsOutlinerItem.h"

#include "Elements/Columns/TypedElementLabelColumns.h"
#include "Elements/Columns/TypedElementMiscColumns.h"
#include "HAL/PlatformApplicationMisc.h"
#include "TedsOutlinerImpl.h"
#include "TedsTableViewerUtils.h"

#include "UObject/PropertyBagRepository.h"

#include "ActorFolders/ActorFolderColumns.h"
#include "Columns/TedsOutlinerColumns.h"
#include "DataStorage/Features.h"
#include "Elements/Columns/TypedElementCompatibilityColumns.h"
#include "Elements/Columns/TypedElementUIColumns.h"
#include "Elements/Interfaces/TypedElementDataStorageInterface.h"
#include "GameFramework/Actor.h"


#include "UObject/InstanceDataObjectUtils.h"

#define LOCTEXT_NAMESPACE "TedsOutliner"

namespace UE::Editor::Outliner
{
const FSceneOutlinerTreeItemType FTedsOutlinerTreeItem::Type(&ISceneOutlinerTreeItem::Type);

FTedsOutlinerTreeItem::FTedsOutlinerTreeItem(const DataStorage::RowHandle& InRowHandle,
	const TWeakPtr<const FTedsOutlinerImpl>& InTedsOutlinerImpl)
	: ISceneOutlinerTreeItem(Type)
	, RowHandle(InRowHandle)
	, TedsOutlinerImpl(InTedsOutlinerImpl)
{
	if (const TSharedPtr<const FTedsOutlinerImpl> TedsOutlinerImplPin = TedsOutlinerImpl.Pin())
	{
		const FTedsOutlinerExpansionStateBridge& Bridge = TedsOutlinerImplPin->GetExpansionStateBridge();
		
		if (Bridge.GetExpansionState.IsSet())
		{
			Flags.bIsExpanded = Bridge.GetExpansionState(RowHandle);
		}
	
	}
}

bool FTedsOutlinerTreeItem::IsValid() const
{
	return true; // TEDS-Outliner TODO: check with TEDS if the item is valid?
}

FSceneOutlinerTreeItemID FTedsOutlinerTreeItem::GetID() const
{
	return FSceneOutlinerTreeItemID(RowHandle);
}

FFolder::FRootObject FTedsOutlinerTreeItem::GetRootObject() const
{
	using namespace UE::Editor::DataStorage;
	ICoreProvider* Storage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName);
	if (!Storage)
	{
		return FFolder::GetInvalidRootObject();
	}

	if (const FRootObjectColumn* RootObjectColumn = Storage->GetColumn<FRootObjectColumn>(RowHandle))
	{
		return RootObjectColumn->RootObject;
	}

	return FFolder::GetInvalidRootObject();
}

FString FTedsOutlinerTreeItem::GetDisplayString() const
{
	using namespace UE::Editor::DataStorage;

	if (const TSharedPtr<const FTedsOutlinerImpl> TedsOutlinerImplPin = TedsOutlinerImpl.Pin())
	{
		if (const FTypedElementLabelColumn* LabelColumn = TedsOutlinerImplPin->GetStorage()->GetColumn<FTypedElementLabelColumn>(RowHandle))
		{
			return LabelColumn->Label;
		}
		if (const FUObjectIdNameColumn* NameColumn = TedsOutlinerImplPin->GetStorage()->GetColumn<FUObjectIdNameColumn>(RowHandle))
		{
			return NameColumn->IdName.ToString();
		}
	}
	

	return TEXT("TEDS Item");
}

bool FTedsOutlinerTreeItem::CanInteract() const
{
	if(!Flags.bInteractive)
	{
		return false;
	}
	return WeakSceneOutliner.Pin()->GetMode()->CanInteract(*this);
}

TSharedRef<SWidget> FTedsOutlinerTreeItem::GenerateLabelWidget(ISceneOutliner& Outliner,
	const STableRow<FSceneOutlinerTreeItemPtr>& InRow)
{
	if (TSharedPtr TedsOutlinerImplPin = TedsOutlinerImpl.Pin())
	{
		return TedsOutlinerImplPin->CreateLabelWidgetForItem(RowHandle, *this, InRow, CanInteract());
	}
	return SNullWidget::NullWidget;
}

void FTedsOutlinerTreeItem::GenerateContextMenu(UToolMenu* Menu, SSceneOutliner& Outliner)
{
	using namespace UE::Editor::DataStorage;
	ICoreProvider* DataStorageInterface = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName);

	if (FTedsOutlinerContextMenuColumn* Column = DataStorageInterface->GetColumn<FTedsOutlinerContextMenuColumn>(RowHandle))
	{
		if (Column->OnCreateContextMenu && Column->OnCreateContextMenu->IsBound())
		{
			Column->OnCreateContextMenu->Execute(Menu, Outliner);
		}
	}
}

void FTedsOutlinerTreeItem::OnExpansionChanged()
{
	if (TSharedPtr<const FTedsOutlinerImpl> TedsOutlinerImplPin = TedsOutlinerImpl.Pin())
	{
		const FTedsOutlinerExpansionStateBridge& Bridge = TedsOutlinerImplPin->GetExpansionStateBridge();
		
		if (Bridge.SetExpansionState.IsSet())
		{
			Bridge.SetExpansionState(RowHandle, Flags.bIsExpanded);
		}
	}
}

DataStorage::RowHandle FTedsOutlinerTreeItem::GetRowHandle() const
{
	return RowHandle;
}
} // namespace UE::Editor::Outliner

#undef LOCTEXT_NAMESPACE