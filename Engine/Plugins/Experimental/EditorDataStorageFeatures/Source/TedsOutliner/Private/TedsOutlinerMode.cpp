// Copyright Epic Games, Inc. All Rights Reserved.

#include "TedsOutlinerMode.h"

#include "Editor.h"
#include "EditorActorFolders.h"
#include "Elements/Columns/TypedElementCompatibilityColumns.h"
#include "Elements/Columns/TypedElementSelectionColumns.h"
#include "Elements/Interfaces/TypedElementWorldInterface.h"
#include "Framework/Commands/GenericCommands.h"
#include "Elements/Interfaces/TypedElementDataStorageCompatibilityInterface.h"
#include "Elements/Interfaces/TedsTypedElementBridgeInterface.h"
#include "DragAndDrop/TedsDragDropOp.h"
#include "DragAndDrop/TedsDragDropOpUtility.h"
#include "DragAndDrop/DropOperationInput.h"
#include "DragAndDrop/DropOperationSystem.h"
#include "ScopedTransaction.h"
#include "TedsOutlinerHierarchy.h"
#include "TedsOutlinerItem.h"
#include "ToolMenus.h"
#include "Columns/TedsOutlinerColumns.h"
#include "DragAndDrop/Widgets/WidgetDropHandler.h"
#include "Elements/Columns/TypedElementHandleColumn.h"
#include "Elements/Framework/TypedElementRegistry.h"
#include "LevelEditorViewport.h"
#include "SceneOutlinerFilters.h"
#include "TedsOutlinerHelpers.h"
#include "TedsOutlinerModeSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TedsOutlinerMode)

#define LOCTEXT_NAMESPACE "TedsOutlinerMode"

namespace UE::Editor::Outliner
{
	namespace Private
	{
		static FName ContextMenuName("TedsOutlinerContextMenu");
		
		static bool GetDropRowsFromPayload(FRowHandleArray& OutRows, const FSceneOutlinerDragDropPayload& Payload)
		{
			if (Payload.DraggedItems.IsEmpty()) // External drop
			{
				return DragAndDrop::GetRowsFromData(OutRows, Payload.SourceOperation);
			}
	
			Payload.ForEachItem<FTedsOutlinerTreeItem>([&OutRows](const FTedsOutlinerTreeItem& TedsItem)
			{
				OutRows.Add(TedsItem.GetRowHandle());
			});
			return true;
		}
	} // namespace Private
FTedsOutlinerMode::FTedsOutlinerMode(const FTedsOutlinerParams& InParams)
	: ISceneOutlinerMode(InParams.SceneOutliner)
{
	UTedsOutlinerConfig::Initialize();
	UTedsOutlinerConfig::Get()->LoadEditorConfig();

	// Get a MutableConfig here to force create a config for the current outliner if it doesn't exist
	const FTedsOutlinerModeConfig* SavedSettings = GetMutableConfig();

	// Create a local struct to use the default values if this outliner doesn't want to save configs
	FTedsOutlinerModeConfig LocalSettings;

	// If this outliner doesn't want to save config (OutlinerIdentifier is empty, use the defaults)
	if (SavedSettings)
	{
		LocalSettings = *SavedSettings;
	}
		
	TedsOutlinerImpl = MakeShared<FTedsOutlinerImpl>(InParams, this, false /* bHybridMode */);
	TedsOutlinerImpl->Init();
	TedsOutlinerImpl->OnSelectionChanged().AddRaw(this, &FTedsOutlinerMode::OnSelectionChanged);

	if (ICoreProvider* Storage = TedsOutlinerImpl->GetStorage())
	{
		if (FTedsOutlinerPendingItemActionsColumn* Column = Storage->GetColumn<FTedsOutlinerPendingItemActionsColumn>(TedsOutlinerImpl->GetOutlinerRowHandle()))
		{
			if (!Column->OnRegisterPendingItemActions)
			{
				Column->OnRegisterPendingItemActions = MakeShared<FOnRegisterPendingItemActions>();
			}
			Column->OnRegisterPendingItemActions->AddRaw(this, &FTedsOutlinerMode::RegisterPendingItemActions);
		}
	}

	bAlwaysFrameSelection = LocalSettings.bAlwaysFrameSelection;
	bCollapseOutlinerTreeOnNewSelection = LocalSettings.bCollapseOutlinerTreeOnNewSelection;

	bShowViewButton = InParams.bShowViewButton;
	// Convert ShowOptionsFilters to FilterInfoMap to appear in the Settings "Show" options menu
	for (const TSharedPtr<FTedsOutlinerFilter>& ShowFilter : InParams.ShowOptionsFilters)
	{
		if (ShowFilter)
		{
			ShowFilter->SetSceneOutlinerImpl(TedsOutlinerImpl);
			const FName ShowFilterName = FName(*ShowFilter->GetName());
			bool bSettingActive = ShowFilter->IsActiveByDefault();
			if (const bool* bSettingActivePtr = LocalSettings.ShowSettingsActive.Find(ShowFilterName))
			{
				if (const bool bSavedSettingActive = *bSettingActivePtr; bSettingActive != bSavedSettingActive)
				{
					bSettingActive = bSavedSettingActive;
					ShowFilter->ActiveStateChanged(bSavedSettingActive);
				}
			}
			
			FSceneOutlinerFilterInfo SceneOutlinerFilterInfo(ShowFilter->GetDisplayName(),
				ShowFilter->GetToolTipText(), bSettingActive,
				FCreateSceneOutlinerFilter::CreateLambda([]()
				{
					return MakeShared<TSceneOutlinerFilter<const ISceneOutlinerTreeItem>>(FSceneOutlinerFilter::EDefaultBehaviour::Pass);
				}));

			SceneOutlinerFilterInfo.OnToggle().AddLambda([this, ShowFilter, ShowFilterName](bool bIsActive)
			{
				ShowFilter->ActiveStateChanged(bIsActive);
				if (FTedsOutlinerModeConfig* Settings = GetMutableConfig())
				{
					Settings->ShowSettingsActive.FindOrAdd(ShowFilterName) = bIsActive;
					SaveConfig();
				}
			});
			FilterInfoMap.Add(ShowFilterName, SceneOutlinerFilterInfo);
		}
	}
	OnInitializeViewMenuExtender = InParams.OnInitializeViewMenuExtender;
	OnExtendOptionsMenu = InParams.OnExtendOptionsMenu;
	OnItemDoubleClickDelegate = InParams.OnItemDoubleClick;
	OnCanPopulateDelegate = InParams.OnCanPopulate;
	OnCustomAddToToolbarDelegate = InParams.OnCustomAddToToolbar;

	CanInteractSelectionOptions = FTypedElementSelectionOptions();
	CanInteractSelectionOptions.SetAllowHidden(true);
	CanInteractSelectionOptions.SetWarnIfLocked(false);

	TedsOutlinerImpl->IsItemCompatible().BindLambda([](const ISceneOutlinerTreeItem& Item)
	{
		return Item.IsA<FTedsOutlinerTreeItem>();
	});

	if (SceneOutliner && InParams.OnRegisterInteractiveFilters.IsBound())
	{
		InParams.OnRegisterInteractiveFilters.Execute(*SceneOutliner);
	}
}

FTedsOutlinerMode::~FTedsOutlinerMode()
{
	if (TedsOutlinerImpl)
	{
		if (DataStorage::ICoreProvider* Storage = TedsOutlinerImpl->GetStorage())
		{
			if (FTedsOutlinerPendingItemActionsColumn* Column =
				Storage->GetColumn<FTedsOutlinerPendingItemActionsColumn>(TedsOutlinerImpl->GetOutlinerRowHandle()))
			{
				if (Column->OnRegisterPendingItemActions)
				{
					Column->OnRegisterPendingItemActions->RemoveAll(this);
				}
			}
		}
		TedsOutlinerImpl->OnSelectionChanged().RemoveAll(this);
		TedsOutlinerImpl->UnregisterQueries();
	}
}

void FTedsOutlinerMode::Rebuild()
{
	TedsOutlinerItemCount = 0;
	Hierarchy = CreateHierarchy();
}

void FTedsOutlinerMode::OnItemAdded(FSceneOutlinerTreeItemPtr Item)
{
	++TedsOutlinerItemCount;

	// Replicate the flag handling in SSceneOutliner::AddItemToTree
	if (SceneOutliner && !PendingItemActionsByRow.IsEmpty() && Item.IsValid())
	{
		if (const FTedsOutlinerTreeItem* TedsItem = Item->CastTo<FTedsOutlinerTreeItem>())
		{
			uint8 Actions = 0;
			if (PendingItemActionsByRow.RemoveAndCopyValue(TedsItem->GetRowHandle(), Actions))
			{
				if (Actions & SceneOutliner::ENewItemAction::Select)
				{
					SceneOutliner->SetItemSelection(Item, true);
				}
				if ((Actions & SceneOutliner::ENewItemAction::Rename) && SceneOutliner->CanExecuteRenameRequest(*Item))
				{
					SceneOutliner->SetPendingRenameItem(Item);
				}
				if (Actions & (SceneOutliner::ENewItemAction::ScrollIntoView | SceneOutliner::ENewItemAction::Rename))
				{
					SceneOutliner->ScrollItemIntoView(Item);
				}
			}
		}
	}
}

void FTedsOutlinerMode::RegisterPendingItemActions(DataStorage::RowHandle Row, uint8 ItemActions)
{
	if (ItemActions == 0)
	{
		return;
	}
	PendingItemActionsByRow.FindOrAdd(Row) |= ItemActions;
}

void FTedsOutlinerMode::OnItemRemoved(FSceneOutlinerTreeItemPtr Item)
{
	--TedsOutlinerItemCount;
}

void FTedsOutlinerMode::OnSelectionChanged(ESelectInfo::Type SelectionType)
{
	TWeakObjectPtr<UTypedElementSelectionSet> WeakSelectionSet = TedsOutlinerImpl->GetSelectionSet();
	DataStorage::ICoreProvider* Storage = TedsOutlinerImpl->GetStorage();

	// Snapshot the current visual selection of rows without TypedElementHandles before SetSelection clears it.
	// (while it won't perform an action, it will persist its selection visually)
	FRowHandleArray PreviouslySelectedItems;
	for (const FSceneOutlinerTreeItemPtr& TreeItem: SceneOutliner->GetSelectedItems())
	{
		if(const FTedsOutlinerTreeItem* TedsItem = TreeItem->CastTo<FTedsOutlinerTreeItem>())
		{
			const RowHandle ItemRow = TedsItem->GetRowHandle();
			if (!Storage->HasColumns<DataStorage::Compatibility::FTypedElementColumn>(ItemRow))
			{
				PreviouslySelectedItems.Add(ItemRow);
			}
		}
	}
	PreviouslySelectedItems.Sort();

	// The selection in TEDS was changed, update the outliner to respond
	SceneOutliner->SetSelection([WeakSelectionSet, Storage, &PreviouslySelectedItems](ISceneOutlinerTreeItem& InItem) -> bool
	{
		UTypedElementSelectionSet* SelectionSet = WeakSelectionSet.Get();
		if (!SelectionSet)
		{
			return false;
		}

		if(const FTedsOutlinerTreeItem* TedsItem = InItem.CastTo<FTedsOutlinerTreeItem>())
		{
			const DataStorage::RowHandle RowHandle = TedsItem->GetRowHandle();

			FTypedElementHandle ElementHandle;
			if (DataStorage::Compatibility::FTypedElementColumn* Column = Storage->GetColumn<DataStorage::Compatibility::FTypedElementColumn>(RowHandle))
			{
				ElementHandle = Column->Handle;
			}

			if (ElementHandle)
			{
				return SelectionSet->IsElementSelected(ElementHandle, FTypedElementIsSelectedOptions());
			}

			// Most items should have a Typed Element Handle, with a few exceptions (ex: UWorlds, ULevels)
			return PreviouslySelectedItems.Contains(RowHandle);
		}
		return false;
	});

	if (SelectionType == ESelectInfo::Direct && !SceneOutliner->GetSelectedItems().IsEmpty())
		{
			if (bCollapseOutlinerTreeOnNewSelection)
			{
				SceneOutliner->CollapseAll();
			}

			if (bAlwaysFrameSelection || bCollapseOutlinerTreeOnNewSelection)
			{
			// Find the tree item matching the bottom (most recently selected) TEDS element
			if (const UTypedElementSelectionSet* SelectionSet = WeakSelectionSet.Get())
			{
				if (const FTypedElementListConstRef ElementList = SelectionSet->GetElementList(); ElementList->Num() > 0)
				{
					const FTypedElementHandle BottomHandle = ElementList->GetElementHandleAt(ElementList->Num() - 1);
					if (TTypedElement<ITedsTypedElementBridgeInterface> BridgeInterface =  UTypedElementRegistry::GetInstance()->GetElement<ITedsTypedElementBridgeInterface>(BottomHandle))
					{
						RowHandle RowHandle = BridgeInterface.GetRowHandle();
						if (FSceneOutlinerTreeItemPtr TreeItem = SceneOutliner->GetTreeItem(RowHandle, true))
						{
							SceneOutliner->ScrollItemIntoView(TreeItem);
						}
					}
				}
			}
		}
	}

	if (FTedsOutlinerSelectionChangeColumn* SelectionChangeColumn = Storage->GetColumn<FTedsOutlinerSelectionChangeColumn>(TedsOutlinerImpl->GetOutlinerRowHandle()))
	{
		if (SelectionChangeColumn->OnSelectionChanged)
		{
			SelectionChangeColumn->OnSelectionChanged->Broadcast(SelectionType);
		}
	}
}
void FTedsOutlinerMode::SynchronizeSelection()
{
	OnSelectionChanged(ESelectInfo::Direct);
}

void FTedsOutlinerMode::OnItemDoubleClick(FSceneOutlinerTreeItemPtr Item)
{
	if (Item->CanInteract())
	{
		if (const FTedsOutlinerTreeItem* TedsItem = Item->CastTo<FTedsOutlinerTreeItem>())
		{
			if (OnItemDoubleClickDelegate.IsBound() &&
				OnItemDoubleClickDelegate.Execute(TedsOutlinerImpl->GetStorage(), TedsOutlinerImpl->GetOutlinerRowHandle(), TedsItem->GetRowHandle()).IsEventHandled())
			{
				return;
			}
		}

		// Default case: Frame cameras to selection
		if (UTypedElementSelectionSet* SelectionSet = TedsOutlinerImpl->GetSelectionSet())
		{
			constexpr bool bActiveViewportOnly = false;
			GEditor->MoveViewportCamerasToElement(SelectionSet, bActiveViewportOnly);
		}
	}
}

void FTedsOutlinerMode::OnItemSelectionChanged(FSceneOutlinerTreeItemPtr Item, ESelectInfo::Type SelectionType, const FSceneOutlinerItemSelection& Selection)
{
	if(SelectionType == ESelectInfo::Direct)
	{
		return; // Direct selection means we selected from outside the Outliner i.e through TEDS, so we don't need to redo the column addition
	}

	TArray<DataStorage::RowHandle> RowHandles;
	
	// The selection in the Outliner changed, update TEDS
	Selection.ForEachItem([&RowHandles](FSceneOutlinerTreeItemPtr& Item)
	{
		if(FTedsOutlinerTreeItem* TedsItem = Item->CastTo<FTedsOutlinerTreeItem>())
		{
			RowHandles.Add(TedsItem->GetRowHandle());
		}
	});

	TedsOutlinerImpl->SetPendingSelectionType(SelectionType);
	TedsOutlinerImpl->SetSelection(RowHandles);
}

bool FTedsOutlinerMode::CanInteract(const ISceneOutlinerTreeItem& Item) const
{
	const TWeakObjectPtr<UTypedElementSelectionSet> WeakSelectionSet = TedsOutlinerImpl->GetSelectionSet();
	const ICoreProvider* Storage = TedsOutlinerImpl->GetStorage();
	
	if (const UTypedElementSelectionSet* SelectionSet = WeakSelectionSet.Get(); SelectionSet && Storage)
	{
		if(const FTedsOutlinerTreeItem* TedsItem = Item.CastTo<FTedsOutlinerTreeItem>())
		{
			const RowHandle RowHandle = TedsItem->GetRowHandle();
	
			if (const Compatibility::FTypedElementColumn* Column = TedsOutlinerImpl->GetStorage()->GetColumn<Compatibility::FTypedElementColumn>(RowHandle))
			{
				return WeakSelectionSet.Get()->CanSelectElement(Column->Handle, CanInteractSelectionOptions);
			}
			
		}
	}
	return true;
}

TSharedPtr<FDragDropOperation> FTedsOutlinerMode::CreateDragDropOperation(const FPointerEvent& MouseEvent, const TArray<FSceneOutlinerTreeItemPtr>& InTreeItems) const
{
	FRowHandleArray DraggedRowHandles;

	for(const FSceneOutlinerTreeItemPtr& Item :InTreeItems)
	{
		const FTedsOutlinerTreeItem* TedsItem = Item->CastTo<FTedsOutlinerTreeItem>();
		if(ensureMsgf(TedsItem, TEXT("We should only have TEDS items in the TEDS Outliner")))
		{
			DraggedRowHandles.Add(TedsItem->GetRowHandle());
		}
	}

	return Widgets::FTedsDragDropOp::New(MoveTemp(DraggedRowHandles));
}

bool FTedsOutlinerMode::ParseDragDrop(FSceneOutlinerDragDropPayload& OutPayload, const FDragDropOperation& Operation) const
{
	if (Operation.IsOfType<Widgets::FTedsDragDropOp>())
	{
		const Widgets::FTedsDragDropOp& TedsOperation = static_cast<const Widgets::FTedsDragDropOp&>(Operation);

		for (RowHandle SourceRow : TedsOperation.GetRows())
		{
			if (FSceneOutlinerTreeItemPtr Item = SceneOutliner->GetTreeItem(SourceRow))
			{
				OutPayload.DraggedItems.Add(Item);
			}
		}
	}
	return true; // Always return true since we support external drops as well
}
	
FSceneOutlinerDragValidationInfo FTedsOutlinerMode::ValidateDrop(const ISceneOutlinerTreeItem& DropTarget, const FSceneOutlinerDragDropPayload& Payload) const
{
	if (ICoreProvider* Storage = TedsOutlinerImpl->GetStorage())
	{
		if (UDropOperationSystem* DropSystem = Storage->FindFactory<UDropOperationSystem>())
		{
			FRowHandleArray DraggedRowHandles;
			if (Private::GetDropRowsFromPayload(DraggedRowHandles, Payload) && !DraggedRowHandles.IsEmpty())
			{
				if (const ICompatibilityProvider* StorageCompatibility = TedsOutlinerImpl->GetStorageCompatibility())
				{
					if (RowHandle TargetRow = Helpers::GetRowHandleFromOutlinerItem(*Storage, *StorageCompatibility, TedsOutlinerImpl->GetOutlinerRowHandle(), DropTarget); TargetRow != InvalidRowHandle)
					{
						FRowHandleArray InputRows;
						if (DropSystem->CreateInputRows(InputRows, DraggedRowHandles.GetRows(), TargetRow, true))
						{
							bool bResult = DropSystem->Test(InputRows.GetRows()) == InputRows.Num();

							FText Description;
							for (RowHandle InputRow : InputRows.GetRows())
							{
								// Find an error text when we failed and a success text if we succeeded.
								if (bResult != Storage->HasColumns<Operations::FTestResultTag>(InputRow))
								{
									continue;
								}

								Description = Operations::Utilities::GetDescription(*Storage, InputRow);
								if (!Description.IsEmpty())
								{
									break;
								}
							}

							DropSystem->RemoveInputRows(InputRows.GetRows());

							return { bResult ? ESceneOutlinerDropCompatibility::CompatibleGeneric : ESceneOutlinerDropCompatibility::IncompatibleGeneric, MoveTemp(Description) };
						}
					}
				}
			}
		}
	}

	return { ESceneOutlinerDropCompatibility::IncompatibleGeneric, LOCTEXT("DropUnsupported", "Drop operation is unsupported.") };
}

void FTedsOutlinerMode::OnDrop(ISceneOutlinerTreeItem& DropTarget, const FSceneOutlinerDragDropPayload& Payload, const FSceneOutlinerDragValidationInfo& ValidationInfo) const
{
	if (ValidationInfo.CompatibilityType != ESceneOutlinerDropCompatibility::CompatibleGeneric)
	{
		return;
	}
		
	if (ICoreProvider* Storage = TedsOutlinerImpl->GetStorage())
	{
		if (UDropOperationSystem* DropSystem = Storage->FindFactory<UDropOperationSystem>())
		{
			FRowHandleArray DraggedRowHandles;
			if (Private::GetDropRowsFromPayload(DraggedRowHandles, Payload) && !DraggedRowHandles.IsEmpty())
			{
				if (const ICompatibilityProvider* StorageCompatibility = TedsOutlinerImpl->GetStorageCompatibility())
				{
					if (RowHandle TargetRow = Helpers::GetRowHandleFromOutlinerItem(*Storage, *StorageCompatibility, TedsOutlinerImpl->GetOutlinerRowHandle(), DropTarget); TargetRow != InvalidRowHandle)
					{
						FRowHandleArray InputRows;
						if (DropSystem->CreateInputRows(InputRows, DraggedRowHandles.GetRows(), TargetRow, true))
						{
							ON_SCOPE_EXIT{ DropSystem->RemoveInputRows(InputRows.GetRows()); };
							{
								FScopedTransaction Transaction(LOCTEXT("Transaction_Drop", "Drop on Outliner."));
								if (DropSystem->Apply(InputRows.GetRows()) != InputRows.Num())
								{
									FWidgetDropHandler::ShowErrorNotifications(*Storage, InputRows.GetRows());
								}
							}
							FWidgetDropHandler::ShowErrorNotifications(*Storage, InputRows.GetRows());
						}
					}
				}
			}
		}
	}
}

TSharedPtr<SWidget> FTedsOutlinerMode::CreateContextMenu()
{
	UToolMenus* ToolMenus = UToolMenus::Get();

	if (!ToolMenus->IsMenuRegistered(Private::ContextMenuName))
	{
		UToolMenu* Menu = ToolMenus->RegisterMenu(Private::ContextMenuName);
		Menu->AddDynamicSection("DynamicHierarchySection", FNewToolMenuDelegate::CreateLambda([](UToolMenu* InMenu)
		{
			if(UTedsOutlinerMenuContext* TedsOutlinerMenuContext = InMenu->FindContext<UTedsOutlinerMenuContext>())
			{
				if(SSceneOutliner* SceneOutliner = TedsOutlinerMenuContext->OwningSceneOutliner)
				{
					TArray<FSceneOutlinerTreeItemPtr> Selection = SceneOutliner->GetTree().GetSelectedItems();

					if(Selection.Num() == 1)
					{
						Selection[0]->GenerateContextMenu(InMenu, *SceneOutliner);
					}
				}
			}
			
		}));
	}

	UTedsOutlinerMenuContext* TedsOutlinerMenuContext = NewObject<UTedsOutlinerMenuContext>();
	TedsOutlinerMenuContext->OwningSceneOutliner = SceneOutliner;
	
	FToolMenuContext MenuContext;
	MenuContext.AddObject(TedsOutlinerMenuContext);

	// Allow the ModifyContextMenu callback to modify the context and menu name
	FName MutableMenuName = Private::ContextMenuName;
	SceneOutliner->GetSharedData().ModifyContextMenu.ExecuteIfBound(MutableMenuName, MenuContext);

	return UToolMenus::Get()->GenerateWidget(MutableMenuName, MenuContext);
}

void FTedsOutlinerMode::Tick()
{
	TedsOutlinerImpl->Tick();
}

bool FTedsOutlinerMode::CanPopulate() const
{
	if (OnCanPopulateDelegate.IsBound())
	{
		return OnCanPopulateDelegate.Execute(TedsOutlinerImpl->GetStorage(), TedsOutlinerImpl->GetOutlinerRowHandle());
	}
	return true;
}

TUniquePtr<ISceneOutlinerHierarchy> FTedsOutlinerMode::CreateHierarchy()
{
	return MakeUnique<FTedsOutlinerHierarchy>(this, TedsOutlinerImpl.ToSharedRef());
}

bool FTedsOutlinerMode::ShowViewButton() const
{
	return bShowViewButton;
}

void FTedsOutlinerMode::InitializeViewMenuExtender(TSharedPtr<FExtender> Extender)
{
	Extender->AddMenuExtension(SceneOutliner::ExtensionHooks::Show, EExtensionHook::After, nullptr, FMenuExtensionDelegate::CreateLambda([this](FMenuBuilder& MenuBuilder)
	{
		MenuBuilder.BeginSection("OutlinerSelectionOptions", LOCTEXT("OptionsHeading", "Options"));

		MenuBuilder.AddMenuEntry(
			LOCTEXT("AlwaysFrameSelectionLabel", "Always Frame Selection"),
			LOCTEXT("AlwaysFrameSelectionTooltip", "When enabled, selecting an object in the Viewport also scrolls to that item in the Outliner."),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateRaw(this, &FTedsOutlinerMode::OnToggleAlwaysFrameSelection),
				FCanExecuteAction(),
				FIsActionChecked::CreateRaw(this, &FTedsOutlinerMode::ShouldAlwaysFrameSelection)
			),
			NAME_None,
			EUserInterfaceActionType::ToggleButton
		);

		MenuBuilder.AddMenuEntry(
			LOCTEXT("CollapseOutlinerTreeOnNewSelectionLabel", "Collapse Hierarchy on Viewport Selection"),
			LOCTEXT("CollapseOutlinerTreeOnNewSelectionTooltip", "When enabled, all items except the one that was just selected are collapsed."),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateRaw(this, &FTedsOutlinerMode::OnToggleCollapseOutlinerTreeOnNewSelection),
				FCanExecuteAction(),
				FIsActionChecked::CreateRaw(this, &FTedsOutlinerMode::CollapseOutlinerTreeOnNewSelection)
			),
			NAME_None,
			EUserInterfaceActionType::ToggleButton
		);

		if (OnExtendOptionsMenu.IsBound())
		{
			OnExtendOptionsMenu.Execute(MenuBuilder);
		}

		MenuBuilder.EndSection();
	}));

	if (OnInitializeViewMenuExtender.IsBound())
	{
		OnInitializeViewMenuExtender.Execute(Extender);
	}
}

bool FTedsOutlinerMode::ShowFilterOptions() const
{
	// Use the same bool as ShowViewButton so we can have any show filters appear in the settings menu, it is better
	// to give an empty ShowOptionsFilters in the params if you do not want to show any settings.
	return bShowViewButton;
}

void FTedsOutlinerMode::BindCommands(const TSharedRef<FUICommandList>& OutCommandList)
{
	OutCommandList->MapAction(
		FGenericCommands::Get().Rename,
		FExecuteAction::CreateRaw(this, &FTedsOutlinerMode::OnExecuteRename),
		FCanExecuteAction::CreateRaw(this, &FTedsOutlinerMode::CanExecuteRename));
}

void FTedsOutlinerMode::OnExecuteRename()
{
	const FSceneOutlinerItemSelection& Selection = SceneOutliner->GetSelection();
	if (Selection.Num() == 1)
	{
		FSceneOutlinerTreeItemPtr ItemToRename = Selection.SelectedItems[0].Pin();

		if (ItemToRename.IsValid() && CanRenameItem(*ItemToRename) && ItemToRename->CanInteract())
		{
			SceneOutliner->SetPendingRenameItem(ItemToRename);
			SceneOutliner->ScrollItemIntoView(ItemToRename);
		}
	}
}

void FTedsOutlinerMode::CustomAddToToolbar(TSharedPtr<class SHorizontalBox> Toolbar)
{
	if (OnCustomAddToToolbarDelegate.IsBound())
	{
		OnCustomAddToToolbarDelegate.Execute(Toolbar, TedsOutlinerImpl->GetOutlinerRowHandle());
	}
}

bool FTedsOutlinerMode::CanExecuteRename()
{
	const FSceneOutlinerItemSelection& Selection = SceneOutliner->GetSelection();
	if (Selection.Num() == 1)
	{
		FSceneOutlinerTreeItemPtr ItemToRename = Selection.SelectedItems[0].Pin();
		return ItemToRename.IsValid() && CanRenameItem(*ItemToRename) && ItemToRename->CanInteract();
	}
	return false;
}

bool FTedsOutlinerMode::CanRenameItem(const ISceneOutlinerTreeItem& Item) const
{
	// RenameRequestEvent is bound during label widget construction (after AddItemToTree), so gate on type.
	return Item.IsValid() && Item.IsA<FTedsOutlinerTreeItem>();
}

bool FTedsOutlinerMode::CanDelete() const
{
	if (UTypedElementRegistry* Registry = UTypedElementRegistry::GetInstance())
	{
		if (UTypedElementSelectionSet* SelectionSet = TedsOutlinerImpl->GetSelectionSet())
		{
			bool bCanDelete = true;
			SelectionSet->ForEachSelectedElementHandle([&Registry, &bCanDelete](const FTypedElementHandle& Handle)
			{
				if (TTypedElement<ITypedElementWorldInterface> WorldElement = Registry->GetElement<ITypedElementWorldInterface>(Handle))
				{
					bCanDelete &= WorldElement.CanDeleteElement();
				}
				return true;
			});
	
			return bCanDelete;
		}
	}
	return false;
}

bool FTedsOutlinerMode::CanCut() const
{
	if (UTypedElementRegistry* Registry = UTypedElementRegistry::GetInstance())
	{
		if (UTypedElementSelectionSet* SelectionSet = TedsOutlinerImpl->GetSelectionSet())
		{
			bool bCanCut = true;
			SelectionSet->ForEachSelectedElementHandle([&Registry, &bCanCut](const FTypedElementHandle& Handle)
			{
				if (TTypedElement<ITypedElementWorldInterface> WorldElement = Registry->GetElement<ITypedElementWorldInterface>(Handle))
				{
					bCanCut &= WorldElement.CanCopyElement();
				}
				return true;
			});
		
			return bCanCut;
		}
	}
	return false;
}

bool FTedsOutlinerMode::CanCopy() const
{
	if (UTypedElementRegistry* Registry = UTypedElementRegistry::GetInstance())
	{
		if (UTypedElementSelectionSet* SelectionSet = TedsOutlinerImpl->GetSelectionSet())
		{
			bool bCanCopy = true;
			SelectionSet->ForEachSelectedElementHandle([&Registry, &bCanCopy](const FTypedElementHandle& Handle)
			{
				if (TTypedElement<ITypedElementWorldInterface> WorldElement = Registry->GetElement<ITypedElementWorldInterface>(Handle))
				{
					bCanCopy &= WorldElement.CanCopyElement();
				}
				return true;
			});
			
			return bCanCopy;
		}
	}
	return false;
}

bool FTedsOutlinerMode::CanPaste() const
{
	if (UTypedElementRegistry* Registry = UTypedElementRegistry::GetInstance())
	{
		if (UTypedElementSelectionSet* SelectionSet = TedsOutlinerImpl->GetSelectionSet())
		{
			bool bCanPaste = true;
			SelectionSet->ForEachSelectedElementHandle([&Registry, &bCanPaste](const FTypedElementHandle& Handle)
			{
				if (TTypedElement<ITypedElementWorldInterface> WorldElement = Registry->GetElement<ITypedElementWorldInterface>(Handle))
				{
					// TODO: There isn't a CanPasteElement and we can't get the paste importer from here, so just assuming CanCopy implies CanPaste for now
					bCanPaste &= WorldElement.CanCopyElement();
				}
				return true;
			});
	
			return bCanPaste;
		}
	}
	return false;
}

FFolder::FRootObject FTedsOutlinerMode::GetRootObject() const
{
	if (TedsOutlinerImpl)
	{
		if (ICoreProvider* Storage = TedsOutlinerImpl->GetStorage())
		{
			if (const FTypedElementWorldColumn* WorldColumn = Storage->GetColumn<FTypedElementWorldColumn>(TedsOutlinerImpl->GetOutlinerRowHandle()))
			{
				if (UWorld* World = WorldColumn->World.Get())
				{
					return FFolder::GetWorldRootFolder(World).GetRootObject();
				}
			}
		}
	}
	return ISceneOutlinerMode::GetRootObject();
}

bool FTedsOutlinerMode::CanCustomizeToolbar() const
{
	return OnCustomAddToToolbarDelegate.IsBound();
}

int32 FTedsOutlinerMode::GetFilteredRowCount() const
{
	return TedsOutlinerItemCount;
}

struct FTedsOutlinerModeConfig* FTedsOutlinerMode::GetMutableConfig()
{
	FName OutlinerIdentifier = SceneOutliner->GetOutlinerIdentifier();

	if (OutlinerIdentifier.IsNone())
	{
		return nullptr;
	}

	return &UTedsOutlinerConfig::Get()->TedsOutliners.FindOrAdd(OutlinerIdentifier);
}


const FTedsOutlinerModeConfig* FTedsOutlinerMode::GetConstConfig() const
{
	FName OutlinerIdentifier = SceneOutliner->GetOutlinerIdentifier();

	if (OutlinerIdentifier.IsNone())
	{
		return nullptr;
	}

	return UTedsOutlinerConfig::Get()->TedsOutliners.Find(OutlinerIdentifier);
}

void FTedsOutlinerMode::SaveConfig()
{
	UTedsOutlinerConfig::Get()->SaveEditorConfig();
}

void FTedsOutlinerMode::OnToggleAlwaysFrameSelection()
{
	bAlwaysFrameSelection = !bAlwaysFrameSelection;
		
	if(FTedsOutlinerModeConfig* Settings = GetMutableConfig())
	{
		Settings->bAlwaysFrameSelection = bAlwaysFrameSelection;
		SaveConfig();
	}
}

bool FTedsOutlinerMode::ShouldAlwaysFrameSelection() const
{
	return bAlwaysFrameSelection;
}

void FTedsOutlinerMode::OnToggleCollapseOutlinerTreeOnNewSelection()
{
	if (FTedsOutlinerModeConfig* Settings = GetMutableConfig())
	{
		Settings->bCollapseOutlinerTreeOnNewSelection = !Settings->bCollapseOutlinerTreeOnNewSelection;
		bCollapseOutlinerTreeOnNewSelection = Settings->bCollapseOutlinerTreeOnNewSelection;
		SaveConfig();
	}
}

bool FTedsOutlinerMode::CollapseOutlinerTreeOnNewSelection() const
{
	return bCollapseOutlinerTreeOnNewSelection;
}

FText FTedsOutlinerMode::GetStatusText() const
{
	static const FText ShowingAllObjectsFmt = LOCTEXT("ShowingAllObjectsFmt", "{Count} {Count}|plural(one=Object,other=Objects)");
	static const FText ShowingAllObjectsSelectedFmt = LOCTEXT("ShowingAllObjectsSelectedFmt", "{Count} {Count}|plural(one=Object,other=Objects) ({Selected} selected)");
	static const FText ShowingNoObjectsFmt = LOCTEXT("ShowingNoObjectsFmt", "No matching objects ({0} total)");
	static const FText ShowingOnlySomeObjectsSelectedFmt = LOCTEXT("ShowingOnlySomeObjectsSelectedFmt", "Showing {Current} of {Total} {Total}|plural(one=Object,other=Objects) ({Selected} selected)");
	static const FText ShowingOnlySomeObjectsFmt = LOCTEXT("ShowingOnlySomeObjectsFmt", "Showing {Current} of {Total} {Total}|plural(one=Object,other=Objects)");

	const int32 TotalCount = TedsOutlinerImpl->GetTotalRowCount();
	const int32 CurrentCount = TedsOutlinerItemCount;
	const int32 SelectedItemCount = SceneOutliner->GetSelection().Num();
	const bool bIsTextFilterActive = SceneOutliner->IsTextFilterActive();

	if (!bIsTextFilterActive)
	{
		if (SelectedItemCount == 0)
		{
			return FText::FormatNamed(ShowingAllObjectsFmt,
				TEXT("Count"), FText::AsNumber(CurrentCount));
		}
		return FText::FormatNamed(ShowingAllObjectsSelectedFmt,
			TEXT("Count"), FText::AsNumber(CurrentCount),
			TEXT("Selected"), FText::AsNumber(SelectedItemCount));
	}
	if (CurrentCount == 0)
	{
		return FText::Format(ShowingNoObjectsFmt,
			FText::AsNumber(TotalCount));
	}
	if (SelectedItemCount != 0)
	{
		return FText::FormatNamed(ShowingOnlySomeObjectsSelectedFmt,
			TEXT("Current"), FText::AsNumber(CurrentCount),
			TEXT("Total"), FText::AsNumber(TotalCount),
			TEXT("Selected"), FText::AsNumber(SelectedItemCount));
	}

	return FText::FormatNamed(ShowingOnlySomeObjectsFmt,
		TEXT("Current"), FText::AsNumber(CurrentCount),
		TEXT("Total"), FText::AsNumber(TotalCount));

}

FSlateColor FTedsOutlinerMode::GetStatusTextColor() const
{
	const bool bIsTextFilterActive = SceneOutliner->IsTextFilterActive();
	if (!bIsTextFilterActive)
	{
		return FSlateColor::UseForeground();
	}
	const int32 CurrentCount = TedsOutlinerItemCount;
	if (CurrentCount == 0)
	{
		return FAppStyle::Get().GetSlateColor("Colors.AccentRed");
	}
	return FAppStyle::Get().GetSlateColor("Colors.AccentGreen");
}

bool FTedsOutlinerMode::ShowStatusBar() const
{
	return true;
}

} // namespace UE::Editor::Outliner

#undef LOCTEXT_NAMESPACE
