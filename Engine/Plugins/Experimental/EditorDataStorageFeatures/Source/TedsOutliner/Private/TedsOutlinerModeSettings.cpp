// Copyright Epic Games, Inc. All Rights Reserved.

#include "TedsOutlinerModeSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TedsOutlinerModeSettings)

TObjectPtr<UTedsOutlinerConfig> UTedsOutlinerConfig::Instance = nullptr;

void UTedsOutlinerConfig::Initialize()
{
	if(!Instance)
	{
		Instance = NewObject<UTedsOutlinerConfig>(); 
		Instance->AddToRoot();
	}
}
