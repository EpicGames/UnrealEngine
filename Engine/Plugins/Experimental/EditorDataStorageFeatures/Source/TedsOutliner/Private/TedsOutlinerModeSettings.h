// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "EditorConfigBase.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UObjectGlobals.h"

#include "TedsOutlinerModeSettings.generated.h"

class FName;
class UObject;

USTRUCT()
struct FTedsOutlinerModeConfig
{
	GENERATED_BODY()

public:
	/** Map containing the state of the given show settings if changed from their default */
	UPROPERTY()
	TMap<FName, bool> ShowSettingsActive = {};

	/** Map containing the state of the given option settings if changed from their default */
	UPROPERTY()
	TMap<FName, bool> OptionSettingsActive = {};

	/** True when the Scene Outliner updates when an actor is selected in the viewport */
	UPROPERTY()
	bool bAlwaysFrameSelection = true;

	/** True if we want to collapse Outliner tree on new selection, except for the item that was just selected */
	UPROPERTY()
	bool bCollapseOutlinerTreeOnNewSelection = false;
};

UCLASS(EditorConfig="TedsOutlinerMode")
class UTedsOutlinerConfig : public UEditorConfigBase
{
	GENERATED_BODY()
	
public:

	static void Initialize();
	static UTedsOutlinerConfig* Get() { return Instance; }

	UPROPERTY(meta=(EditorConfig))
	TMap<FName, FTedsOutlinerModeConfig> TedsOutliners;

private:

	static TObjectPtr<UTedsOutlinerConfig> Instance;
};