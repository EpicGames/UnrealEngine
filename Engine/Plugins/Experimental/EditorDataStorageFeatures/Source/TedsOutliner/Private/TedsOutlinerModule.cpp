// Copyright Epic Games, Inc. All Rights Reserved.

#include "TedsOutlinerModule.h"

#include "Engine/Engine.h"
#include "Engine/World.h"
#include "LevelEditor.h"
#include "SceneOutlinerHelpers.h"
#include "SceneOutlinerPublicTypes.h"
#include "TedsOutlinerMode.h"
#include "TedsOutlinerFilter.h"
#include "WorkspaceMenuStructureModule.h"
#include "WorkspaceMenuStructure.h"
#include "Elements/Columns/TypedElementCompatibilityColumns.h"
#include "Elements/Columns/TypedElementMiscColumns.h"
#include "Elements/Columns/TypedElementPackageColumns.h"
#include "Elements/Columns/TypedElementTypeInfoColumns.h"
#include "Elements/Columns/TypedElementVisibilityColumns.h"
#include "Columns/TedsActorWorldPartitionColumns.h"
#include "Columns/TedsActorMobilityColumns.h"
#include "Columns/TedsActorSocketColumns.h"
#include "Columns/TedsActorUncachedLightsColumns.h"
#include "WorldPartition/WorldPartition.h"
#include "TedsAlertColumns.h"
#include "DataStorage/Features.h"
#include "Elements/Interfaces/TypedElementDataStorageInterface.h"
#include "Elements/Framework/TypedElementQueryContext.h"
#include "Modules/ModuleManager.h"
#include "Compatibility/SceneOutlinerTedsBridge.h"
#include "Compatibility/SceneOutlinerRowHandleColumn.h"
#include "Layers/Columns/LayersColumns.h"
#include "Factories/TedsEditorHierarchyFactory.h"
#include "Editor.h"
#include "ActorDesc/TedsActorDescColumns.h"
#include "TedsOutlinerHelpers.h"
#include "Operations/DragAndDrop/RegisterDropOperations.h"
#include "Widgets/Docking/SDockTab.h"

#define LOCTEXT_NAMESPACE "TedsOutlinerModule"

namespace UE::Editor::Outliner
{
	namespace Private
	{
		static bool bUseNewRevisionControlWidgets = false;
		static bool bUseFavoritesWidgets = false;
	} // namespace Private

	void RefreshLevelEditorTedsOutliner(bool bAlwaysInvoke)
	{
		FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>("LevelEditor");
		FTedsOutlinerModule& TedsOutlinerModule = FModuleManager::GetModuleChecked<FTedsOutlinerModule>("TedsOutliner");
		FName TabId = TedsOutlinerModule.GetTedsOutlinerTabName();

		TSharedPtr<FTabManager> LevelEditorTabManager = LevelEditorModule.GetLevelEditorTabManager();
		if(LevelEditorTabManager.IsValid() && (bAlwaysInvoke || LevelEditorTabManager->FindExistingLiveTab(TabId)))
		{
			LevelEditorTabManager->TryInvokeTab(TabId);
		}
	}
	
	static FAutoConsoleVariableRef CVarUseNewRevisionControlWidgets(
		TEXT("TEDS.UI.UseNewRevisionControlWidgets"),
		Private::bUseNewRevisionControlWidgets,
		TEXT("Use new TEDS-based source control widgets in the Outliner (requires TEDS-Outliner to be enabled)")
		, FConsoleVariableDelegate::CreateLambda([](IConsoleVariable*)
		{
			RefreshLevelEditorTedsOutliner(false);
		}));

	static FAutoConsoleVariableRef CVarUseFavoritesWidgets(
		TEXT("UnifiedFavorites.LevelEditorOutliner.Widget"),
		Private::bUseFavoritesWidgets,
		TEXT("Use unified favorites in the Level Editor Outliner (requires TEDS-Outliner to be enabled)")
		, FConsoleVariableDelegate::CreateLambda([](IConsoleVariable*)
			{
				RefreshLevelEditorTedsOutliner(false);
			}));

	// CVar to summon the TEDS-Outliner as a separate tab
	static FAutoConsoleCommand OpenTableViewerConsoleCommand(
		TEXT("TEDS.UI.OpenTedsOutliner"),
		TEXT("Spawn the test TEDS-Outliner Integration."),
		FConsoleCommandWithArgsDelegate::CreateLambda([](const TArray<FString>& Args)
		{
			RefreshLevelEditorTedsOutliner(true);
		}));

FTedsOutlinerModule::FTedsOutlinerModule()
{
}

TSharedRef<ISceneOutliner> FTedsOutlinerModule::CreateTedsOutliner(const FSceneOutlinerInitializationOptions& InInitOptions, const FTedsOutlinerParams& InInitTedsOptions) const
{
	using namespace UE::Editor::DataStorage;
	ensureMsgf(AreEditorDataStorageFeaturesEnabled(), TEXT("Unable to initialize the Teds-Outliner before TEDS itself is initialized."));
	
	FSceneOutlinerInitializationOptions InitOptions(InInitOptions);
	FTedsOutlinerParams InitTedsOptions(InInitTedsOptions);

	InitOptions.ModeFactory = FCreateSceneOutlinerMode::CreateLambda([&InitTedsOptions](SSceneOutliner* Outliner)
	{
		InitTedsOptions.SceneOutliner = Outliner;
		
		return new FTedsOutlinerMode(InitTedsOptions);
	});
	
	// Add the custom column that displays row handles
	if (InInitTedsOptions.bShowRowHandleColumn)
	{
		// Explicitly set the priority Index to one less than the Label Priority (10) since we cannot use priority groups (not
		// passed in as a TEDS Column), and we want it to appear after high-priority icon columns but before the Label Column.
		constexpr int RowHandleColumnPriorityIndex = 9;
		
		InitOptions.ColumnMap.Add(FSceneOutlinerRowHandleColumn::GetID(),
			FSceneOutlinerColumnInfo(ESceneOutlinerColumnVisibility::Invisible, RowHandleColumnPriorityIndex,
				FCreateSceneOutlinerColumn::CreateLambda([](ISceneOutliner& InSceneOutliner)
				{
					return MakeShareable(new FSceneOutlinerRowHandleColumn(InSceneOutliner));
				})));
	}
	
	TSharedRef<ISceneOutliner> TedsOutlinerShared = SNew(SSceneOutliner, InitOptions);
	
	return TedsOutlinerShared;
}

void FTedsOutlinerModule::StartupModule()
{
	IModuleInterface::StartupModule();

	TedsOutlinerTabName = TEXT("LevelEditorTedsOutliner");
	RegisterLevelEditorTedsOutlinerTab();

	auto OnDataStorage = [this]
	{
		Storage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName);
		if (Storage)
		{
			Operations::RegisterDropOperations(*Storage);
		}
	};
	
	if (AreEditorDataStorageFeaturesEnabled())
	{
		OnDataStorage();
	}
	else
	{
		TedsInitializedHandle = OnEditorDataStorageFeaturesEnabled().AddLambda(OnDataStorage);
	}

	// Register generic (non-SceneGraph) drop operations on the TEDS Outliner once the editor is up.
	// SceneGraph-specific drop ops live in EntityEditor and are registered there separately when EntityFramework is enabled.
	EditorInitializedHandle = FEditorDelegates::OnEditorInitialized.AddLambda([this](double)
	{
		using namespace UE::Editor::DataStorage;
		if (AreEditorDataStorageFeaturesEnabled() && Storage)
		{
			Operations::RegisterDropOperations(*Storage);
		}
	});
}

void FTedsOutlinerModule::ShutdownModule()
{
	OnEditorDataStorageFeaturesEnabled().Remove(TedsInitializedHandle);
	FEditorDelegates::OnEditorInitialized.Remove(EditorInitializedHandle);
	UnregisterLevelEditorTedsOutlinerTab();
	IModuleInterface::ShutdownModule();
}

FTedsOutlinerColumnDescription FTedsOutlinerModule::GetLevelEditorTedsOutlinerColumnDescription()
{
	using namespace UE::Editor::DataStorage::Columns;
	using EColumnPriorityGroup = FTedsOutlinerColumnParams::EColumnPriorityGroup;
		
	TArray<TWeakObjectPtr<const UScriptStruct>> OutColumns({
		FVisibleInEditorColumn::StaticStruct(),
		FTedsPrimaryPackageObjectTag::StaticStruct(),
		FEditorDataStorageWorldPartitionPinnedColumn::StaticStruct(),
		FTypedElementClassTypeInfoColumn::StaticStruct(),
		FWorldPartitionDataLayerColumn::StaticStruct(),
		FWorldPartitionSubPackageColumn::StaticStruct(),
		FWorldPartitionContentBundleColumn::StaticStruct(),
		FLevelColumn::StaticStruct(), // PackageShortName
		FTedsActorMobilityColumn::StaticStruct(),
		FEditorDataStorageActorLayersColumn::StaticStruct(),
		FTedsActorSocketColumn::StaticStruct(),
		FUObjectIdNameColumn::StaticStruct(),
		FTedsActorUncachedLightsTag::StaticStruct(),
		FAlertColumn::StaticStruct(),
		FChildAlertColumn::StaticStruct()});

	if (CVarUseFavoritesWidgets->GetBool())
	{
		OutColumns.Add(FFavoriteTag::StaticStruct());
	}

	if (CVarUseNewRevisionControlWidgets->GetBool())
	{
		OutColumns.Add(FTypedElementPackageReference::StaticStruct());
		if (Storage)
		{
			Helpers::AddTedsColumnMappingToOutlinerColumn(Storage, FTypedElementPackageReference::StaticStruct(),
				FSceneOutlinerBuiltInColumnTypes::SourceControl());
		}
	}
	else if(Storage)
	{
		Helpers::RemoveTedsColumnMappingToOutlinerColumn(Storage, FTypedElementPackageReference::StaticStruct());
	}

	TMap<TWeakObjectPtr<const UScriptStruct>, FTedsOutlinerColumnParams> OutColumnParams
	{
		{FVisibleInEditorColumn::StaticStruct(), FTedsOutlinerColumnParams(EColumnPriorityGroup::Left)},
		{FTedsPrimaryPackageObjectTag::StaticStruct(), FTedsOutlinerColumnParams(EColumnPriorityGroup::Left)},
		{FWorldPartitionPinnedColumn::StaticStruct(), FTedsOutlinerColumnParams(ESceneOutlinerColumnVisibility::Invisible, EColumnPriorityGroup::Left)},
		{FWorldPartitionDataLayerColumn::StaticStruct(), FTedsOutlinerColumnParams(ESceneOutlinerColumnVisibility::Invisible)},
		{FWorldPartitionSubPackageColumn::StaticStruct(), FTedsOutlinerColumnParams(ESceneOutlinerColumnVisibility::Invisible)},
		{FWorldPartitionContentBundleColumn::StaticStruct(), FTedsOutlinerColumnParams(ESceneOutlinerColumnVisibility::Invisible)},
		{FLevelColumn::StaticStruct(), FTedsOutlinerColumnParams(ESceneOutlinerColumnVisibility::Invisible)},
		{FTedsActorMobilityColumn::StaticStruct(), FTedsOutlinerColumnParams(ESceneOutlinerColumnVisibility::Invisible)},
		{FEditorDataStorageActorLayersColumn::StaticStruct(), FTedsOutlinerColumnParams(ESceneOutlinerColumnVisibility::Invisible)},
		{FTedsActorSocketColumn::StaticStruct(), FTedsOutlinerColumnParams(ESceneOutlinerColumnVisibility::Invisible)},
		{FUObjectIdNameColumn::StaticStruct(), FTedsOutlinerColumnParams(ESceneOutlinerColumnVisibility::Invisible)},
		{FTedsActorUncachedLightsTag::StaticStruct(), FTedsOutlinerColumnParams(ESceneOutlinerColumnVisibility::Invisible)},
		{FTypedElementPackageReference::StaticStruct(), FTedsOutlinerColumnParams(EColumnPriorityGroup::Right)}
	};
	
	return FTedsOutlinerColumnDescription(OutColumns, OutColumnParams);
}

TSharedRef<SWidget> FTedsOutlinerModule::CreateLevelEditorTedsOutliner()
{
	if(!DataStorage::AreEditorDataStorageFeaturesEnabled())
	{
		return SNew(STextBlock)
		.Text(LOCTEXT("TEDSPluginNotEnabledText", "You need to enable the Typed Element Data Storage plugin to see the table viewer!"));
	}

	Storage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName);
	StorageUi = GetMutableDataStorageFeature<IUiProvider>(UiFeatureName);

	if (Storage && StorageUi)
	{
		const RowHandle WidgetPurposeRow = StorageUi->FindPurpose(
				IUiProvider::FPurposeInfo("LevelEditor", "SceneOutliner", "TedsOutliner").GeneratePurposeID());

		IUiProvider::FWidgetConstructorPtr OutSceneOutlinerConstructorPtr;

		auto AssignWidgetToColumn = [&OutSceneOutlinerConstructorPtr] (IUiProvider::FWidgetConstructorPtr WidgetConstructor, TConstArrayView<TWeakObjectPtr<const UScriptStruct>>)
			{
				OutSceneOutlinerConstructorPtr = MoveTemp(WidgetConstructor);
				return false;
			};

		StorageUi->CreateWidgetConstructors(WidgetPurposeRow, FMetaDataView(), AssignWidgetToColumn);
		if (OutSceneOutlinerConstructorPtr)
		{
			const FName NamePurposeTableName = TEXT("Editor_WidgetTable");
			
			if (RowHandle RowHandle = Storage->AddRow(Storage->FindTable(NamePurposeTableName));
				Storage->IsRowAvailable(RowHandle))
			{
				DataStorage::FMetaData OutlinerMetaData;
				OutlinerMetaData.AddImmutableData(TEXT("OutlinerIdentifier"), TedsOutlinerTabName.ToString());

				if (const TSharedPtr<SWidget> OutlinerWidget =
					StorageUi->ConstructWidget(RowHandle, *OutSceneOutlinerConstructorPtr, FGenericMetaDataView(OutlinerMetaData)))
				{
					return OutlinerWidget.ToSharedRef();
				}
			}
		}
	}
	
	return SNullWidget::NullWidget;
}

TSharedRef<SDockTab> FTedsOutlinerModule::OpenLevelEditorTedsOutliner(const FSpawnTabArgs& SpawnTabArgs)
{
	return SNew(SDockTab)
		.TabRole(ETabRole::NomadTab)
		[
			CreateLevelEditorTedsOutliner()
		];
}

// The TEDS-Outliner as a separate tab
void FTedsOutlinerModule::RegisterLevelEditorTedsOutlinerTab()
{
	FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>("LevelEditor");
		
	LevelEditorTabManagerChangedHandle = LevelEditorModule.OnTabManagerChanged().AddLambda([this]()
	{
		FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>("LevelEditor");

		TSharedPtr<FTabManager> LevelEditorTabManager = LevelEditorModule.GetLevelEditorTabManager();

		LevelEditorTabManager->RegisterTabSpawner(TedsOutlinerTabName, FOnSpawnTab::CreateRaw(this, &FTedsOutlinerModule::OpenLevelEditorTedsOutliner))
		.SetDisplayName(LOCTEXT("TedsOutlinerTabName", "Data-Driven Outliner (Experimental)"))
		.SetGroup(WorkspaceMenu::GetMenuStructure().GetLevelEditorOutlinerCategory())
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Outliner"))
		.SetAutoGenerateMenuEntry(false); // This can only be summoned from the Cvar now
	
	});
}

void FTedsOutlinerModule::UnregisterLevelEditorTedsOutlinerTab()
{
	FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>("LevelEditor");
	LevelEditorModule.OnTabManagerChanged().Remove(LevelEditorTabManagerChangedHandle);
	TSharedPtr<FTabManager> LevelEditorTabManager = LevelEditorModule.GetLevelEditorTabManager();
}

FName FTedsOutlinerModule::GetTedsOutlinerTabName()
{
	return TedsOutlinerTabName;
}

FTedsSceneOutlinerWorldFilter::FTedsSceneOutlinerWorldFilter(const FName& InFilterName, const TWeakObjectPtr<UWorld>& InSpecifiedWorldToDisplay)
		: FTedsOutlinerFilter(
          	InFilterName,
          	FText(), 
          	Queries::BuildConstQueryFunction<bool>(
				[this](TConstQueryContext<RowBatchInfo> Context, TResult<bool>& Result, TConstBatch<FTypedElementWorldColumn> WorldColumns)
				{
					QueryFunction(Context, Result, WorldColumns);
				}),
          	false)
		, SpecifiedWorldToDisplay(InSpecifiedWorldToDisplay)
{
	BuildWorldFilter();
	if (GEditor)
	{
		GEditor->OnLevelActorListChanged().AddRaw(this, &FTedsSceneOutlinerWorldFilter::BuildWorldFilter);
	}
}

FTedsSceneOutlinerWorldFilter::~FTedsSceneOutlinerWorldFilter()
{
	if (GEditor)
	{
		GEditor->OnLevelActorListChanged().RemoveAll(this);
	}
}

void FTedsSceneOutlinerWorldFilter::BuildWorldFilter()
{
	if (TedsOutlinerImpl.IsValid())
	{
		// Store a temporary World Ptr so we can check if the newly chosen world is the same as
		// the previous, if so, we don't have to regenerate anything since nothing changed
		TWeakObjectPtr<UWorld> TempWorld;
		SceneOutliner::FSceneOutlinerHelpers::ChooseRepresentingWorld(TempWorld, SpecifiedWorldToDisplay, UserChosenWorld);
		if (TempWorld.IsValid() && TempWorld != RepresentingWorld)
		{
			RepresentingWorld = TempWorld;

			if (const TSharedPtr<FTedsOutlinerImpl> TedsOutlinerImplPin = TedsOutlinerImpl.Pin())
			{
				if (ICoreProvider* Storage = TedsOutlinerImplPin->GetStorage())
				{
					Storage->AddColumn(TedsOutlinerImplPin->GetOutlinerRowHandle(), FTypedElementWorldColumn{ .World = RepresentingWorld });
				}

				// Drive World Partition column availability off the newly-represented world.
				const UWorld* World = RepresentingWorld.Get();
				const UWorldPartition* WorldPartition = World ? World->GetWorldPartition() : nullptr;
				const bool bIsPartitioned = WorldPartition != nullptr;

				TArray<TWeakObjectPtr<const UScriptStruct>> WorldPartitionColumns = {
					FWorldPartitionPinnedColumn::StaticStruct(),
					FWorldPartitionDataLayerColumn::StaticStruct(),
					FWorldPartitionSubPackageColumn::StaticStruct()
				};

				if (!bIsPartitioned || WorldPartition->IsContentBundleEnabled())
				{
					WorldPartitionColumns.Emplace(FWorldPartitionContentBundleColumn::StaticStruct());
				}

				const TArray<TWeakObjectPtr<const UScriptStruct>> LayersColumns = {
					Layers::FActorLayersColumn::StaticStruct()
				};

				if (bIsPartitioned)
				{
					TedsOutlinerImplPin->IncludeExcludeColumns(/*Included*/ WorldPartitionColumns, /*Excluded*/ LayersColumns);
				}
				else
				{
					TedsOutlinerImplPin->IncludeExcludeColumns(/*Included*/ LayersColumns, /*Excluded*/ WorldPartitionColumns);
				}

				// Perform a FullRefresh here since the Outliner does this when Filters change but we have to do it manually
				// since this filter isn't managed by the FilterBar.
				TedsOutlinerImplPin->FullRefresh();
			}
		}
	}
}

void FTedsSceneOutlinerWorldFilter::QueryFunction(TConstQueryContext<RowBatchInfo> Context, TResult<bool>& Result, TConstBatch<FTypedElementWorldColumn> WorldColumns)
{
	// We always want this to be valid, so if not, find a new world to filter for
	if (!RepresentingWorld.IsValid())
	{
		BuildWorldFilter();
	}
	
	Context.ForEachRow([this, &Result](RowHandle Row, const FTypedElementWorldColumn& WorldColumn)
		  {
			  if (WorldColumn.World.IsValid() && RepresentingWorld.IsValid())
			  {
				  Result.Add(Row, WorldColumn.World.Get() == RepresentingWorld.Get());
			  }
			  else
			  {
				  Result.Add(Row, false);
			  }
		  }, WorldColumns);
}

void FTedsSceneOutlinerWorldFilter::SetUserChosenWorld(const TWeakObjectPtr<UWorld>& InWorld)
{
	UserChosenWorld = InWorld;
	BuildWorldFilter();
}

TWeakObjectPtr<UWorld> FTedsSceneOutlinerWorldFilter::GetUserChosenWorld() const
{
	return UserChosenWorld;
}

	TWeakObjectPtr<UWorld> FTedsSceneOutlinerWorldFilter::GetRepresentingWorld() const
{
	return RepresentingWorld;
}
	
} // namespace UE::Editor::Outliner

IMPLEMENT_MODULE(UE::Editor::Outliner::FTedsOutlinerModule, TedsOutliner);

#undef LOCTEXT_NAMESPACE // TedsOutlinerModule
