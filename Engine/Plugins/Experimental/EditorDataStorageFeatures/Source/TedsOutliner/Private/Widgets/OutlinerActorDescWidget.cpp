// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/OutlinerActorDescWidget.h"

#include "Elements/Columns/TypedElementLabelColumns.h"
#include "Elements/Framework/TypedElementAttributeBinding.h"
#include "Widgets/Text/STextBlock.h"
#include "SceneOutlinerPublicTypes.h"
#include "TedsTableViewerUtils.h"
#include "Elements/Columns/TypedElementSlateWidgetColumns.h"
#include "Styling/StyleColors.h"
#include "Widgets/Images/SImage.h"
#include "ActorDesc/TedsActorDescColumns.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(OutlinerActorDescWidget)

#define LOCTEXT_NAMESPACE "OutlinerActorDescWidget"


void UActorDescWidgetFactory::RegisterWidgetConstructors(UE::Editor::DataStorage::ICoreProvider& DataStorage,
	UE::Editor::DataStorage::IUiProvider& DataStorageUi) const
{
	using namespace UE::Editor::DataStorage::Queries;
	
	DataStorageUi.RegisterWidgetFactory<FOutlinerActorDescIconWidgetConstructor>(
		DataStorageUi.FindPurpose(IUiProvider::FPurposeInfo("SceneOutliner", "RowLabel", "Icon").GeneratePurposeID()),
		TColumn<FTypedElementLabelColumn>() && TColumn<FActorDescTag>() && TColumn<FEditorDataStorageWorldPartitionHandleColumn>());

	DataStorageUi.RegisterWidgetFactory<FOutlinerActorDescTextWidgetConstructor>(
		DataStorageUi.FindPurpose(IUiProvider::FPurposeInfo("SceneOutliner", "RowLabel", "Text").GeneratePurposeID()),
		TColumn<FTypedElementLabelColumn>() && TColumn<FActorDescTag>() && TColumn<FEditorDataStorageWorldPartitionHandleColumn>());
}

FOutlinerActorDescIconWidgetConstructor::FOutlinerActorDescIconWidgetConstructor()
	: Super(StaticStruct())
{
}

TSharedPtr<SWidget> FOutlinerActorDescIconWidgetConstructor::CreateWidget(UE::Editor::DataStorage::ICoreProvider* DataStorage,
	UE::Editor::DataStorage::IUiProvider* DataStorageUi, UE::Editor::DataStorage::RowHandle TargetRow, UE::Editor::DataStorage::RowHandle WidgetRow,
	const UE::Editor::DataStorage::FMetaDataView& Arguments)
{
	using namespace UE::Editor::DataStorage;
	
	return SNew(SBox)
				.WidthOverride(static_cast<float>(FSceneOutlinerDefaultTreeItemMetrics::IconSize()))
				.HeightOverride(static_cast<float>(FSceneOutlinerDefaultTreeItemMetrics::IconSize()))
				[
					SNew(SImage)
					.Image(TableViewerUtils::GetIconForRow(DataStorage, TargetRow))
					.ColorAndOpacity(FSlateColor::UseSubduedForeground())
				];
}

FOutlinerActorDescTextWidgetConstructor::FOutlinerActorDescTextWidgetConstructor()
	: Super(StaticStruct())
{
}

TSharedPtr<SWidget> FOutlinerActorDescTextWidgetConstructor::CreateWidget(UE::Editor::DataStorage::ICoreProvider* DataStorage,
	UE::Editor::DataStorage::IUiProvider* DataStorageUi, UE::Editor::DataStorage::RowHandle TargetRow, UE::Editor::DataStorage::RowHandle WidgetRow,
	const UE::Editor::DataStorage::FMetaDataView& Arguments)
{
	using namespace UE::Editor::DataStorage;
	
	FAttributeBinder Binder(TargetRow, DataStorage);
	
	const RowHandle OutlinerWidgetRowHandle = TableViewerUtils::GetTableViewerUiRow(DataStorage, WidgetRow);
	FAttributeBinder OutlinerWidgetRowBinder(OutlinerWidgetRowHandle, DataStorage);

	return SNew(STextBlock)
				.Text(Binder.BindTextFormat(LOCTEXT("ActorDescLabel", 
							"{Name} (Unloaded)"))
							.Arg(TEXT("Name"), &FTypedElementLabelColumn::Label))
				.HighlightText(OutlinerWidgetRowBinder.BindData(&FHighlightTextColumn::HighlightText))
				.ColorAndOpacity(FSlateColor::UseSubduedForeground());
}

#undef LOCTEXT_NAMESPACE // "OutlinerActorDescWidget"
