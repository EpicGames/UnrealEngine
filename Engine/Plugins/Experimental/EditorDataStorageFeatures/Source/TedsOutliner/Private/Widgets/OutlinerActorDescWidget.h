// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Columns/TedsOutlinerColumns.h"
#include "Elements/Interfaces/TypedElementDataStorageFactory.h"
#include "Elements/Interfaces/TypedElementDataStorageUiInterface.h"
#include "Internationalization/Text.h"
#include "UObject/ObjectMacros.h"
#include "Styling/SlateColor.h"

#include "OutlinerActorDescWidget.generated.h"

class UScriptStruct;
struct ISceneOutlinerTreeItem;

UCLASS()
class UActorDescWidgetFactory : public UEditorDataStorageFactory
{
	GENERATED_BODY()

public:
	~UActorDescWidgetFactory() override = default;

	TEDSOUTLINER_API void RegisterWidgetConstructors(UE::Editor::DataStorage::ICoreProvider& DataStorage,
		UE::Editor::DataStorage::IUiProvider& DataStorageUi) const override;
};

// Label widget for folders in the Outliner that shows an icon 
USTRUCT()
struct FOutlinerActorDescIconWidgetConstructor : public FSimpleWidgetConstructor
{
	GENERATED_BODY()

public:
	TEDSOUTLINER_API FOutlinerActorDescIconWidgetConstructor(); 
	~FOutlinerActorDescIconWidgetConstructor() override = default;

	TEDSOUTLINER_API virtual TSharedPtr<SWidget> CreateWidget(
		UE::Editor::DataStorage::ICoreProvider* DataStorage,
		UE::Editor::DataStorage::IUiProvider* DataStorageUi,
		UE::Editor::DataStorage::RowHandle TargetRow,
		UE::Editor::DataStorage::RowHandle WidgetRow, 
		const UE::Editor::DataStorage::FMetaDataView& Arguments) override;
};

// Label widget for folders in the Outliner that shows the text label 
USTRUCT()
struct FOutlinerActorDescTextWidgetConstructor : public FSimpleWidgetConstructor
{
	GENERATED_BODY()

public:
	TEDSOUTLINER_API FOutlinerActorDescTextWidgetConstructor(); 
	~FOutlinerActorDescTextWidgetConstructor() override = default;

	TEDSOUTLINER_API virtual TSharedPtr<SWidget> CreateWidget(
		UE::Editor::DataStorage::ICoreProvider* DataStorage,
		UE::Editor::DataStorage::IUiProvider* DataStorageUi,
		UE::Editor::DataStorage::RowHandle TargetRow,
		UE::Editor::DataStorage::RowHandle WidgetRow, 
		const UE::Editor::DataStorage::FMetaDataView& Arguments) override;

};