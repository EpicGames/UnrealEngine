// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/OutlinerActorLayersWidget.h"

#include "Elements/Framework/TypedElementAttributeBinding.h"
#include "Layers/Columns/LayersColumns.h"
#include "TedsOutlinerHelpers.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(OutlinerActorLayersWidget)

#define LOCTEXT_NAMESPACE "OutlinerActorLayersWidget"

void UOutlinerActorLayersWidgetFactory::RegisterWidgetConstructors(UE::Editor::DataStorage::ICoreProvider& DataStorage,
	UE::Editor::DataStorage::IUiProvider& DataStorageUi) const
{
	using namespace UE::Editor::DataStorage::Queries;

	DataStorageUi.RegisterWidgetFactory<FOutlinerActorLayersWidgetConstructor>(
		DataStorageUi.FindPurpose(IUiProvider::FPurposeInfo("SceneOutliner", "Cell", NAME_None).GeneratePurposeID()),
		TColumn<UE::Editor::Layers::FActorLayersColumn>());
}

FOutlinerActorLayersWidgetConstructor::FOutlinerActorLayersWidgetConstructor()
	: FSimpleWidgetConstructor(StaticStruct())
{
}

FText FOutlinerActorLayersWidgetConstructor::CreateWidgetDisplayNameText(
	UE::Editor::DataStorage::ICoreProvider* DataStorage,
	UE::Editor::DataStorage::RowHandle Row) const
{
	return LOCTEXT("OutlinerActorLayersWidgetDisplayName", "Layers");
}

TSharedPtr<SWidget> FOutlinerActorLayersWidgetConstructor::CreateWidget(
	UE::Editor::DataStorage::ICoreProvider* DataStorage,
	UE::Editor::DataStorage::IUiProvider* DataStorageUi,
	UE::Editor::DataStorage::RowHandle TargetRow,
	UE::Editor::DataStorage::RowHandle WidgetRow,
	const UE::Editor::DataStorage::FMetaDataView& Arguments)
{
	using namespace UE::Editor::DataStorage;
	using namespace UE::Editor::Layers;

	if (!DataStorage->IsRowAvailable(TargetRow))
	{
		return SNew(STextBlock)
			.Text(LOCTEXT("MissingRowReferenceColumn", "Unable to retrieve row reference."));
	}

	FAttributeBinder Binder(TargetRow, DataStorage);

	return SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		.Padding(8, 0, 0, 0)
		[
			SNew(STextBlock)
				.ColorAndOpacity(FSlateColor::UseSubduedForeground())
				.HighlightText(UE::Editor::Outliner::Helpers::GetHighlightTextAttribute(DataStorage, WidgetRow))
				.Text(Binder.BindData<FText, FRowHandleArray>(&FActorLayersColumn::Layers, [DataStorage](const FRowHandleArray& Layers) -> FText
				{
					FString Result;

					for (RowHandle LayerRow : Layers.GetRows())
					{
						if (FLayerNameColumn* LayerNameColumn = DataStorage->GetColumn<FLayerNameColumn>(LayerRow))
						{
							if (Result.Len())
							{
								Result += TEXT(", ");
							}

							Result += LayerNameColumn->LayerName.ToString();
						}
					}
					return FText::FromString(Result);
				}))
		];
}

TArray<TSharedPtr<const UE::Editor::DataStorage::FColumnSearcherInterface>> FOutlinerActorLayersWidgetConstructor::ConstructColumnSearchers(
	UE::Editor::DataStorage::ICoreProvider* DataStorage,
	UE::Editor::DataStorage::IUiProvider* DataStorageUi,
	const UE::Editor::DataStorage::FMetaDataView& Arguments)
{
	using namespace UE::Editor::DataStorage;
	return TArray<TSharedPtr<const FColumnSearcherInterface>>(
		{
			MakeShared<FOutlinerActorLayersWidgetSearcher>()
		});
}

void UE::Editor::DataStorage::FOutlinerActorLayersWidgetSearcher::GetSearchableString(
	FString& SearchableString, const ICoreProvider& Storage, const Layers::FActorLayersColumn& Column) const
{
	for (const RowHandle LayerRow : Column.Layers.GetRows())
	{
		if (const Layers::FLayerNameColumn* LayerNameColumn = Storage.GetColumn<Layers::FLayerNameColumn>(LayerRow))
		{
			if (SearchableString.Len())
			{
				SearchableString += TEXT(", ");
			}

			SearchableString += LayerNameColumn->LayerName.ToString();
		}
	}
}

#undef LOCTEXT_NAMESPACE
