// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/OutlinerCDOTextWidget.h"

#include "ActorEditorUtils.h"
#include "Columns/SlateDelegateColumns.h"
#include "Elements/Columns/TypedElementCompatibilityColumns.h"
#include "Elements/Columns/TypedElementLabelColumns.h"
#include "Elements/Framework/TypedElementAttributeBinding.h"
#include "TedsOutlinerHelpers.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"
#include "Widgets/Text/STextBlock.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(OutlinerCDOTextWidget)

#define LOCTEXT_NAMESPACE "OutlinerCDOTextWidget"

void UOutlinerCDOTextWidgetFactory::RegisterWidgetConstructors(UE::Editor::DataStorage::ICoreProvider& DataStorage,
	UE::Editor::DataStorage::IUiProvider& DataStorageUi) const
{
	using namespace UE::Editor::DataStorage::Queries;

	DataStorageUi.RegisterWidgetFactory<FOutlinerCDOTextWidgetConstructor>(DataStorageUi.FindPurpose(
		IUiProvider::FPurposeInfo("SceneOutliner", "RowLabel", "Text").GeneratePurposeID()),
		TColumn<FTypedElementLabelColumn>() && TColumn<FTypedElementClassDefaultObjectTag>());
}

FOutlinerCDOTextWidgetConstructor::FOutlinerCDOTextWidgetConstructor()
	: Super(FOutlinerCDOTextWidgetConstructor::StaticStruct())
{
}

FOutlinerCDOTextWidgetConstructor::FOutlinerCDOTextWidgetConstructor(const UScriptStruct* InTypeInfo)
	: Super(InTypeInfo)
{
}

TSharedPtr<SWidget> FOutlinerCDOTextWidgetConstructor::CreateWidget(UE::Editor::DataStorage::ICoreProvider* DataStorage,
	UE::Editor::DataStorage::IUiProvider* DataStorageUi,
	UE::Editor::DataStorage::RowHandle TargetRow,
	UE::Editor::DataStorage::RowHandle WidgetRow, const UE::Editor::DataStorage::FMetaDataView& Arguments)
{
	using namespace UE::Editor::DataStorage;
	if (!DataStorage->IsRowAvailable(TargetRow))
	{
		return SNew(STextBlock)
			.Text(LOCTEXT("MissingRowReferenceColumn", "Unable to retrieve row reference."));
	}

	using namespace UE::Editor::DataStorage;
	FAttributeBinder TargetRowBinder(TargetRow, DataStorage);

	TSharedPtr<STextBlock> TextBlock = SNew(STextBlock)
		.IsEnabled(false)
		.Text(TargetRowBinder.BindData((&FTypedElementLabelColumn::Label), [](const FString& InString)
		{
			FString Name = InString;
			Name.RemoveFromStart(TEXT("Default__"));
			Name.RemoveFromEnd(TEXT("_C"));
			return FText::FromString(Name);
		}, FString()))
		.ToolTipText(TargetRowBinder.BindText(&FTypedElementLabelColumn::Label))
		.HighlightText(UE::Editor::Outliner::Helpers::GetHighlightTextAttribute(DataStorage, WidgetRow));

	return TextBlock;
}

#undef LOCTEXT_NAMESPACE
