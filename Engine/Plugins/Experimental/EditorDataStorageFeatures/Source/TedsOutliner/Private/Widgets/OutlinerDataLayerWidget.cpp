// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/OutlinerDataLayerWidget.h"

#include "SceneOutlinerHelpers.h"
#include "Columns/TedsActorWorldPartitionColumns.h"
#include "Elements/Framework/TypedElementAttributeBinding.h"
#include "TedsOutlinerHelpers.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(OutlinerDataLayerWidget)

#define LOCTEXT_NAMESPACE "OutlinerDataLayerWidget"

void UOutlinerDataLayerWidgetFactory::RegisterWidgetConstructors(UE::Editor::DataStorage::ICoreProvider& DataStorage,
	UE::Editor::DataStorage::IUiProvider& DataStorageUi) const
{
	using namespace UE::Editor::DataStorage::Queries;

	DataStorageUi.RegisterWidgetFactory<FOutlinerDataLayerWidgetConstructor>(
		DataStorageUi.FindPurpose(IUiProvider::FPurposeInfo("SceneOutliner", "Cell", NAME_None).GeneratePurposeID()),
		TColumn<FWorldPartitionDataLayerColumn>());
}

FOutlinerDataLayerWidgetConstructor::FOutlinerDataLayerWidgetConstructor()
	: FSimpleWidgetConstructor(StaticStruct())
{
}

FText FOutlinerDataLayerWidgetConstructor::CreateWidgetDisplayNameText(
	UE::Editor::DataStorage::ICoreProvider* DataStorage, UE::Editor::DataStorage::RowHandle Row) const
{
	return LOCTEXT("SceneOutlinerDataLayerColumn", "Data Layer");
}

TSharedPtr<SWidget> FOutlinerDataLayerWidgetConstructor::CreateWidget(
	UE::Editor::DataStorage::ICoreProvider* DataStorage,
	UE::Editor::DataStorage::IUiProvider* DataStorageUi,
	UE::Editor::DataStorage::RowHandle TargetRow,
	UE::Editor::DataStorage::RowHandle WidgetRow,
	const UE::Editor::DataStorage::FMetaDataView& Arguments)
{
	using namespace UE::Editor::DataStorage;

	if (!DataStorage->IsRowAvailable(TargetRow))
	{
		return SNew(STextBlock)
			.Text(LOCTEXT("MissingRowReferenceColumn", "Unable to retrieve row reference."));
	}

	FAttributeBinder Binder(TargetRow, DataStorage);

	return SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		.Padding(8, 0, 0, 0)
		[
			SNew(STextBlock)
				.ColorAndOpacity(FSlateColor::UseSubduedForeground())
				.Text(Binder.BindText(&FWorldPartitionDataLayerColumn::DataLayers))
				.HighlightText(UE::Editor::Outliner::Helpers::GetHighlightTextAttribute(DataStorage, WidgetRow))
		];
}

#undef LOCTEXT_NAMESPACE
