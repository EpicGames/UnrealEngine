// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/OutlinerFolderLabelWidget.h"

#include "ActorFolders/ActorFolderColumns.h"
#include "Columns/SceneOutlinerColumns.h"
#include "Columns/SlateDelegateColumns.h"
#include "Compatibility/SceneOutlinerTedsBridge.h"
#include "Elements/Columns/TypedElementCompatibilityColumns.h"
#include "Elements/Columns/TypedElementLabelColumns.h"
#include "Elements/Columns/TypedElementFolderColumns.h"
#include "Elements/Columns/TypedElementMiscColumns.h"
#include "Elements/Columns/TypedElementSlateWidgetColumns.h"
#include "Elements/Framework/TypedElementAttributeBinding.h"
#include "Elements/Columns/TypedElementUIColumns.h"
#include "ISceneOutliner.h"
#include "SceneOutlinerHelpers.h"
#include "TedsOutlinerHelpers.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"
#include "Widgets/Text/STextBlock.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(OutlinerFolderLabelWidget)

#define LOCTEXT_NAMESPACE "OutlinerFolderLabelWidget"

namespace UE::Editor::Outliner::FolderLabel::Private
{
	static TWeakPtr<ISceneOutlinerTreeItem> GetTreeItemForRow(UE::Editor::DataStorage::ICoreProvider* DataStorage,
		UE::Editor::DataStorage::RowHandle TargetRow, UE::Editor::DataStorage::RowHandle WidgetRow)
	{
		if (FSceneOutlinerColumn* TedsOutlinerColumn = DataStorage->GetColumn<FSceneOutlinerColumn>(WidgetRow))
		{
			if (TSharedPtr<ISceneOutliner> Outliner = TedsOutlinerColumn->Outliner.Pin())
			{
				return UE::Editor::Outliner::Helpers::GetTreeItemFromRowHandle(DataStorage, Outliner.ToSharedRef(), TargetRow);
			}
		}

		return nullptr;
	}

	static FText GetDisplayText(UE::Editor::DataStorage::ICoreProvider* DataStorage, UE::Editor::DataStorage::RowHandle TargetRow,
		UE::Editor::DataStorage::RowHandle WidgetRow)
	{
		if (const FTypedElementLabelColumn* LabelColumn = DataStorage->GetColumn<FTypedElementLabelColumn>(TargetRow))
		{
			FText Label = FText::FromString(LabelColumn->Label);

			const FFolderCompatibilityColumn* FolderCompatibilityColumn = DataStorage->GetColumn<FFolderCompatibilityColumn>(TargetRow);
			const FTypedElementWorldColumn* WorldColumn = DataStorage->GetColumn<FTypedElementWorldColumn>(TargetRow);
			const bool IsInEditingMode = DataStorage->HasColumns<FIsInEditingModeTag>(WidgetRow);

			if (FolderCompatibilityColumn && WorldColumn)
			{
				if (!IsInEditingMode)
				{
					FText IsCurrentSuffixText =
						SceneOutliner::FSceneOutlinerHelpers::IsFolderCurrent(FolderCompatibilityColumn->Folder, WorldColumn->World.Get())
							? FText(LOCTEXT("IsCurrentSuffix", " (Current)"))
							: FText::GetEmpty();

					return FText::Format(LOCTEXT("LevelInstanceDisplay", "{0}{1}"), Label, IsCurrentSuffixText);
				}
			}

			return Label;
		}

		return FText();
	}

	static FText GetTooltipText(UE::Editor::DataStorage::ICoreProvider* DataStorage, UE::Editor::DataStorage::RowHandle TargetRow)
	{
		const FFolderCompatibilityColumn* FolderCompatibilityColumn = DataStorage->GetColumn<FFolderCompatibilityColumn>(TargetRow);
		const FTypedElementWorldColumn* WorldColumn = DataStorage->GetColumn<FTypedElementWorldColumn>(TargetRow);
		const FTypedElementLabelColumn* LabelColumn = DataStorage->GetColumn<FTypedElementLabelColumn>(TargetRow);

		if (LabelColumn && WorldColumn && FolderCompatibilityColumn)
		{
			FText Description = SceneOutliner::FSceneOutlinerHelpers::IsFolderCurrent(FolderCompatibilityColumn->Folder, WorldColumn->World.Get())
					? LOCTEXT("ActorFolderIsCurrentDescription", "\nThis is your current folder. New actors you create will appear here.")
					: FText::GetEmpty();

			return FText::Format(LOCTEXT("DataLayerTooltipText", "{0}{1}"), FText::FromString(LabelColumn->Label), Description);
		}
		return FText();
	}

	static FSlateColor GetForegroundColor(UE::Editor::DataStorage::ICoreProvider* DataStorage,
		UE::Editor::DataStorage::RowHandle TargetRow, UE::Editor::DataStorage::RowHandle WidgetRow)
	{
		// CommonLabelData has some Outliner specific color logic
		TWeakPtr<ISceneOutlinerTreeItem> OutlinerTreeItem = GetTreeItemForRow(DataStorage, TargetRow, WidgetRow);

		if (TSharedPtr<ISceneOutlinerTreeItem> Item = OutlinerTreeItem.Pin())
		{
			FSceneOutlinerCommonLabelData CommonLabelData;
			CommonLabelData.WeakSceneOutliner = Item->WeakSceneOutliner;

			if (TOptional<FLinearColor> BaseColor = CommonLabelData.GetForegroundColor(*Item))
			{
				return BaseColor.GetValue();
			}

			const FFolderCompatibilityColumn* FolderCompatibilityColumn = DataStorage->GetColumn<FFolderCompatibilityColumn>(TargetRow);
			const FTypedElementWorldColumn* WorldColumn = DataStorage->GetColumn<FTypedElementWorldColumn>(TargetRow);

			if (FolderCompatibilityColumn && WorldColumn)
			{
				if (SceneOutliner::FSceneOutlinerHelpers::IsFolderCurrent(FolderCompatibilityColumn->Folder, WorldColumn->World.Get()))
				{
					return FAppStyle::Get().GetSlateColor("Colors.AccentGreen");
				}
			}
		}

		return FSlateColor::UseForeground();
	}
}

void UOutlinerFolderLabelWidgetFactory::RegisterWidgetConstructors(UE::Editor::DataStorage::ICoreProvider& DataStorage,
	UE::Editor::DataStorage::IUiProvider& DataStorageUi) const
{
	using namespace UE::Editor::DataStorage::Queries;
	using IUiProvider = UE::Editor::DataStorage::IUiProvider;

	DataStorageUi.RegisterWidgetFactory<FOutlinerFolderIconWidgetConstructor>(
		DataStorageUi.FindPurpose(IUiProvider::FPurposeInfo("SceneOutliner", "RowLabel", "Icon").GeneratePurposeID()),
		TColumn<FFolderTag>() && TColumn<FFolderCompatibilityColumn>());

	DataStorageUi.RegisterWidgetFactory<FOutlinerFolderTextWidgetConstructor>(
		DataStorageUi.FindPurpose(IUiProvider::FPurposeInfo("SceneOutliner", "RowLabel", "Text").GeneratePurposeID()),
		TColumn<FTypedElementLabelColumn>() && TColumn<FFolderTag>() && TColumn<FFolderCompatibilityColumn>());
}

// 
// FOutlinerFolderIconWidgetConstructor
// 

FOutlinerFolderIconWidgetConstructor::FOutlinerFolderIconWidgetConstructor()
	: FSimpleWidgetConstructor(StaticStruct())
{
}

TSharedPtr<SWidget> FOutlinerFolderIconWidgetConstructor::CreateWidget(UE::Editor::DataStorage::ICoreProvider* DataStorage,
	UE::Editor::DataStorage::IUiProvider* DataStorageUi, UE::Editor::DataStorage::RowHandle TargetRow,
	UE::Editor::DataStorage::RowHandle WidgetRow, const UE::Editor::DataStorage::FMetaDataView& Arguments)
{
	using namespace UE::Editor::DataStorage;

	if (!DataStorage->IsRowAvailable(TargetRow))
	{
		return SNullWidget::NullWidget;
	}

	return SNew(SBox)
		.WidthOverride(16.0f)
		.HeightOverride(16.0f)
		[
			SNew(SImage)
				.Image_Lambda([DataStorage, TargetRow, WidgetRow]()
				{
					bool bIsExpanded = DataStorage->HasColumns<FExpandedInUITag>(TargetRow);

					TWeakPtr<ISceneOutlinerTreeItem> OutlinerTreeItem =
						UE::Editor::Outliner::FolderLabel::Private::GetTreeItemForRow(DataStorage, TargetRow, WidgetRow);

					// If this item does not have any children, we want to treat it as not expanded and show the closed folder icon
					if (TSharedPtr<ISceneOutlinerTreeItem> Item = OutlinerTreeItem.Pin())
					{
						bIsExpanded &= !Item->GetChildren().IsEmpty();
					}

					return bIsExpanded
						? FAppStyle::GetBrush("SceneOutliner.FolderOpen")
						: FAppStyle::GetBrush("SceneOutliner.FolderClosed");
				})
		];
}

// 
// FOutlinerFolderTextWidgetConstructor
//

FOutlinerFolderTextWidgetConstructor::FOutlinerFolderTextWidgetConstructor()
	: FSimpleWidgetConstructor(StaticStruct())
{
}

TSharedPtr<SWidget> FOutlinerFolderTextWidgetConstructor::CreateWidget(UE::Editor::DataStorage::ICoreProvider* DataStorage,
	UE::Editor::DataStorage::IUiProvider* DataStorageUi, UE::Editor::DataStorage::RowHandle TargetRow,
	UE::Editor::DataStorage::RowHandle WidgetRow, const UE::Editor::DataStorage::FMetaDataView& Arguments)
{
	using namespace UE::Editor::DataStorage;

	if (!DataStorage->IsRowAvailable(TargetRow))
	{
		return SNew(STextBlock)
			.Text(LOCTEXT("MissingRowReferenceColumn", "Unable to retrieve row reference."));
	}

	bool bEditable = false;

	FMetaDataEntryView MetaDataEntryView = Arguments.FindForColumn<FTypedElementLabelColumn>(UE::Editor::DataStorage::IsEditableName);

	if (MetaDataEntryView.IsSet())
	{
		if (const bool* EditableMetaDataPtr = MetaDataEntryView.TryGetExact<bool>())
		{
			bEditable = *EditableMetaDataPtr;
		}
	}

	if (bEditable)
	{
		FAttributeBinder WidgetRowBinder(WidgetRow, DataStorage);

		TSharedPtr<SInlineEditableTextBlock> TextBlock = SNew(SInlineEditableTextBlock)
			.HighlightText(UE::Editor::Outliner::Helpers::GetHighlightTextAttribute(DataStorage, WidgetRow))
			.OnTextCommitted_Lambda(
				[DataStorage, TargetRow](const FText& NewText, ETextCommit::Type CommitInfo)
				{
					// This callback happens on the game thread so it's safe to directly call into the data storage.
					FString NewLabelText = NewText.ToString();
					if (FTypedElementLabelHashColumn* LabelHashColumn = DataStorage->GetColumn<FTypedElementLabelHashColumn>(TargetRow))
					{
						LabelHashColumn->LabelHash = CityHash64(reinterpret_cast<const char*>(*NewLabelText), NewLabelText.Len() * sizeof(**NewLabelText));
					}
					if (FTypedElementLabelColumn* LabelColumn = DataStorage->GetColumn<FTypedElementLabelColumn>(TargetRow))
					{
						LabelColumn->Label = MoveTemp(NewLabelText);
					}
					DataStorage->AddColumn<FTypedElementSyncBackToWorldTag>(TargetRow);
				})
			.OnVerifyTextChanged_Lambda([DataStorage, TargetRow](const FText& Label, FText& ErrorMessage)
				{
					const FFolderCompatibilityColumn* FolderCompatibilityColumn = DataStorage->GetColumn<FFolderCompatibilityColumn>(TargetRow);
					const FTypedElementWorldColumn* WorldColumn = DataStorage->GetColumn<FTypedElementWorldColumn>(TargetRow);

					if (FolderCompatibilityColumn && WorldColumn)
					{
						return SceneOutliner::FSceneOutlinerHelpers::ValidateFolderName(FolderCompatibilityColumn->Folder, WorldColumn->World.Get(),
							Label, ErrorMessage);
					}

					ErrorMessage = LOCTEXT("MissingColumns", "Could not find folder information to rename.");

					return false;
				})
			.Text_Static(&UE::Editor::Outliner::FolderLabel::Private::GetDisplayText, DataStorage, TargetRow, WidgetRow)
			.ToolTipText_Static(&UE::Editor::Outliner::FolderLabel::Private::GetTooltipText, DataStorage, TargetRow)
			.ColorAndOpacity_Static(&UE::Editor::Outliner::FolderLabel::Private::GetForegroundColor, DataStorage, TargetRow, WidgetRow)
			.IsSelected(WidgetRowBinder.BindEvent(&FExternalWidgetSelectionColumn::IsSelected))
			.OnEnterEditingMode_Lambda([DataStorage, WidgetRow]()
			{
				DataStorage->AddColumn<FIsInEditingModeTag>(WidgetRow);
			})
			.OnExitEditingMode_Lambda([DataStorage, WidgetRow]()
			{
				DataStorage->RemoveColumn<FIsInEditingModeTag>(WidgetRow);
			})
			.DoubleSelectDelay(0.f);

		DataStorage->AddColumn<FWidgetEnterEditModeColumn>(WidgetRow, FWidgetEnterEditModeColumn{
				.OnEnterEditMode = MakeShared<FSimpleDelegate>(FSimpleDelegate::CreateLambda([TextBlock]()
				{
					TextBlock->EnterEditingMode();
				}))
			});

		return TextBlock;
	}

	FAttributeBinder TargetRowBinder(TargetRow, DataStorage);

	return SNew(STextBlock)
		.IsEnabled(false)
		.Text(TargetRowBinder.BindText(&FTypedElementLabelColumn::Label))
		.ToolTipText(TargetRowBinder.BindText(&FTypedElementLabelColumn::Label))
		.HighlightText(UE::Editor::Outliner::Helpers::GetHighlightTextAttribute(DataStorage, WidgetRow));
}

#undef LOCTEXT_NAMESPACE
