// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Columns/TedsOutlinerColumns.h"
#include "Elements/Interfaces/TypedElementDataStorageFactory.h"
#include "Elements/Interfaces/TypedElementDataStorageUiInterface.h"
#include "Internationalization/Text.h"
#include "UObject/ObjectMacros.h"
#include "Styling/SlateColor.h"

#include "OutlinerFolderLabelWidget.generated.h"

class UScriptStruct;
struct ISceneOutlinerTreeItem;

UCLASS()
class UOutlinerFolderLabelWidgetFactory : public UEditorDataStorageFactory
{
	GENERATED_BODY()

public:
	~UOutlinerFolderLabelWidgetFactory() override = default;

	TEDSOUTLINER_API void RegisterWidgetConstructors(UE::Editor::DataStorage::ICoreProvider& DataStorage,
		UE::Editor::DataStorage::IUiProvider& DataStorageUi) const override;
};

// Icon component widget for folder rows.
USTRUCT()
struct FOutlinerFolderIconWidgetConstructor : public FSimpleWidgetConstructor
{
	GENERATED_BODY()

public:
	TEDSOUTLINER_API FOutlinerFolderIconWidgetConstructor();
	~FOutlinerFolderIconWidgetConstructor() override = default;

	TEDSOUTLINER_API virtual TSharedPtr<SWidget> CreateWidget(
		UE::Editor::DataStorage::ICoreProvider* DataStorage,
		UE::Editor::DataStorage::IUiProvider* DataStorageUi,
		UE::Editor::DataStorage::RowHandle TargetRow,
		UE::Editor::DataStorage::RowHandle WidgetRow,
		const UE::Editor::DataStorage::FMetaDataView& Arguments) override;
};

// Text component widget for folder rows. Registered to the "SceneOutliner/RowLabel/Text" sub-purpose.
USTRUCT()
struct FOutlinerFolderTextWidgetConstructor : public FSimpleWidgetConstructor
{
	GENERATED_BODY()

public:
	TEDSOUTLINER_API FOutlinerFolderTextWidgetConstructor();
	~FOutlinerFolderTextWidgetConstructor() override = default;

	TEDSOUTLINER_API virtual TSharedPtr<SWidget> CreateWidget(
		UE::Editor::DataStorage::ICoreProvider* DataStorage,
		UE::Editor::DataStorage::IUiProvider* DataStorageUi,
		UE::Editor::DataStorage::RowHandle TargetRow,
		UE::Editor::DataStorage::RowHandle WidgetRow,
		const UE::Editor::DataStorage::FMetaDataView& Arguments) override;
};
