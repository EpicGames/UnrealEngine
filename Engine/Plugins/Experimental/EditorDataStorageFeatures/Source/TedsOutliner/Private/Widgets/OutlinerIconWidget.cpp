// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/OutlinerIconWidget.h"

#include "Elements/Columns/TypedElementOverrideColumns.h"
#include "Elements/Columns/TypedElementTypeInfoColumns.h"
#include "Elements/Framework/TypedElementAttributeBinding.h"
#include "TedsOutlinerHelpers.h"
#include "TedsTableViewerUtils.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Images/SLayeredImage.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(OutlinerIconWidget)

#define LOCTEXT_NAMESPACE "OutlinerIconWidget"

void UOutlinerIconWidgetFactory::RegisterWidgetConstructors(UE::Editor::DataStorage::ICoreProvider& DataStorage,
	UE::Editor::DataStorage::IUiProvider& DataStorageUi) const
{
	using namespace UE::Editor::DataStorage::Queries;

	DataStorageUi.RegisterWidgetFactory<FOutlinerIconWidgetConstructor>(
		DataStorageUi.FindPurpose(IUiProvider::FPurposeInfo("SceneOutliner", "RowLabel", "Icon").GeneratePurposeID()),
		TColumn<FTypedElementClassTypeInfoColumn>() || TColumn<FTypedElementScriptStructTypeInfoColumn>());
}

const FSlateBrush* FOutlinerIconWidgetLogic::GetOverrideBadgeFirstLayer(const UE::Editor::DataStorage::ICoreProvider& Storage,
	UE::Editor::DataStorage::RowHandle Row) const
{
	switch (GetOverriddenState(Storage, Row))
	{
	case EOverriddenState::Added:
		return FAppStyle::GetBrush("SceneOutliner.OverrideAddedBase");

	case EOverriddenState::AllOverridden:
		return FAppStyle::GetBrush("SceneOutliner.OverrideBase");

	case EOverriddenState::HasOverrides:
	case EOverriddenState::SubObjectsHasOverrides:
		return FAppStyle::GetBrush("SceneOutliner.OverrideInsideBase");

	case EOverriddenState::NoOverrides:
		// No icon for no overrides
		break;
	}

	return FAppStyle::GetBrush("NoBrush");
}

const FSlateBrush* FOutlinerIconWidgetLogic::GetOverrideBadgeSecondLayer(const UE::Editor::DataStorage::ICoreProvider& Storage,
	UE::Editor::DataStorage::RowHandle Row) const
{
	switch (GetOverriddenState(Storage, Row))
	{
	case EOverriddenState::Added:
		return FAppStyle::GetBrush("SceneOutliner.OverrideAdded");

	case EOverriddenState::AllOverridden:
		return FAppStyle::GetBrush("SceneOutliner.OverrideHere");

	case EOverriddenState::HasOverrides:
	case EOverriddenState::SubObjectsHasOverrides:
		return FAppStyle::GetBrush("SceneOutliner.OverrideInside");

	case EOverriddenState::NoOverrides:
		// No icon for no overrides
		break;
	}

	return FAppStyle::GetBrush("NoBrush");
}

FText FOutlinerIconWidgetLogic::GetOverrideTooltip(const UE::Editor::DataStorage::ICoreProvider& Storage,
	UE::Editor::DataStorage::RowHandle Row) const
{
	switch (GetOverriddenState(Storage, Row))
	{
	case EOverriddenState::Added:
		return LOCTEXT("OverrideAddedTooltip", "This entity has been added.");

	case EOverriddenState::AllOverridden:
		return LOCTEXT("OverrideAllTooltip", "This entity has all its properties overridden.");

	case EOverriddenState::HasOverrides:
	case EOverriddenState::SubObjectsHasOverrides:
		return LOCTEXT("OverrideInsideTooltip", "At least one property or child has an override.");

	case EOverriddenState::NoOverrides:
		// No icon for no overrides
		break;
	}

	return FText::GetEmpty();
}

EOverriddenState FOutlinerIconWidgetLogic::GetOverriddenState(const UE::Editor::DataStorage::ICoreProvider& Storage,
	UE::Editor::DataStorage::RowHandle Row) const
{
	const FObjectOverrideColumn* Column = Storage.GetColumn<FObjectOverrideColumn>(Row);
	return Column ? Column->OverriddenState : EOverriddenState::NoOverrides;
}

FOutlinerIconWidgetConstructor::FOutlinerIconWidgetConstructor(const UScriptStruct* InTypeInfo)
	: Super(InTypeInfo)
{
}

TSharedPtr<SWidget> FOutlinerIconWidgetConstructor::CreateWidget(UE::Editor::DataStorage::ICoreProvider* DataStorage,
	UE::Editor::DataStorage::IUiProvider* DataStorageUi, UE::Editor::DataStorage::RowHandle TargetRow,
	UE::Editor::DataStorage::RowHandle WidgetRow, const UE::Editor::DataStorage::FMetaDataView& Arguments)
{
	using namespace UE::Editor::DataStorage;
	if (!DataStorage->IsRowAvailable(TargetRow))
	{
		return SNullWidget::NullWidget;
	}

	const FOutlinerIconWidgetLogic& Logic = GetLogic();
	
	TSharedRef<SLayeredImage> LayeredImageWidget = SNew(SLayeredImage)
		.Image(TableViewerUtils::GetIconForRow(DataStorage, TargetRow))
		.ToolTipText_Lambda([&Logic, DataStorage, TargetRow]()
		{
			return Logic.GetOverrideTooltip(*DataStorage, TargetRow);
		})
		.ColorAndOpacity(UE::Editor::Outliner::Helpers::GetLabelWidgetForegroundColor(DataStorage, TargetRow, WidgetRow));

	LayeredImageWidget->AddLayer(TAttribute<const FSlateBrush*>::CreateLambda([&Logic, DataStorage, TargetRow]()
		{
			return Logic.GetOverrideBadgeFirstLayer(*DataStorage, TargetRow);
		}));
	LayeredImageWidget->AddLayer(TAttribute<const FSlateBrush*>::CreateLambda([&Logic, DataStorage, TargetRow]()
		{
			return Logic.GetOverrideBadgeSecondLayer(*DataStorage, TargetRow);
		}));

	return LayeredImageWidget;
}

const FOutlinerIconWidgetLogic& FOutlinerIconWidgetConstructor::GetLogic() const
{
	static FOutlinerIconWidgetLogic Logic;
	return Logic;
}

#undef LOCTEXT_NAMESPACE
