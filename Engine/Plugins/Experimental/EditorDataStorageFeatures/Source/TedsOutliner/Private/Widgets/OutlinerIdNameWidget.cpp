// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/OutlinerIdNameWidget.h"

#include "SceneOutlinerHelpers.h"
#include "Columns/SlateHeaderColumns.h"
#include "Elements/Columns/TypedElementCompatibilityColumns.h"
#include "Elements/Columns/TypedElementMiscColumns.h"
#include "Elements/Columns/TypedElementTypeInfoColumns.h"
#include "Elements/Columns/TypedElementSlateWidgetColumns.h"
#include "Elements/Framework/TypedElementAttributeBinding.h"
#include "TedsOutlinerHelpers.h"
#include "Widgets/Text/STextBlock.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(OutlinerIdNameWidget)

#define LOCTEXT_NAMESPACE "OutlinerOutlinerIdNameWidget"

//
// Cell Factory
//
void UOutlinerIdNameWidgetFactory::RegisterWidgetConstructors(UE::Editor::DataStorage::ICoreProvider& DataStorage,
	UE::Editor::DataStorage::IUiProvider& DataStorageUi) const
{
	using namespace UE::Editor::DataStorage::Queries;

	DataStorageUi.RegisterWidgetFactory<FOutlinerIdNameWidgetConstructor>(
	DataStorageUi.FindPurpose(UE::Editor::DataStorage::IUiProvider::FPurposeInfo("SceneOutliner", "Cell", NAME_None).GeneratePurposeID()),
		TColumn<FUObjectIdNameColumn>());
}

FOutlinerIdNameWidgetConstructor::FOutlinerIdNameWidgetConstructor()
	: FSimpleWidgetConstructor(StaticStruct())
{
}

TSharedPtr<SWidget> FOutlinerIdNameWidgetConstructor::CreateWidget(
	UE::Editor::DataStorage::ICoreProvider* DataStorage,
	UE::Editor::DataStorage::IUiProvider* DataStorageUi,
	UE::Editor::DataStorage::RowHandle TargetRow,
	UE::Editor::DataStorage::RowHandle WidgetRow,
	const UE::Editor::DataStorage::FMetaDataView& Arguments)
{
	using namespace UE::Editor::DataStorage;
	if (!DataStorage->IsRowAvailable(TargetRow))
	{
		return SNew(STextBlock)
			.Text(LOCTEXT("MissingRowReferenceColumn", "Unable to retrieve row reference."));
	}

	FAttributeBinder TargetRowBinder(TargetRow, DataStorage);

	TSharedPtr<SWidget> Widget = SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		.Padding(8, 0, 0, 0)
		[
			SNew(STextBlock)
				.ColorAndOpacity(FSlateColor::UseSubduedForeground())
				.Text(TargetRowBinder.BindText(&FUObjectIdNameColumn::IdName))
				.ToolTipText(TargetRowBinder.BindText(&FUObjectIdNameColumn::IdName))
				.HighlightText(UE::Editor::Outliner::Helpers::GetHighlightTextAttribute(DataStorage, WidgetRow))
		];

	return Widget;
}

FText FOutlinerIdNameWidgetConstructor::CreateWidgetDisplayNameText(UE::Editor::DataStorage::ICoreProvider* DataStorage, 
	UE::Editor::DataStorage::RowHandle Row) const
{
	return LOCTEXT("SceneOutlinerColumnIdName", "ID Name");
}

#undef LOCTEXT_NAMESPACE
