// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/OutlinerLabelWidget.h"

#include "ActorFolders/ActorFolderColumns.h"
#include "Columns/SlateHeaderColumns.h"
#include "Elements/Columns/TypedElementCompatibilityColumns.h"
#include "Elements/Columns/TypedElementFolderColumns.h"
#include "Elements/Columns/TypedElementTypeInfoColumns.h"
#include "Elements/Columns/TypedElementLabelColumns.h"
#include "Layout/Margin.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/Text/STextBlock.h"
#include "Sorting/NaturalSortKey.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(OutlinerLabelWidget)

#define LOCTEXT_NAMESPACE "OutlinerLabelWidget"

namespace UE::Editor::DataStorage
{
	FColumnSorterInterface::ESortType FOutlinerLabelWidgetSorter::GetSortType() const
	{
		return ESortType::HybridSort;
	}
	
	FText FOutlinerLabelWidgetSorter::GetShortName() const
	{
		return LOCTEXT("OutlinerLabelWidgetSorter", "Outliner Label Sorter");
	}

	int32 FOutlinerLabelWidgetSorter::Compare(const ICoreProvider& Storage, RowHandle Left, RowHandle Right) const
	{
		const int32 LeftTypePriority = GetTypeInfoPriorityForRow(Storage, Left);
		const int32 RightTypePriority = GetTypeInfoPriorityForRow(Storage, Right);

		if (LeftTypePriority != RightTypePriority)
		{
			return LeftTypePriority - RightTypePriority;
		}

		const FTypedElementLabelColumn* LeftLabelColumn = Storage.GetColumn<FTypedElementLabelColumn>(Left);
		const FTypedElementLabelColumn* RightLabelColumn = Storage.GetColumn<FTypedElementLabelColumn>(Right);
		if (LeftLabelColumn && RightLabelColumn)
		{
			// Compare using the same encoded key that CalculatePrefix emits so the
			// HybridSort's prefix buckets and comparator tiebreaker stay consistent.
			// Must use CompareNaturalSortKeys (not FString::Compare) because the encoded
			// key contains embedded 0x0000 TCHARs that would truncate strcmp-based compares.
			return CompareNaturalSortKeys(
				BuildNaturalSortKey(LeftLabelColumn->Label),
				BuildNaturalSortKey(RightLabelColumn->Label));
		}
		
		return 0;
	}

	FPrefixInfo FOutlinerLabelWidgetSorter::CalculatePrefix(const ICoreProvider& Storage, RowHandle Row, uint32 ByteIndex) const
	{
		if (const FTypedElementLabelColumn* LabelColumn = Storage.GetColumn<FTypedElementLabelColumn>(Row))
		{
			const FString SortKey = BuildNaturalSortKey(LabelColumn->Label);
			return CreateSortPrefix(ByteIndex, GetTypeInfoPriorityForRow(Storage, Row), TSortStringView(FSortCaseSensitive{}, SortKey));
		}
		return CreateSortPrefix(ByteIndex, -1);
	}

	int32 FOutlinerLabelWidgetSorter::GetTypeInfoPriorityForRow(const ICoreProvider& Storage, RowHandle Row) const
	{
		if (Storage.HasColumns<FEditorDataStorageWorldTag>(Row))
		{
			return 0;
		}
		
		if (Storage.HasColumns<FEditorDataStorageLevelTag>(Row))
		{
			return 1;
		}
		
		if(Storage.HasColumns<FFolderCompatibilityColumn>(Row))
		{
			return 2;
		}
		
		return 3;
	}
}

void UOutlinerLabelWidgetFactory::RegisterWidgetConstructors(UE::Editor::DataStorage::ICoreProvider& DataStorage,
	UE::Editor::DataStorage::IUiProvider& DataStorageUi) const
{
	using namespace UE::Editor::DataStorage::Queries;
	DataStorageUi.RegisterWidgetFactory<FOutlinerLabelHeaderConstructor>(
		DataStorageUi.FindPurpose(UE::Editor::DataStorage::IUiProvider::FPurposeInfo("SceneOutliner", "Header", NAME_None).GeneratePurposeID()),
		TColumn<FTypedElementLabelColumn>());
	
	DataStorageUi.RegisterWidgetFactory<FOutlinerLabelHeaderConstructor>(
		DataStorageUi.FindPurpose(UE::Editor::DataStorage::IUiProvider::FPurposeInfo("SceneOutliner", "Header", NAME_None).GeneratePurposeID()),
		TColumn<FTypedElementLabelColumn>() && (TColumn<FTypedElementClassTypeInfoColumn>() || TColumn<FTypedElementScriptStructTypeInfoColumn>()));
    	
	DataStorageUi.RegisterWidgetFactory<FOutlinerLabelWidgetConstructor>(
		DataStorageUi.FindPurpose(UE::Editor::DataStorage::IUiProvider::FPurposeInfo("SceneOutliner", "RowLabel", NAME_None).GeneratePurposeID()),
		TColumn<FTypedElementLabelColumn>());

	DataStorageUi.RegisterWidgetFactory<FOutlinerLabelWidgetConstructor>(
		DataStorageUi.FindPurpose(UE::Editor::DataStorage::IUiProvider::FPurposeInfo("SceneOutliner", "Cell", NAME_None).GeneratePurposeID()),
		TColumn<FTypedElementLabelColumn>());

}

void UOutlinerLabelWidgetFactory::RegisterWidgetPurposes(UE::Editor::DataStorage::IUiProvider& DataStorageUi) const
{
	DataStorageUi.RegisterWidgetPurpose(
	UE::Editor::DataStorage::IUiProvider::FPurposeInfo("SceneOutliner", "RowLabel", "Icon",
		UE::Editor::DataStorage::IUiProvider::EPurposeType::UniqueByNameAndColumn,
		LOCTEXT("IconItemCellWidgetPurpose", "The icon widget to use in cells for the Scene Outliner specific to the Item label column.")));

	DataStorageUi.RegisterWidgetPurpose(
	UE::Editor::DataStorage::IUiProvider::FPurposeInfo("SceneOutliner", "RowLabel", "Text",
		UE::Editor::DataStorage::IUiProvider::EPurposeType::UniqueByNameAndColumn,
		LOCTEXT("TextItemCellWidgetPurpose", "The text widget to use in cells for the Scene Outliner specific to the Item label column.")));
}

static void GetAllColumns(TArray<TWeakObjectPtr<const UScriptStruct>>& OutColumns, const UE::Editor::DataStorage::ICoreProvider& DataStorage, UE::Editor::DataStorage::RowHandle Row)
{
	OutColumns.Empty();
	DataStorage.ListColumns(Row, [&OutColumns](const UScriptStruct& ColumnType)
		{
			OutColumns.Emplace(&ColumnType);
			return true;
		});
}

FOutlinerLabelHeaderConstructor::FOutlinerLabelHeaderConstructor()
	: FSimpleWidgetConstructor(StaticStruct())
{
}

TSharedPtr<SWidget> FOutlinerLabelHeaderConstructor::CreateWidget(UE::Editor::DataStorage::ICoreProvider* DataStorage,
	UE::Editor::DataStorage::IUiProvider* DataStorageUi, UE::Editor::DataStorage::RowHandle TargetRow, UE::Editor::DataStorage::RowHandle WidgetRow,
	const UE::Editor::DataStorage::FMetaDataView& Arguments)
{
	DataStorage->AddColumn(WidgetRow, FHeaderWidgetSizeColumn
	{
		.ColumnSizeMode = EColumnSizeMode::Fill,
		.Width = 5.0f
	});
	
	return SNew(STextBlock)
		.Text(LOCTEXT("OutlinerLabelHeaderText", "Item Label"));
}

TArray<TSharedPtr<const UE::Editor::DataStorage::FColumnSorterInterface>> FOutlinerLabelHeaderConstructor::ConstructColumnSorters(
	UE::Editor::DataStorage::ICoreProvider* DataStorage,
	UE::Editor::DataStorage::IUiProvider* DataStorageUi,
	const UE::Editor::DataStorage::FMetaDataView& Arguments)
{
	using namespace UE::Editor::DataStorage;
	return TArray<TSharedPtr<const FColumnSorterInterface>>(
		{
			MakeShared<FOutlinerLabelWidgetSorter>()
		});
}

FOutlinerLabelWidgetConstructor::FOutlinerLabelWidgetConstructor()
	: FSimpleWidgetConstructor(StaticStruct())
{
}

TSharedPtr<SWidget> FOutlinerLabelWidgetConstructor::CreateWidget(UE::Editor::DataStorage::ICoreProvider* DataStorage,
	UE::Editor::DataStorage::IUiProvider* DataStorageUi, UE::Editor::DataStorage::RowHandle TargetRow,
	UE::Editor::DataStorage::RowHandle WidgetRow, const UE::Editor::DataStorage::FMetaDataView& Arguments)
{
	if (DataStorage->IsRowAvailable(TargetRow))
	{
		UE::Editor::DataStorage::IUiProvider::FWidgetConstructorPtr OutIconWidgetConstructorPtr;
		UE::Editor::DataStorage::IUiProvider::FWidgetConstructorPtr OutTextWidgetConstructorPtr;

		auto AssignIconWidgetToColumn = [&OutIconWidgetConstructorPtr](UE::Editor::DataStorage::IUiProvider::FWidgetConstructorPtr WidgetConstructor, TConstArrayView<TWeakObjectPtr<const UScriptStruct>>)
			{
				OutIconWidgetConstructorPtr = MoveTemp(WidgetConstructor);
				return false;
			};

		auto AssignTextWidgetToColumn = [&OutTextWidgetConstructorPtr](UE::Editor::DataStorage::IUiProvider::FWidgetConstructorPtr WidgetConstructor, TConstArrayView<TWeakObjectPtr<const UScriptStruct>>)
			{
				OutTextWidgetConstructorPtr = MoveTemp(WidgetConstructor);
				return false;
			};

		TArray<TWeakObjectPtr<const UScriptStruct>> Columns;
		GetAllColumns(Columns, *DataStorage, TargetRow);
		DataStorageUi->CreateWidgetConstructors(
			DataStorageUi->FindPurpose(UE::Editor::DataStorage::IUiProvider::FPurposeInfo("SceneOutliner", "RowLabel", "Icon").GeneratePurposeID()),
			UE::Editor::DataStorage::IUiProvider::EMatchApproach::LongestMatch, Columns, Arguments, AssignIconWidgetToColumn);
		GetAllColumns(Columns, *DataStorage, TargetRow);
		DataStorageUi->CreateWidgetConstructors(
			DataStorageUi->FindPurpose(UE::Editor::DataStorage::IUiProvider::FPurposeInfo("SceneOutliner", "RowLabel", "Text").GeneratePurposeID()),
			UE::Editor::DataStorage::IUiProvider::EMatchApproach::LongestMatch, Columns, Arguments, AssignTextWidgetToColumn);

		static const FMargin ColumnItemPadding(8, 0);
		constexpr float IconNameHorizontalPadding = 8.f;

		TSharedPtr<SWidget> IconWidget = SNullWidget::NullWidget;
		TSharedPtr<SWidget> TextWidget = SNullWidget::NullWidget;

		if (OutIconWidgetConstructorPtr)
		{
			IconWidget = DataStorageUi->ConstructWidget(WidgetRow, *OutIconWidgetConstructorPtr, Arguments);
		}

		if (OutTextWidgetConstructorPtr)
		{
			TextWidget = DataStorageUi->ConstructWidget(WidgetRow, *OutTextWidgetConstructorPtr, Arguments);
		}

		if (!IconWidget.IsValid())
		{
			IconWidget = SNullWidget::NullWidget;
		}

		if (!TextWidget.IsValid())
		{
			TextWidget = SNullWidget::NullWidget;
		}

		return SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				IconWidget.ToSharedRef()
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SSpacer)
					.Size(FVector2D(5.0f, 0.0f))
			]
			+ SHorizontalBox::Slot()
			.FillWidth(1.0f)
			.VAlign(VAlign_Center)
			[
				TextWidget.ToSharedRef()
			];
	}

	return SNullWidget::NullWidget;
}

FText FOutlinerLabelWidgetConstructor::CreateWidgetDisplayNameText(
	UE::Editor::DataStorage::ICoreProvider* DataStorage, UE::Editor::DataStorage::RowHandle Row) const
{
	return LOCTEXT("OutlinerLabelDisplayText", "Item Label");
}

#undef LOCTEXT_NAMESPACE
