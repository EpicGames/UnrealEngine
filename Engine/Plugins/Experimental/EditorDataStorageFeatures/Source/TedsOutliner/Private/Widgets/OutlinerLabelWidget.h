// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/Interfaces/TypedElementDataStorageFactory.h"
#include "Elements/Interfaces/TypedElementDataStorageUiInterface.h"
#include "UObject/ObjectMacros.h"

#include "OutlinerLabelWidget.generated.h"

struct FTypedElementLabelColumn;

namespace UE::Editor::DataStorage
{
	class FOutlinerLabelWidgetSorter final : public FColumnSorterInterface
	{
	public:
		virtual ESortType GetSortType() const override;
		
		virtual FText GetShortName() const override;

		virtual int32 Compare(const ICoreProvider& Storage, RowHandle Left, RowHandle Right) const override;
		
		virtual FPrefixInfo CalculatePrefix(const ICoreProvider& Storage, RowHandle Row, uint32 ByteIndex) const override;

	private:
		int32 GetTypeInfoPriorityForRow(const ICoreProvider& Storage, RowHandle Row) const;
	};
	}

UCLASS()
class UOutlinerLabelWidgetFactory : public UEditorDataStorageFactory
{
	GENERATED_BODY()

public:
	~UOutlinerLabelWidgetFactory() override = default;

	TEDSOUTLINER_API void RegisterWidgetConstructors(UE::Editor::DataStorage::ICoreProvider& DataStorage,
		UE::Editor::DataStorage::IUiProvider& DataStorageUi) const override;

	TEDSOUTLINER_API void RegisterWidgetPurposes(UE::Editor::DataStorage::IUiProvider& DataStorageUi) const override;
};

// Label header widget for the Scene Outliner
USTRUCT()
struct FOutlinerLabelHeaderConstructor : public FSimpleWidgetConstructor
{
	GENERATED_BODY()

public:
	TEDSOUTLINER_API FOutlinerLabelHeaderConstructor();
	~FOutlinerLabelHeaderConstructor() override = default;

	TEDSOUTLINER_API virtual TSharedPtr<SWidget> CreateWidget(
		UE::Editor::DataStorage::ICoreProvider* DataStorage,
		UE::Editor::DataStorage::IUiProvider* DataStorageUi,
		UE::Editor::DataStorage::RowHandle TargetRow,
		UE::Editor::DataStorage::RowHandle WidgetRow, 
		const UE::Editor::DataStorage::FMetaDataView& Arguments) override;
	
	TEDSOUTLINER_API virtual TArray<TSharedPtr<const UE::Editor::DataStorage::FColumnSorterInterface>> ConstructColumnSorters(
		UE::Editor::DataStorage::ICoreProvider* DataStorage,
		UE::Editor::DataStorage::IUiProvider* DataStorageUi,
		const UE::Editor::DataStorage::FMetaDataView& Arguments) override;
};

// Label widget for the Scene Outliner that shows an icon (with optional override information) + a text label
USTRUCT()
struct FOutlinerLabelWidgetConstructor : public FSimpleWidgetConstructor
{
	GENERATED_BODY()

public:
	TEDSOUTLINER_API FOutlinerLabelWidgetConstructor();
	~FOutlinerLabelWidgetConstructor() override = default;

	TEDSOUTLINER_API virtual FText CreateWidgetDisplayNameText(
		UE::Editor::DataStorage::ICoreProvider* DataStorage, UE::Editor::DataStorage::RowHandle Row = UE::Editor::DataStorage::InvalidRowHandle) const override;

	TEDSOUTLINER_API virtual TSharedPtr<SWidget> CreateWidget(
		UE::Editor::DataStorage::ICoreProvider* DataStorage,
		UE::Editor::DataStorage::IUiProvider* DataStorageUi,
		UE::Editor::DataStorage::RowHandle TargetRow,
		UE::Editor::DataStorage::RowHandle WidgetRow, 
		const UE::Editor::DataStorage::FMetaDataView& Arguments) override;
};