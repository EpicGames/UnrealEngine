// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/OutlinerLevelWidget.h"

#include "SceneOutlinerHelpers.h"
#include "Columns/TedsOutlinerColumns.h"
#include "Elements/Columns/TypedElementCompatibilityColumns.h"
#include "Elements/Columns/TypedElementMiscColumns.h"
#include "Elements/Columns/TypedElementPackageColumns.h"
#include "Elements/Framework/TypedElementAttributeBinding.h"
#include "Elements/Interfaces/TypedElementDataStorageUiInterface.h"
#include "Engine/Level.h"
#include "TedsOutlinerHelpers.h"
#include "Misc/PackageName.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "OutlinerLevelWidget"

namespace UE::Editor::DataStorage
{
	// Helper to extract the Short name from the package row
	FString GetLevelName(const ICoreProvider& Storage, RowHandle Row)
	{
		if (const FTypedElementPackageReference* PackageRefColumn = Storage.GetColumn<FTypedElementPackageReference>(Row))
		{
			if (const FTypedElementPackagePathColumn* PackagePathColumn = Storage.GetColumn<FTypedElementPackagePathColumn>(PackageRefColumn->Row))
			{
				return FPackageName::GetShortName(PackagePathColumn->Path);
			}
		}
		return FString();
	}
	
	FText FLevelWidgetSorter::GetShortName() const
	{
		return LOCTEXT("LevelWidgetSorter", "Level Sorter");
	}
    
	FColumnSorterInterface::ESortType FLevelWidgetSorter::GetSortType() const
	{
		return ESortType::HybridSort;
	}

	int32 FLevelWidgetSorter::Compare(const ICoreProvider& Storage, RowHandle Left, RowHandle Right) const
	{
		return GetLevelName(Storage, Left).Compare(GetLevelName(Storage, Right));
	}

	FPrefixInfo FLevelWidgetSorter::CalculatePrefix(const ICoreProvider& Storage, RowHandle Row, uint32 ByteIndex) const
	{
		return CreateSortPrefix(ByteIndex, TSortStringView(FSortCaseSensitive{}, GetLevelName(Storage, Row)));
	}
	
	void FLevelWidgetSearcher::GetSearchableString(FString& SearchableString, const ICoreProvider& Storage, RowHandle Row) const
	{
		SearchableString = GetLevelName(Storage, Row);
	}

	FText GetOutlinerLevelDisplayText(ICoreProvider* DataStorage, RowHandle Row)
	{
		// The Outliner row will be passed in as the Row, see if there is a world assigned to it
		if (const FTypedElementWorldColumn* WorldColumn = DataStorage->GetColumn<FTypedElementWorldColumn>(Row))
		{
			if (const TWeakObjectPtr<UWorld> World = WorldColumn->World; World.IsValid())
			{
				if (World->PersistentLevel && World->PersistentLevel->IsUsingExternalActors())
				{
					return LOCTEXT("OutlinerPackageShortNameColumnDisplayName", "Package Short Name");
				}
			}
		}
		return LOCTEXT("OutlinerLevelColumnDisplayName", "Level");
	}
}

void UOutlinerLevelWidgetFactory::RegisterWidgetConstructors(UE::Editor::DataStorage::ICoreProvider& DataStorage,
	UE::Editor::DataStorage::IUiProvider& DataStorageUi) const
{
	using namespace UE::Editor::DataStorage::Queries;

	DataStorageUi.RegisterWidgetFactory<FOutlinerLevelWidgetConstructor>(
		DataStorageUi.FindPurpose(UE::Editor::DataStorage::IUiProvider::FPurposeInfo("SceneOutliner", "Cell", NAME_None).GeneratePurposeID()),
		TColumn<FLevelColumn>());

	DataStorageUi.RegisterWidgetFactory<FOutlinerLevelHeaderWidgetConstructor>(
		DataStorageUi.FindPurpose(UE::Editor::DataStorage::IUiProvider::FPurposeInfo("SceneOutliner", "Header", NAME_None).GeneratePurposeID()),
		TColumn<FLevelColumn>());
}

FOutlinerLevelHeaderWidgetConstructor::FOutlinerLevelHeaderWidgetConstructor()
	: FSimpleWidgetConstructor(StaticStruct())
{
}

TSharedPtr<SWidget> FOutlinerLevelHeaderWidgetConstructor::CreateWidget(
	UE::Editor::DataStorage::ICoreProvider* DataStorage,
	UE::Editor::DataStorage::IUiProvider* DataStorageUi,
	UE::Editor::DataStorage::RowHandle TargetRow,
	UE::Editor::DataStorage::RowHandle WidgetRow,
	const UE::Editor::DataStorage::FMetaDataView& Arguments)
{
	using namespace UE::Editor::DataStorage;
	
	FName TabIdentifier;
	const FMetaDataEntryView OutlinerIdMeta = Arguments.FindGeneric("OutlinerIdentifier");
	if(const FString* const* IdMetaData = OutlinerIdMeta.TryGetExact<const FString*>())
	{
		TabIdentifier = FName(**IdMetaData);
	}
	const RowHandle OutlinerRow = DataStorage->LookupMappedRow(UE::Editor::Outliner::MappingDomain, FMapKey(TabIdentifier));

	return SNew(STextBlock)
		.Text(GetOutlinerLevelDisplayText(DataStorage, OutlinerRow));
}

FOutlinerLevelWidgetConstructor::FOutlinerLevelWidgetConstructor()
	: FSimpleWidgetConstructor(StaticStruct())
{
}

TSharedPtr<SWidget> FOutlinerLevelWidgetConstructor::CreateWidget(
	UE::Editor::DataStorage::ICoreProvider* DataStorage,
	UE::Editor::DataStorage::IUiProvider* DataStorageUi,
	UE::Editor::DataStorage::RowHandle TargetRow,
	UE::Editor::DataStorage::RowHandle WidgetRow,
	const UE::Editor::DataStorage::FMetaDataView& Arguments)
{
	using namespace UE::Editor::DataStorage;
	if (!DataStorage->IsRowAvailable(TargetRow))
	{
		return SNew(STextBlock)
			.Text(LOCTEXT("MissingRowReferenceColumn", "Unable to retrieve row reference."));
	}

	FAttributeBinder Binder(TargetRow, DataStorage);

	return SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		.Padding(8, 0, 0, 0)
		[
			SNew(STextBlock)
				.ColorAndOpacity(FSlateColor::UseSubduedForeground())
				.Text(Binder.BindData(&FTypedElementPackageReference::Row, [TargetRow, DataStorage](RowHandle PackageRow)
				{
					if (const FTypedElementPackagePathColumn* PackagePathColumn = DataStorage->GetColumn<FTypedElementPackagePathColumn>(PackageRow))
					{
						return FText::FromString(FPackageName::GetShortName(PackagePathColumn->Path));
					}
					if (const FLevelColumn* LevelColumn = DataStorage->GetColumn<FLevelColumn>(TargetRow))
					{
						if (const TWeakObjectPtr<ULevel> Level = LevelColumn->Level; Level.IsValid())
						{
							return FText::FromString(Level->GetOuter()->GetName());
						}
					}

					return LOCTEXT("OutlinerLevelUnknown", "<unknown>");
				}))
				.HighlightText(UE::Editor::Outliner::Helpers::GetHighlightTextAttribute(DataStorage, WidgetRow))
		];
}

FText FOutlinerLevelWidgetConstructor::CreateWidgetDisplayNameText(UE::Editor::DataStorage::ICoreProvider* DataStorage, UE::Editor::DataStorage::RowHandle Row) const
{
	return GetOutlinerLevelDisplayText(DataStorage, Row);
}

TArray<TSharedPtr<const UE::Editor::DataStorage::FColumnSorterInterface>> FOutlinerLevelWidgetConstructor::ConstructColumnSorters(
	UE::Editor::DataStorage::ICoreProvider* DataStorage,
	UE::Editor::DataStorage::IUiProvider* DataStorageUi,
	const UE::Editor::DataStorage::FMetaDataView& Arguments)
{
	using namespace UE::Editor::DataStorage;
	return TArray<TSharedPtr<const FColumnSorterInterface>>(
		{
			MakeShared<FLevelWidgetSorter>()
		});
}

TArray<TSharedPtr<const UE::Editor::DataStorage::FColumnSearcherInterface>> FOutlinerLevelWidgetConstructor::ConstructColumnSearchers(
	UE::Editor::DataStorage::ICoreProvider* DataStorage,
	UE::Editor::DataStorage::IUiProvider* DataStorageUi,
	const UE::Editor::DataStorage::FMetaDataView& Arguments)
{
	using namespace UE::Editor::DataStorage;
	return TArray<TSharedPtr<const FColumnSearcherInterface>>(
		{
			MakeShared<FLevelWidgetSearcher>()
		});
}

#undef LOCTEXT_NAMESPACE
