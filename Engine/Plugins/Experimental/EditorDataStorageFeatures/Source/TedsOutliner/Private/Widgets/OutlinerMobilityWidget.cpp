// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/OutlinerMobilityWidget.h"

#include "SceneOutlinerHelpers.h"
#include "Elements/Columns/TypedElementCompatibilityColumns.h"
#include "Elements/Columns/TypedElementMiscColumns.h"
#include "Columns/TedsActorMobilityColumns.h"
#include "Elements/Framework/TypedElementAttributeBinding.h"
#include "TedsOutlinerHelpers.h"
#include "Widgets/Text/STextBlock.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(OutlinerMobilityWidget)

#define LOCTEXT_NAMESPACE "OutlinerMobilityWidget"

FText GetComponentMobility(const EComponentMobility::Type InMobility)
{
	switch(InMobility)
	{
	case EComponentMobility::Movable:
		return LOCTEXT("OutlinerMobilityTypeMovable", "Movable");
	case EComponentMobility::Static:
		return LOCTEXT("OutlinerMobilityTypeStatic", "Static");
	case EComponentMobility::Stationary:
		return LOCTEXT("OutlinerMobilityTypeStationary", "Stationary");
	default:
		return LOCTEXT("OutlinerMobilityTypeUnknown", "<unknown>");
	}
}

namespace UE::Editor::DataStorage
{
	void FMobilityWidgetSearcher::GetSearchableString(FString& SearchableString, const ICoreProvider& Storage, const FTedsActorMobilityColumn& Column) const
	{
		SearchableString = GetComponentMobility(Column.Mobility).ToString();
	}
}

void UOutlinerMobilityWidgetFactory::RegisterWidgetConstructors(UE::Editor::DataStorage::ICoreProvider& DataStorage,
	UE::Editor::DataStorage::IUiProvider& DataStorageUi) const
{
	using namespace UE::Editor::DataStorage::Queries;

	DataStorageUi.RegisterWidgetFactory<FOutlinerMobilityWidgetConstructor>(
	DataStorageUi.FindPurpose(UE::Editor::DataStorage::IUiProvider::FPurposeInfo("SceneOutliner", "Cell", NAME_None).GeneratePurposeID()),
		TColumn<FTedsActorMobilityColumn>());
}

FOutlinerMobilityWidgetConstructor::FOutlinerMobilityWidgetConstructor()
	: FSimpleWidgetConstructor(StaticStruct())
{
}

TSharedPtr<SWidget> FOutlinerMobilityWidgetConstructor::CreateWidget(
	UE::Editor::DataStorage::ICoreProvider* DataStorage,
	UE::Editor::DataStorage::IUiProvider* DataStorageUi,
	UE::Editor::DataStorage::RowHandle TargetRow,
	UE::Editor::DataStorage::RowHandle WidgetRow,
	const UE::Editor::DataStorage::FMetaDataView& Arguments)
{
	using namespace UE::Editor::DataStorage;
	if (!DataStorage->IsRowAvailable(TargetRow))
	{
		return SNew(STextBlock)
			.Text(LOCTEXT("MissingRowReferenceColumn", "Unable to retrieve row reference."));
	}

	FAttributeBinder Binder(TargetRow, DataStorage);

	return SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(8, 0, 0, 0)
			[
				SNew(STextBlock)
					.ColorAndOpacity(FSlateColor::UseSubduedForeground())
					.Text(Binder.BindData(&FTedsActorMobilityColumn::Mobility, [](EComponentMobility::Type InMobility)
						{
							return GetComponentMobility(InMobility);
						}))
					.HighlightText(UE::Editor::Outliner::Helpers::GetHighlightTextAttribute(DataStorage, WidgetRow))
			];
}

TArray<TSharedPtr<const UE::Editor::DataStorage::FColumnSearcherInterface>> FOutlinerMobilityWidgetConstructor::ConstructColumnSearchers(
	UE::Editor::DataStorage::ICoreProvider* DataStorage,
	UE::Editor::DataStorage::IUiProvider* DataStorageUi,
	const UE::Editor::DataStorage::FMetaDataView& Arguments)
{
	using namespace UE::Editor::DataStorage;
	return TArray<TSharedPtr<const FColumnSearcherInterface>>(
		{
			MakeShared<FMobilityWidgetSearcher>()
		});
}

#undef LOCTEXT_NAMESPACE
