// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/OutlinerPinnedWidget.h"

#include "ActorDesc/TedsActorDescColumns.h"
#include "Columns/SlateHeaderColumns.h"
#include "Columns/TedsActorWorldPartitionColumns.h"
#include "Columns/TedsOutlinerColumns.h"
#include "Elements/Columns/TypedElementMiscColumns.h"
#include "Compatibility/SceneOutlinerTedsBridge.h"
#include "DataStorage/Features.h"
#include "Elements/Columns/TypedElementCompatibilityColumns.h"
#include "Elements/Columns/TypedElementSelectionColumns.h"
#include "Elements/Framework/TypedElementAttributeBinding.h"
#include "Elements/Framework/TypedElementQueryBuilder.h"
#include "Elements/Interfaces/TypedElementDataStorageInterface.h"
#include "Widgets/SBoxPanel.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(OutlinerPinnedWidget)

#define LOCTEXT_NAMESPACE "OutlinerPinnedWidget"

//
// Factory
//

FOutlinerPinnedWidgetConstructor::FOutlinerPinnedWidgetConstructor()
	: FSimpleWidgetConstructor(StaticStruct())
{
}

FText FOutlinerPinnedWidgetConstructor::CreateWidgetDisplayNameText(
	UE::Editor::DataStorage::ICoreProvider* DataStorage, UE::Editor::DataStorage::RowHandle Row) const
{
	return LOCTEXT("SceneOutlinerPinnedColumn", "Pinned");
}

void UOutlinerPinnedFactory::RegisterWidgetConstructors(UE::Editor::DataStorage::ICoreProvider& DataStorage,
	UE::Editor::DataStorage::IUiProvider& DataStorageUi) const
{
	using namespace UE::Editor::DataStorage::Queries;
	DataStorageUi.RegisterWidgetFactory<FOutlinerPinnedWidgetConstructor>(
		DataStorageUi.FindPurpose(IUiProvider::FPurposeInfo("SceneOutliner", "Cell", NAME_None).GeneratePurposeID()),
		TColumn<FWorldPartitionPinnedColumn>());
	DataStorageUi.RegisterWidgetFactory<FOutlinerPinnedHeaderConstructor>(
		DataStorageUi.FindPurpose(IUiProvider::FPurposeInfo("SceneOutliner", "Header", NAME_None).GeneratePurposeID()),
		TColumn<FWorldPartitionPinnedColumn>());
}

TSharedPtr<SWidget> FOutlinerPinnedWidgetConstructor::CreateWidget(UE::Editor::DataStorage::ICoreProvider* DataStorage,
	UE::Editor::DataStorage::IUiProvider* DataStorageUi,
	UE::Editor::DataStorage::RowHandle TargetRow, UE::Editor::DataStorage::RowHandle WidgetRow,
	const UE::Editor::DataStorage::FMetaDataView& Arguments)
{
	return SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		[
			SNew(STedsPinnedWidget, TargetRow)
				.ToolTipText(LOCTEXT("SceneOutlinerPinnedToggleTooltip",
					"Toggles Force Load: Keep the selected items loaded in the editor even when they don't overlap a loaded World Partition region."))
		];
}

//
// Header Constructor
//

FOutlinerPinnedHeaderConstructor::FOutlinerPinnedHeaderConstructor()
	: FSimpleWidgetConstructor(StaticStruct())
{
}

TSharedPtr<SWidget> FOutlinerPinnedHeaderConstructor::CreateWidget(UE::Editor::DataStorage::ICoreProvider* DataStorage,
	UE::Editor::DataStorage::IUiProvider* DataStorageUi,
	UE::Editor::DataStorage::RowHandle TargetRow, UE::Editor::DataStorage::RowHandle WidgetRow,
	const UE::Editor::DataStorage::FMetaDataView& Arguments)
{
	DataStorage->AddColumn(WidgetRow, FHeaderWidgetSizeColumn
	{
		.ColumnSizeMode = EColumnSizeMode::Fixed,
		.Width = 24.0f
	});

	return SNew(SImage)
		.DesiredSizeOverride(FVector2D(16.f, 16.f))
		.Image(FAppStyle::Get().GetBrush(TEXT("Icons.Unpinned")))
		.ToolTipText(LOCTEXT("SceneOutlinerPinnedHeaderTooltip", "Force Load"));
}

//
// STedsPinnedWidget
//

void STedsPinnedWidget::Construct(const FArguments&, const RowHandle& InTargetRow)
{
	using namespace UE::Editor::DataStorage;

	TargetRow = InTargetRow;

	static const FName NAME_PinnedBrush = TEXT("Icons.Pinned");
	static const FName NAME_UnpinnedBrush = TEXT("Icons.Unpinned");

	FAttributeBinder Binder(InTargetRow, GetDataStorage());

	bIsPinnedAttr = Binder.BindData(&FWorldPartitionPinnedColumn::bIsPinned);
	bIsSelectedAttr = Binder.BindColumnsPresence<FTypedElementSelectionColumn>([](bool bHasColumn) { return bHasColumn; });

	SImage::Construct(
		SImage::FArguments()
		.ColorAndOpacity(this, &STedsPinnedWidget::GetForegroundColor)
		.Image(Binder.BindData(&FWorldPartitionPinnedColumn::bIsPinned,
			[Pinned = FAppStyle::Get().GetBrush(NAME_PinnedBrush),
			Unpinned = FAppStyle::Get().GetBrush(NAME_UnpinnedBrush)]
			(const bool& bIsPinned) -> const FSlateBrush*
			{
				return bIsPinned ? Pinned : Unpinned;
			}))
	);
}

FReply STedsPinnedWidget::HandleClick()
{
	const bool bNewPinnedState = !bIsPinnedAttr.Get();

	TArray<RowHandle> SelectedRows;
	GetSelectedRows(SelectedRows);

	if (SelectedRows.Contains(TargetRow))
	{
		for (const RowHandle Row : SelectedRows)
		{
			SetIsPinned(Row, bNewPinnedState);
		}
	}
	else
	{
		SetIsPinned(TargetRow, bNewPinnedState);
	}

	return FReply::Handled();
}

FReply STedsPinnedWidget::OnMouseButtonDoubleClick(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent)
{
	return HandleClick();
}

FReply STedsPinnedWidget::OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton)
	{
		return HandleClick().PreventThrottling();
	}
	return FReply::Unhandled();
}

FSlateColor STedsPinnedWidget::GetForegroundColor() const
{
	if (!bIsPinnedAttr.Get() && !IsHovered() && !bIsSelectedAttr.Get())
	{
		return FLinearColor::Transparent;
	}

	return IsHovered() ? FSlateColor::UseForeground() : FSlateColor::UseSubduedForeground();
}

void STedsPinnedWidget::SetIsPinned(RowHandle InRow, bool bPinned)
{
	using namespace UE::Editor::DataStorage;

	if (ICoreProvider* DataStorage = GetDataStorage())
	{
		CommitPinnedState(*DataStorage, InRow, bPinned);
	}
}

void STedsPinnedWidget::CommitPinnedState(UE::Editor::DataStorage::ICoreProvider& DataStorage, RowHandle Row, bool bPinned)
{
	if (UE::Editor::DataStorage::FWorldPartitionPinnedColumn* PinnedColumn = DataStorage.GetColumn<UE::Editor::DataStorage::FWorldPartitionPinnedColumn>(Row))
	{
		PinnedColumn->bIsPinned = bPinned;
	}
	DataStorage.AddColumn<FTypedElementSyncBackToWorldTag>(Row);
}

void STedsPinnedWidget::GetSelectedRows(TArray<RowHandle>& OutSelectedRows) const
{
	using namespace UE::Editor::DataStorage;
	using namespace UE::Editor::DataStorage::Queries;

	if (ICoreProvider* DataStorage = GetDataStorage())
	{
		if (const FTypedElementSelectionColumn* SelectionColumn = DataStorage->GetColumn<FTypedElementSelectionColumn>(TargetRow))
		{
			FName SelectionSet = SelectionColumn->SelectionSet;

			static QueryHandle AllSelectedItemsQuery =
				DataStorage->RegisterQuery(
					Select()
						.ReadOnly<FTypedElementSelectionColumn>()
					.Where()
					.Compile());

			DataStorage->RunQuery(AllSelectedItemsQuery,
				CreateDirectQueryCallbackBinding([&OutSelectedRows, SelectionSet](const IDirectQueryContext&, RowHandle Row, const FTypedElementSelectionColumn& InSelectionColumn)
				{
					if (InSelectionColumn.SelectionSet == SelectionSet)
					{
						OutSelectedRows.Add(Row);
					}
				}));
		}
	}
}

UE::Editor::DataStorage::ICoreProvider* STedsPinnedWidget::GetDataStorage()
{
	using namespace UE::Editor::DataStorage;
	return GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName);
}

#undef LOCTEXT_NAMESPACE
