// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/OutlinerSubPackageWidget.h"

#include "SceneOutlinerHelpers.h"
#include "Columns/TedsActorWorldPartitionColumns.h"
#include "Elements/Framework/TypedElementAttributeBinding.h"
#include "TedsOutlinerHelpers.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(OutlinerSubPackageWidget)

#define LOCTEXT_NAMESPACE "OutlinerSubPackageWidget"

void UOutlinerSubPackageWidgetFactory::RegisterWidgetConstructors(UE::Editor::DataStorage::ICoreProvider& DataStorage,
	UE::Editor::DataStorage::IUiProvider& DataStorageUi) const
{
	using namespace UE::Editor::DataStorage::Queries;

	DataStorageUi.RegisterWidgetFactory<FOutlinerSubPackageWidgetConstructor>(
		DataStorageUi.FindPurpose(IUiProvider::FPurposeInfo("SceneOutliner", "Cell", NAME_None).GeneratePurposeID()),
		TColumn<FWorldPartitionSubPackageColumn>());
}

FOutlinerSubPackageWidgetConstructor::FOutlinerSubPackageWidgetConstructor()
	: FSimpleWidgetConstructor(StaticStruct())
{
}

TSharedPtr<SWidget> FOutlinerSubPackageWidgetConstructor::CreateWidget(
	UE::Editor::DataStorage::ICoreProvider* DataStorage,
	UE::Editor::DataStorage::IUiProvider* DataStorageUi,
	UE::Editor::DataStorage::RowHandle TargetRow,
	UE::Editor::DataStorage::RowHandle WidgetRow,
	const UE::Editor::DataStorage::FMetaDataView& Arguments)
{
	using namespace UE::Editor::DataStorage;

	if (!DataStorage->IsRowAvailable(TargetRow))
	{
		return SNew(STextBlock)
			.Text(LOCTEXT("MissingRowReferenceColumn", "Unable to retrieve row reference."));
	}

	FAttributeBinder Binder(TargetRow, DataStorage);

	return SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		.Padding(8, 0, 0, 0)
		[
			SNew(STextBlock)
			.ColorAndOpacity(FSlateColor::UseSubduedForeground())
			.Text(Binder.BindText(&FWorldPartitionSubPackageColumn::SubPackage))
			.HighlightText(UE::Editor::Outliner::Helpers::GetHighlightTextAttribute(DataStorage, WidgetRow))
		];
}

FText FOutlinerSubPackageWidgetConstructor::CreateWidgetDisplayNameText(
	UE::Editor::DataStorage::ICoreProvider* DataStorage,
	UE::Editor::DataStorage::RowHandle Row) const
{
	return LOCTEXT("OutlinerSubPackageWidgetDisplayName", "Sub Package");
}

#undef LOCTEXT_NAMESPACE
