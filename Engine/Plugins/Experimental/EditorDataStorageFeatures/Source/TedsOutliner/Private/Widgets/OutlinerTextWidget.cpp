// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/OutlinerTextWidget.h"

#include "ActorEditorUtils.h"
#include "ISceneOutliner.h"
#include "ITedsTableViewer.h"
#include "TedsOutlinerHelpers.h"
#include "TedsTableViewerUtils.h"
#include "TedsTableViewerWidgetColumns.h"
#include "Columns/SceneOutlinerColumns.h"
#include "Columns/SlateDelegateColumns.h"
#include "DataStorage/Features.h"
#include "Elements/Columns/TypedElementLabelColumns.h"
#include "Elements/Columns/TypedElementMiscColumns.h"
#include "Elements/Columns/TypedElementSlateWidgetColumns.h"
#include "Elements/Framework/TypedElementAttributeBinding.h"
#include "Settings/EditorStyleSettings.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"
#include "Widgets/Text/STextBlock.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(OutlinerTextWidget)

#define LOCTEXT_NAMESPACE "OutlinerTextWidget"

namespace UE::Editor::Outliner::TextWidget::Private
{
	// We are using an inheritance based model with a custom SWidget class just so we can manage lifetimes of delegates easier using BindSP
	class SOutlinerEditableTextWidget : public SInlineEditableTextBlock
	{
	public:
		SLATE_BEGIN_ARGS(SOutlinerEditableTextWidget)
			: _DataStorage(nullptr)
			, _DataRow(DataStorage::InvalidRowHandle)
			, _WidgetRow(DataStorage::InvalidRowHandle)
		{}

		SLATE_ARGUMENT(DataStorage::ICoreProvider*, DataStorage)
		SLATE_ARGUMENT(DataStorage::RowHandle, DataRow)
		SLATE_ARGUMENT(DataStorage::RowHandle, WidgetRow)
	SLATE_END_ARGS()
		
		void Construct(const FArguments& InArgs)
		{
			DataStorage = InArgs._DataStorage;
			DataRow = InArgs._DataRow;
			WidgetRow = InArgs._WidgetRow;
			
			using namespace UE::Editor::DataStorage;

			FAttributeBinder DataRowBinder(DataRow, DataStorage);
			FAttributeBinder WidgetRowBinder(WidgetRow, DataStorage);

			

			const bool bShouldUseMiddleEllipsis = GetDefault<UEditorStyleSettings>()->bEnableMiddleEllipsis;

			SInlineEditableTextBlock::Construct(
				SInlineEditableTextBlock::FArguments()
				.OnTextCommitted_Static(&FOutlinerTextWidgetConstructor::OnCommitText, DataStorage, DataRow)
				.OnVerifyTextChanged_Static(&FOutlinerTextWidgetConstructor::OnVerifyText)
				.Text(DataRowBinder.BindText(&FTypedElementLabelColumn::Label))
				.ToolTipText(DataRowBinder.BindText(&FTypedElementLabelColumn::Label))
				.OverflowPolicy(bShouldUseMiddleEllipsis ? ETextOverflowPolicy::MiddleEllipsis : TOptional<ETextOverflowPolicy>())
				.IsSelected(WidgetRowBinder.BindEvent(&FExternalWidgetSelectionColumn::IsSelected))
				.HighlightText(UE::Editor::Outliner::Helpers::GetHighlightTextAttribute(DataStorage, WidgetRow))
				.ColorAndOpacity(Helpers::GetLabelWidgetForegroundColor(DataStorage, DataRow, WidgetRow))
				.DoubleSelectDelay(0.0f)
			);

			BindTableViewerRenameDelegate();
		}

		virtual ~SOutlinerEditableTextWidget() override = default;

	protected:

		void BindTableViewerRenameDelegate()
		{
			// The Teds Outliner directly uses this column to enter edit mode
			DataStorage->AddColumn<FWidgetEnterEditModeColumn>(WidgetRow, FWidgetEnterEditModeColumn{
				.OnEnterEditMode = MakeShared<FSimpleDelegate>(FSimpleDelegate::CreateLambda([this]()
				{
					EnterEditingMode();
				}))
				});

			// The table viewer, if present, is the grandparent widget of our widget row (Table -> Row -> Cell)
			const DataStorage::RowHandle ParentRow = DataStorage::TableViewerUtils::GetTableViewerUiRow(DataStorage, WidgetRow);

			if (DataStorage->IsRowAvailable(ParentRow))
			{
				// Bind to selection change on the table viewer
				if (DataStorage::FTableViewerSelectionChangedColumn* SelectionChangedColumn =
					DataStorage->GetColumn<DataStorage::FTableViewerSelectionChangedColumn>(ParentRow))
				{
					if (!SelectionChangedColumn->OnSelectionChanged)
					{
						SelectionChangedColumn->OnSelectionChanged = MakeShared<FSimpleMulticastDelegate>();
					}
					SelectionChangedColumn->OnSelectionChanged->AddSP(this, &SOutlinerEditableTextWidget::OnTableViewerSelectionChanged, ParentRow);
				}
			}
		}

		void OnTableViewerSelectionChanged(DataStorage::RowHandle TableViewerRow)
		{
			DataStorage::FTableViewerRenameSelectionColumn* RenameColumn = DataStorage->GetColumn<DataStorage::FTableViewerRenameSelectionColumn>(TableViewerRow);
			DataStorage::FTableViewerReferenceColumn* ReferenceColumn = DataStorage->GetColumn<FTedsTableViewerReferenceColumn>(TableViewerRow);

			// On selection change, if the data row for this widget is selected add it as 'active' for the rename. So when FTableViewerRenameSelectionColumn
			// is called on the table viewer the event is propagated to this widget
			if (RenameColumn && ReferenceColumn)
			{
				if (RenameColumn->OnRenameSelection)
				{
					RenameColumn->OnRenameSelection->RemoveAll(this);
				}

				if (TSharedPtr<DataStorage::ITableViewer> TableViewer = ReferenceColumn->TableViewerWeak.Pin())
				{
					TableViewer->ForEachSelectedRow([this, RenameColumn](DataStorage::RowHandle SelectedRow)
						{
							if (SelectedRow == DataRow)
							{
								if (!RenameColumn->OnRenameSelection)
								{
									RenameColumn->OnRenameSelection = MakeShared<FSimpleMulticastDelegate>();
								}
								RenameColumn->OnRenameSelection->AddSPLambda(this, [this]()
									{
										EnterEditingMode();
									});
							}
						});
				}
			}
		}

	protected:

		DataStorage::ICoreProvider* DataStorage;
		DataStorage::RowHandle WidgetRow;
		DataStorage::RowHandle DataRow;
	};
}

template<>
struct TWidgetTypeTraits<UE::Editor::Outliner::TextWidget::Private::SOutlinerEditableTextWidget>
{
	static constexpr bool SupportsInvalidation() { return true; }
};

void UOutlinerTextWidgetFactory::RegisterWidgetConstructors(UE::Editor::DataStorage::ICoreProvider& DataStorage,
	UE::Editor::DataStorage::IUiProvider& DataStorageUi) const
{
	using namespace UE::Editor::DataStorage::Queries;

	DataStorageUi.RegisterWidgetFactory<FOutlinerTextWidgetConstructor>(DataStorageUi.FindPurpose(
		IUiProvider::FPurposeInfo("SceneOutliner", "RowLabel", "Text").GeneratePurposeID()),
		TColumn<FTypedElementLabelColumn>());
}

FOutlinerTextWidgetConstructor::FOutlinerTextWidgetConstructor()
	: Super(FOutlinerTextWidgetConstructor::StaticStruct())
{
}

FOutlinerTextWidgetConstructor::FOutlinerTextWidgetConstructor(const UScriptStruct* InTypeInfo)
	: Super(InTypeInfo)
{
}

TSharedPtr<SWidget> FOutlinerTextWidgetConstructor::CreateWidget(UE::Editor::DataStorage::ICoreProvider* DataStorage,
	UE::Editor::DataStorage::IUiProvider* DataStorageUi,
	UE::Editor::DataStorage::RowHandle TargetRow,
	UE::Editor::DataStorage::RowHandle WidgetRow, const UE::Editor::DataStorage::FMetaDataView& Arguments)
{
	using namespace UE::Editor::DataStorage;
	if (!DataStorage->IsRowAvailable(TargetRow))
	{
		return SNew(STextBlock)
			.Text(LOCTEXT("MissingRowReferenceColumn", "Unable to retrieve row reference."));
	}

	TSharedPtr<SWidget> Result = SNullWidget::NullWidget;

	bool bEditable = false;

	FMetaDataEntryView MetaDataEntryView = Arguments.FindForColumn<FTypedElementLabelColumn>(IsEditableName);

	if (MetaDataEntryView.IsSet())
	{
		if (const bool* EditableMetaDataPtr = MetaDataEntryView.TryGetExact<bool>())
		{
			bEditable = *EditableMetaDataPtr;
		}
	}

	if (bEditable)
	{
		Result = CreateEditableWidget(DataStorage, DataStorageUi, TargetRow, WidgetRow, Arguments);
	}
	else
	{
		Result = CreateNonEditableWidget(DataStorage, DataStorageUi, TargetRow, WidgetRow, Arguments);
	}

	return Result;
}

void FOutlinerTextWidgetConstructor::OnCommitText(
	const FText& NewText,
	ETextCommit::Type,
	UE::Editor::DataStorage::ICoreProvider* DataStorage,
	UE::Editor::DataStorage::RowHandle TargetRow)
{
	FString NewLabelText = NewText.ToString();

	// This callback happens on the game thread so it's safe to directly call into the data storage.
	if (FTypedElementLabelHashColumn* LabelHashColumn = DataStorage->GetColumn<FTypedElementLabelHashColumn>(TargetRow))
	{
		LabelHashColumn->LabelHash = CityHash64(reinterpret_cast<const char*>(*NewLabelText), NewLabelText.Len() * sizeof(**NewLabelText));
	}
	if (FTypedElementLabelColumn* LabelColumn = DataStorage->GetColumn<FTypedElementLabelColumn>(TargetRow))
	{
		LabelColumn->Label = MoveTemp(NewLabelText);
	}
	DataStorage->AddColumn<FTypedElementSyncBackToWorldTag>(TargetRow);
}

bool FOutlinerTextWidgetConstructor::OnVerifyText(const FText& Label, FText& ErrorMessage)
{
	// Note: The use of actor specific functionality should be minimized, but this function acts generic enough that the 
	// use of actor is just in names.
	return FActorEditorUtils::ValidateActorName(Label, ErrorMessage);
}

TSharedPtr<SWidget> FOutlinerTextWidgetConstructor::CreateEditableWidget(
	UE::Editor::DataStorage::ICoreProvider* DataStorage,
	UE::Editor::DataStorage::IUiProvider* DataStorageUi,
	UE::Editor::DataStorage::RowHandle TargetRow,
	UE::Editor::DataStorage::RowHandle WidgetRow,
	const UE::Editor::DataStorage::FMetaDataView& Arguments)
{
	return CreateEditableTextBlock(DataStorage, TargetRow, WidgetRow);
}

TSharedPtr<SWidget> FOutlinerTextWidgetConstructor::CreateNonEditableWidget(
	UE::Editor::DataStorage::ICoreProvider* DataStorage,
	UE::Editor::DataStorage::IUiProvider* DataStorageUi,
	UE::Editor::DataStorage::RowHandle TargetRow,
	UE::Editor::DataStorage::RowHandle WidgetRow,
	const UE::Editor::DataStorage::FMetaDataView& Arguments)
{
	return CreateNonEditableTextBlock(DataStorage, TargetRow, WidgetRow);
}

TSharedPtr<SInlineEditableTextBlock> FOutlinerTextWidgetConstructor::CreateEditableTextBlock(
	UE::Editor::DataStorage::ICoreProvider* DataStorage,
	UE::Editor::DataStorage::RowHandle TargetRow,
	UE::Editor::DataStorage::RowHandle WidgetRow) const
{

	return SNew(UE::Editor::Outliner::TextWidget::Private::SOutlinerEditableTextWidget)
		.DataStorage(DataStorage)
		.DataRow(TargetRow)
		.WidgetRow(WidgetRow);
}

TSharedPtr<STextBlock> FOutlinerTextWidgetConstructor::CreateNonEditableTextBlock(
	UE::Editor::DataStorage::ICoreProvider* DataStorage,
	UE::Editor::DataStorage::RowHandle TargetRow,
	UE::Editor::DataStorage::RowHandle WidgetRow) const
{
	using namespace UE::Editor::DataStorage;
	FAttributeBinder TargetRowBinder(TargetRow, DataStorage);

	const RowHandle OutlinerWidgetRowHandle = TableViewerUtils::GetTableViewerUiRow(DataStorage, WidgetRow);
	FAttributeBinder OutlinerWidgetRowBinder(OutlinerWidgetRowHandle, DataStorage);

	TSharedPtr<STextBlock> TextBlock = SNew(STextBlock)
		.Text(TargetRowBinder.BindText(&FTypedElementLabelColumn::Label))
		.ToolTipText(TargetRowBinder.BindText(&FTypedElementLabelColumn::Label))
		.HighlightText(OutlinerWidgetRowBinder.BindData(&FHighlightTextColumn::HighlightText))
		.ColorAndOpacity(UE::Editor::Outliner::Helpers::GetLabelWidgetForegroundColor(DataStorage, TargetRow, WidgetRow));

	return TextBlock;
}

#undef LOCTEXT_NAMESPACE
