// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/OutlinerUncachedLightsWidget.h"

#include "TedsOutlinerHelpers.h"
#include "Columns/TedsActorMobilityColumns.h"
#include "Columns/TedsActorUncachedLightsColumns.h"
#include "Components/HorizontalBox.h"
#include "Elements/Columns/TypedElementCompatibilityColumns.h"
#include "Elements/Framework/TypedElementAttributeBinding.h"
#include "GameFramework/Actor.h"
#include "Widgets/Text/STextBlock.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(OutlinerUncachedLightsWidget)

#define LOCTEXT_NAMESPACE "OutlinerUncachedLightsWidget"

int32 GetNumUncachedLightsFromObjColumn(const FTypedElementUObjectColumn* ObjectColumn)
{
	if (const AActor* ActorInstance = Cast<AActor>(ObjectColumn->Object))
	{
		return ActorInstance->GetNumUncachedStaticLightingInteractions();
	}
	return 0;
}

namespace UE::Editor::DataStorage
{
	void FUncachedLightsWidgetSearcher::GetSearchableString(FString& SearchableString, const ICoreProvider& Storage, RowHandle Row) const
	{
		if (Storage.HasColumns<FTedsActorUncachedLightsTag>(Row))
		{
			if (const FTypedElementUObjectColumn* ObjectColumn = Storage.GetColumn<FTypedElementUObjectColumn>(Row))
			{
				const int32 Count = GetNumUncachedLightsFromObjColumn(ObjectColumn);
				if (Count > 0)
				{
					SearchableString = FString::Printf(TEXT("%7d"), Count);
				}
			}
		}
	}

	FColumnSorterInterface::ESortType FUncachedLightsWidgetSorter::GetSortType() const
	{
		return ESortType::FixedSize64;
	}
	
	FText FUncachedLightsWidgetSorter::GetShortName() const
	{
		return LOCTEXT("OutlinerUncachedLightsWidgetSorter", "Outliner Uncached Lights Sorter");
	}

	int32 FUncachedLightsWidgetSorter::Compare(const ICoreProvider& Storage, RowHandle Left, RowHandle Right) const
	{
		return 0;
	}

	FPrefixInfo FUncachedLightsWidgetSorter::CalculatePrefix(const ICoreProvider& Storage, RowHandle Row, uint32 ByteIndex) const
	{
		if (Storage.HasColumns<FTedsActorUncachedLightsTag>(Row))
		{
			if (const FTypedElementUObjectColumn* ObjectColumn = Storage.GetColumn<FTypedElementUObjectColumn>(Row))
			{
				const int32 Count = GetNumUncachedLightsFromObjColumn(ObjectColumn);
				if (Count > 0)
				{
					return CreateSortPrefix(ByteIndex, Count);
				}
			}
		}
		
		return CreateSortPrefix(ByteIndex, -1);
	}
}

void UOutlinerUncachedLightsWidgetFactory::RegisterWidgetConstructors(UE::Editor::DataStorage::ICoreProvider& DataStorage,
	UE::Editor::DataStorage::IUiProvider& DataStorageUi) const
{
	using namespace UE::Editor::DataStorage::Queries;

	DataStorageUi.RegisterWidgetFactory<FOutlinerUncachedLightsWidgetConstructor>(
		DataStorageUi.FindPurpose(UE::Editor::DataStorage::IUiProvider::FPurposeInfo("SceneOutliner", "Cell", NAME_None).GeneratePurposeID()),
		TColumn<FTedsActorUncachedLightsTag>());
}

FOutlinerUncachedLightsWidgetConstructor::FOutlinerUncachedLightsWidgetConstructor()
	: FSimpleWidgetConstructor(StaticStruct())
{
}

TSharedPtr<SWidget> FOutlinerUncachedLightsWidgetConstructor::CreateWidget(
	UE::Editor::DataStorage::ICoreProvider* DataStorage,
	UE::Editor::DataStorage::IUiProvider* DataStorageUi,
	UE::Editor::DataStorage::RowHandle TargetRow,
	UE::Editor::DataStorage::RowHandle WidgetRow,
	const UE::Editor::DataStorage::FMetaDataView& Arguments)
{
	using namespace UE::Editor::DataStorage;

	FAttributeBinder Binder(TargetRow, DataStorage);

	return SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		.Padding(8.f, 0.f, 0.f, 0.f)
		[
			SNew(STextBlock)
			.ColorAndOpacity(FSlateColor::UseSubduedForeground())
			.Text(Binder.BindColumn<FTypedElementUObjectColumn>([](const FTypedElementUObjectColumn& ObjectColumn)
			{
				const int32 Count = GetNumUncachedLightsFromObjColumn(&ObjectColumn);
				if (Count > 0)
				{
					return FText::AsNumber(Count);
				}
				return FText::GetEmpty();
			}))
			.HighlightText(UE::Editor::Outliner::Helpers::GetHighlightTextAttribute(DataStorage, WidgetRow))
		];
}

FText FOutlinerUncachedLightsWidgetConstructor::CreateWidgetDisplayNameText(
	UE::Editor::DataStorage::ICoreProvider* DataStorage, UE::Editor::DataStorage::RowHandle Row) const
{
	return LOCTEXT("UncachedLightsDisplayName", "# Uncached Lights");
}

TArray<TSharedPtr<const UE::Editor::DataStorage::FColumnSearcherInterface>> FOutlinerUncachedLightsWidgetConstructor::ConstructColumnSearchers(
	UE::Editor::DataStorage::ICoreProvider* DataStorage,
	UE::Editor::DataStorage::IUiProvider* DataStorageUi,
	const UE::Editor::DataStorage::FMetaDataView& Arguments)
{
	using namespace UE::Editor::DataStorage;
	return TArray<TSharedPtr<const FColumnSearcherInterface>>(
		{
			MakeShared<FUncachedLightsWidgetSearcher>()
		});
}

TArray<TSharedPtr<const UE::Editor::DataStorage::FColumnSorterInterface>> FOutlinerUncachedLightsWidgetConstructor::ConstructColumnSorters(
	UE::Editor::DataStorage::ICoreProvider* DataStorage,
	UE::Editor::DataStorage::IUiProvider* DataStorageUi,
	const UE::Editor::DataStorage::FMetaDataView& Arguments)
{
	using namespace UE::Editor::DataStorage;
	return TArray<TSharedPtr<const FColumnSorterInterface>>(
		{
			MakeShared<FUncachedLightsWidgetSorter>()
		});
}

#undef LOCTEXT_NAMESPACE
