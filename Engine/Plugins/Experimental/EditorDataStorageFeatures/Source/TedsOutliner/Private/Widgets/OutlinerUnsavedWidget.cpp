// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/OutlinerUnsavedWidget.h"

#include "Columns/SlateHeaderColumns.h"
#include "Elements/Columns/TypedElementPackageColumns.h"
#include "Elements/Framework/TypedElementAttributeBinding.h"
#include "Styling/AppStyle.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Text/STextBlock.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(OutlinerUnsavedWidget)

#define LOCTEXT_NAMESPACE "OutlinerUnsavedWidget"

namespace UE::Editor::DataStorage
{
	FText FUnsavedWidgetSorter::GetShortName() const
	{
		return LOCTEXT("UnsavedWidgetSorter", "Unsaved Sorter");
	}

	FColumnSorterInterface::ESortType FUnsavedWidgetSorter::GetSortType() const
	{
		return ESortType::FixedSize64;
	}

	int32 FUnsavedWidgetSorter::Compare(const ICoreProvider& Storage, RowHandle Left, RowHandle Right) const
	{
		return 0;
	}

	FPrefixInfo FUnsavedWidgetSorter::CalculatePrefix(
		const ICoreProvider& Storage, RowHandle Row, uint32 ByteIndex) const
	{
		return CreateSortPrefix(ByteIndex, Storage.HasColumns<FTedsPackageDirtyTag>(Row));
	}
}

void UOutlinerUnsavedWidgetFactory::RegisterWidgetConstructors(UE::Editor::DataStorage::ICoreProvider& DataStorage,
	UE::Editor::DataStorage::IUiProvider& DataStorageUi) const
{
	using namespace UE::Editor::DataStorage::Queries;

	DataStorageUi.RegisterWidgetFactory<FOutlinerUnsavedWidgetConstructor>(
	DataStorageUi.FindPurpose(UE::Editor::DataStorage::IUiProvider::FPurposeInfo("General", "Cell", NAME_None).GeneratePurposeID()),
		TColumn<FTedsPrimaryPackageObjectTag>());
	DataStorageUi.RegisterWidgetFactory<FOutlinerUnsavedHeaderConstructor>(
		DataStorageUi.FindPurpose(UE::Editor::DataStorage::IUiProvider::FPurposeInfo("General", "Header", NAME_None).GeneratePurposeID()),
		TColumn<FTedsPrimaryPackageObjectTag>());
}

FOutlinerUnsavedWidgetConstructor::FOutlinerUnsavedWidgetConstructor()
	: FSimpleWidgetConstructor(StaticStruct())
{
}

FOutlinerUnsavedHeaderConstructor::FOutlinerUnsavedHeaderConstructor()
	: FSimpleWidgetConstructor(StaticStruct())
{
}

TSharedPtr<SWidget> FOutlinerUnsavedHeaderConstructor::CreateWidget(UE::Editor::DataStorage::ICoreProvider* DataStorage,
	UE::Editor::DataStorage::IUiProvider* DataStorageUi,
	UE::Editor::DataStorage::RowHandle TargetRow, UE::Editor::DataStorage::RowHandle WidgetRow, const UE::Editor::DataStorage::FMetaDataView& Arguments)
{
	DataStorage->AddColumn(WidgetRow, FHeaderWidgetSizeColumn
	{
		.ColumnSizeMode = EColumnSizeMode::Fixed,
		.Width = 24.0f
	});

	return SNew(SImage)
		.DesiredSizeOverride(FVector2D(12.f, 12.f))
		.Image(FAppStyle::Get().GetBrush("Icons.DirtyBadge"))
		.ToolTipText(LOCTEXT("SceneOutlinerUnsavedHeaderTooltip", "Unsaved"));
}

TSharedPtr<SWidget> FOutlinerUnsavedWidgetConstructor::CreateWidget(
	UE::Editor::DataStorage::ICoreProvider* DataStorage,
	UE::Editor::DataStorage::IUiProvider* DataStorageUi,
	UE::Editor::DataStorage::RowHandle TargetRow,
	UE::Editor::DataStorage::RowHandle WidgetRow,
	const UE::Editor::DataStorage::FMetaDataView& Arguments)
{
	using namespace UE::Editor::DataStorage;
	if (!DataStorage->IsRowAvailable(TargetRow))
	{
		return SNew(STextBlock)
			.Text(LOCTEXT("MissingRowReferenceColumn", "Unable to retrieve row reference."));
	}

	FAttributeBinder Binder(TargetRow, DataStorage);

	return SNew(SBox)
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			[
				SNew(SImage)
					.ColorAndOpacity(FSlateColor::UseForeground())
					.Image(Binder.BindColumnsPresence(Queries::TColumn<FTedsPackageDirtyTag>(), [](bool bIsUnsaved)
						{
							// Since the widget maps to the FTedsPrimaryPackageObjectTag, it means that the row is mapped to a package. Meaning that
							// the presence of FTedsPackageDirtyTag just lets us know to use the IsDirty icon vs the IsSavable icon
							if (bIsUnsaved)
							{
								return FAppStyle::GetBrush("Icons.DirtyBadge");
							}
							else
							{
								return FAppStyle::GetBrush("Icons.SaveableBadge");
							}
						}))
			];
}

TArray<TSharedPtr<const UE::Editor::DataStorage::FColumnSorterInterface>> FOutlinerUnsavedWidgetConstructor::ConstructColumnSorters(
	UE::Editor::DataStorage::ICoreProvider* DataStorage,
	UE::Editor::DataStorage::IUiProvider* DataStorageUi,
	const UE::Editor::DataStorage::FMetaDataView& Arguments)
{
	using namespace UE::Editor::DataStorage;
	return TArray<TSharedPtr<const FColumnSorterInterface>>(
		{
			MakeShared<FUnsavedWidgetSorter>()
		});
}

FText FOutlinerUnsavedWidgetConstructor::CreateWidgetDisplayNameText(UE::Editor::DataStorage::ICoreProvider* DataStorage,
	UE::Editor::DataStorage::RowHandle Row) const
{
	return LOCTEXT("OutlinerUnsavedWidgetDisplayName", "Unsaved");
}

#undef LOCTEXT_NAMESPACE
