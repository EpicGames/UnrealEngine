// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/OutlinerVisibilityWidget.h"

#include "Columns/SceneOutlinerColumns.h"
#include "Columns/SlateHeaderColumns.h"
#include "Columns/TedsOutlinerColumns.h"
#include "Compatibility/SceneOutlinerTedsBridge.h"
#include "CoreGlobals.h" // GUndo
#include "DataStorage/Features.h"
#include "Elements/Columns/TypedElementCompatibilityColumns.h"
#include "Elements/Columns/TypedElementHiearchyColumns.h"
#include "Elements/Columns/TypedElementMiscColumns.h"
#include "Elements/Columns/TypedElementSelectionColumns.h"
#include "Elements/Columns/TypedElementVisibilityColumns.h"
#include "Elements/Framework/TypedElementQueryBuilder.h"
#include "Elements/Interfaces/TypedElementDataStorageCompatibilityInterface.h"
#include "Elements/Interfaces/TypedElementDataStorageInterface.h"
#include "Misc/ITransaction.h"
#include "Widgets/SBoxPanel.h"
#include "ISceneOutliner.h"
#include "ActorTreeItem.h"
#include "TedsOutlinerHelpers.h"
#include "TedsOutlinerItem.h"
#include "TedsTableViewerUtils.h"
#include "TedsTableViewerWidgetColumns.h"
#include "ActorFolders/ActorFolderColumns.h"
#include "Factories/TedsEditorHierarchyFactory.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(OutlinerVisibilityWidget)

#define LOCTEXT_NAMESPACE "OutlinerVisibilityWidget"

//
// Cell Factory
//

FOutlinerVisibilityWidgetConstructor::FOutlinerVisibilityWidgetConstructor()
	: FSimpleWidgetConstructor(StaticStruct())
{
}

void UOutlinerVisibilityWidgetFactory::RegisterWidgetConstructors(UE::Editor::DataStorage::ICoreProvider& DataStorage,
	UE::Editor::DataStorage::IUiProvider& DataStorageUi) const
{
	using namespace UE::Editor::DataStorage::Queries;
	DataStorageUi.RegisterWidgetFactory<FOutlinerVisibilityWidgetConstructor>(
		DataStorageUi.FindPurpose(IUiProvider::FPurposeInfo("SceneOutliner", "Cell", NAME_None).GeneratePurposeID()),
		TColumn<FVisibleInEditorColumn>());
}

TSharedPtr<SWidget> FOutlinerVisibilityWidgetConstructor::CreateWidget(UE::Editor::DataStorage::ICoreProvider* DataStorage,
	UE::Editor::DataStorage::IUiProvider* DataStorageUi,
	UE::Editor::DataStorage::RowHandle TargetRow, UE::Editor::DataStorage::RowHandle WidgetRow, const UE::Editor::DataStorage::FMetaDataView& Arguments)
{
	return SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		[
			SNew(STedsOutlinerVisibilityWidget, TargetRow, WidgetRow)
				.ToolTipText(LOCTEXT("SceneOutlinerVisibilityToggleTooltip", "Toggles the visibility of this object in the level editor."))
		];
}

//
// Header Factory
//

FOutlinerVisibilityHeaderConstructor::FOutlinerVisibilityHeaderConstructor()
	: FSimpleWidgetConstructor(StaticStruct())
{
}

void UOutlinerVisibilityHeaderFactory::RegisterWidgetConstructors(UE::Editor::DataStorage::ICoreProvider& DataStorage,
	UE::Editor::DataStorage::IUiProvider& DataStorageUi) const
{
	using namespace UE::Editor::DataStorage::Queries;
	DataStorageUi.RegisterWidgetFactory<FOutlinerVisibilityHeaderConstructor>(
		DataStorageUi.FindPurpose(IUiProvider::FPurposeInfo("SceneOutliner", "Header", NAME_None).GeneratePurposeID()), TColumn<FVisibleInEditorColumn>());
}

TSharedPtr<SWidget> FOutlinerVisibilityHeaderConstructor::CreateWidget(UE::Editor::DataStorage::ICoreProvider* DataStorage,
	UE::Editor::DataStorage::IUiProvider* DataStorageUi,
	UE::Editor::DataStorage::RowHandle TargetRow, UE::Editor::DataStorage::RowHandle WidgetRow, const UE::Editor::DataStorage::FMetaDataView& Arguments)
{
	DataStorage->AddColumn(WidgetRow, FHeaderWidgetSizeColumn
	{
		.ColumnSizeMode = EColumnSizeMode::Fixed,
		.Width = 24.0f
	});

	return SNew(SImage)
		.DesiredSizeOverride(FVector2D(16.f, 16.f))
		.Image(FAppStyle::Get().GetBrush(TEXT("Level.VisibleHighlightIcon16x")))
		.ToolTipText(LOCTEXT("SceneOutlinerVisibilityHeaderTooltip", "Visibility"));
}

//
// SVisibilityWidget
//

class FTEDSVisibilityDragDropOp : public FDragDropOperation
{
public:

	DRAG_DROP_OPERATOR_TYPE(FTEDSVisibilityDragDropOp, FDragDropOperation)

	/** Flag which defines whether to hide destination items or not */
	bool bHidden;

	/** Undo transaction stolen from the gutter which is kept alive for the duration of the drag */
	TUniquePtr<FScopedTransaction> UndoTransaction;

	/** The widget decorator to use */
	virtual TSharedPtr<SWidget> GetDefaultDecorator() const override
	{
		return SNullWidget::NullWidget;
	}

	/** Create a new drag and drop operation out of the specified flag */
	static TSharedRef<FTEDSVisibilityDragDropOp> New(const bool _bHidden, TUniquePtr<FScopedTransaction>& ScopedTransaction)
	{
		TSharedRef<FTEDSVisibilityDragDropOp> Operation = MakeShareable(new FTEDSVisibilityDragDropOp);

		Operation->bHidden = _bHidden;
		Operation->UndoTransaction = MoveTemp(ScopedTransaction);

		Operation->Construct();
		return Operation;
	}
};

void STedsOutlinerVisibilityWidget::Construct(const FArguments& InArgs, const RowHandle& InTargetRow, const RowHandle& InWidgetRow)
{
	using namespace UE::Editor::DataStorage;
	
	TargetRow = InTargetRow;
	WidgetRow = InWidgetRow;

	SImage::Construct(
		SImage::FArguments()
		.IsEnabled(this, &STedsOutlinerVisibilityWidget::IsEnabled)
		.ColorAndOpacity(this, &STedsOutlinerVisibilityWidget::GetForegroundColor)
		.Image(this, &STedsOutlinerVisibilityWidget::GetBrush)
	);
	
	if (const ICoreProvider* DataStorage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName))
	{
		const FHierarchyHandle Hierarchy = DataStorage->FindHierarchyByName(UTedsEditorHierarchyFactory::EditorObjectHierarchyName);
		EditorObjectHierarchyParentTag = DataStorage->GetParentTagType(Hierarchy);
	}

	static const FName NAME_VisibleHoveredBrush = TEXT("Level.VisibleHighlightIcon16x");
	static const FName NAME_VisibleNotHoveredBrush = TEXT("Level.VisibleIcon16x");
	static const FName NAME_NotVisibleHoveredBrush = TEXT("Level.NotVisibleHighlightIcon16x");
	static const FName NAME_NotVisibleNotHoveredBrush = TEXT("Level.NotVisibleIcon16x");

	VisibleHoveredBrush = FAppStyle::Get().GetBrush(NAME_VisibleHoveredBrush);
	VisibleNotHoveredBrush = FAppStyle::Get().GetBrush(NAME_VisibleNotHoveredBrush);

	NotVisibleHoveredBrush = FAppStyle::Get().GetBrush(NAME_NotVisibleHoveredBrush);
	NotVisibleNotHoveredBrush = FAppStyle::Get().GetBrush(NAME_NotVisibleNotHoveredBrush);

}

bool STedsOutlinerVisibilityWidget::IsEnabled() const
{
	using namespace UE::Editor::DataStorage;
 
	if (const ICoreProvider* DataStorage = GetDataStorage(); DataStorage && EditorObjectHierarchyParentTag.IsValid())
	{
		// If a folder is empty, disable changing its visibility from hidden
		if (DataStorage->HasColumns<FFolderCompatibilityColumn>(TargetRow) &&
			!DataStorage->HasColumns(TargetRow, TConstArrayView<TWeakObjectPtr<const UScriptStruct>>({EditorObjectHierarchyParentTag})))
		{
			return false;
		}
	}
 
	return true;
}

bool STedsOutlinerVisibilityWidget::CanCommitVisibility(UE::Editor::DataStorage::ICoreProvider& DataStorage, RowHandle Row) const
{
	if (EditorObjectHierarchyParentTag.IsValid())
	{
		// If a folder is empty, disable changing its visibility from hidden
		if (DataStorage.HasColumns<FFolderCompatibilityColumn>(Row) &&
			!DataStorage.HasColumns(Row, TConstArrayView<TWeakObjectPtr<const UScriptStruct>>({EditorObjectHierarchyParentTag})))
		{
			return false;
		}
	}
	return true;
}

FReply STedsOutlinerVisibilityWidget::OnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (MouseEvent.IsMouseButtonDown(EKeys::LeftMouseButton))
	{
		return FReply::Handled().BeginDragDrop(FTEDSVisibilityDragDropOp::New(!IsVisible(), UndoTransaction));
	}
	else
	{
		return FReply::Unhandled();
	}
}

/** If a visibility drag drop operation has entered this widget, set its item to the new visibility state */
void STedsOutlinerVisibilityWidget::OnDragEnter(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	auto VisibilityOp = DragDropEvent.GetOperationAs<FTEDSVisibilityDragDropOp>();
	if (VisibilityOp.IsValid())
	{
		SetIsVisible(TargetRow, !VisibilityOp->bHidden);
	}
}

FReply STedsOutlinerVisibilityWidget::HandleClick()
{
	using namespace UE::Editor::DataStorage;

	if (!IsEnabled())
	{
		return FReply::Unhandled();
	}

	// Open an undo transaction
	UndoTransaction.Reset(new FScopedTransaction(LOCTEXT("SetOutlinerItemVisibility", "Set Item Visibility")));

	const bool bVisible = !IsVisible();

	TArray<RowHandle> SelectedRows;
	GetSelectedRows(SelectedRows);

	FRowHandleArray ChangedRows;
	if (SelectedRows.Contains(TargetRow))
	{
		for (RowHandle Row : SelectedRows)
		{
			SetIsVisible(Row, bVisible, &ChangedRows);
		}
	}
	else
	{
		SetIsVisible(TargetRow, bVisible, &ChangedRows);
	}

	// Store undo change for the changed rows
	if (GUndo && !ChangedRows.IsEmpty())
	{
		// We need some UObject to use for the transaction. We'll grab the world from one of the rows
		UWorld* WorldToUse = nullptr;
		ICoreProvider* DataStorage = GetDataStorage();
		if (ensure(DataStorage))
		{
			for (RowHandle Row : ChangedRows.GetRows())
			{
				if (const FTypedElementWorldColumn* WorldColumn = DataStorage->GetColumn<FTypedElementWorldColumn>(Row))
				{
					if (UWorld* World = WorldColumn->World.Get())
					{
						WorldToUse = World;
						break;
					}
				}
			}
		}
		
		if (ensure(WorldToUse))
		{
			GUndo->StoreUndo(WorldToUse, MakeUnique<FVisibilityColumnChange>(MoveTemp(ChangedRows), bVisible));
		}
	}

	return FReply::Handled().DetectDrag(SharedThis(this), EKeys::LeftMouseButton);
}

FReply STedsOutlinerVisibilityWidget::OnMouseButtonDoubleClick(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent)
{
	return HandleClick();
}

/** Called when the mouse button is pressed down on this widget */
FReply STedsOutlinerVisibilityWidget::OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (MouseEvent.GetEffectingButton() != EKeys::LeftMouseButton)
	{
		return FReply::Unhandled();
	}

	return HandleClick();
}

/** Process a mouse up message */
FReply STedsOutlinerVisibilityWidget::OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton)
	{
		UndoTransaction.Reset();
		return FReply::Handled();
	}

	return FReply::Unhandled();
}

/** Called when this widget had captured the mouse, but that capture has been revoked for some reason. */
void STedsOutlinerVisibilityWidget::OnMouseCaptureLost(const FCaptureLostEvent& CaptureLostEvent)
{
	UndoTransaction.Reset();
}

/** Get the brush for this widget */
const FSlateBrush* STedsOutlinerVisibilityWidget::GetBrush() const
{
	if (IsVisible())
	{
		return IsHovered() ? VisibleHoveredBrush : VisibleNotHoveredBrush;
	}
	else
	{
		return IsHovered() ? NotVisibleHoveredBrush : NotVisibleNotHoveredBrush;
	}
}

FSlateColor STedsOutlinerVisibilityWidget::GetForegroundColor() const
{
	const bool bIsSelected = IsSelected();
	const bool bIsVisible = IsVisible();
	const bool bIsHovered = IsHovered();

	// make the foreground brush transparent if it is not selected and it is visible
	if (bIsVisible && !bIsHovered && !bIsSelected)
	{
		return FLinearColor::Transparent;
	}
	else if (bIsHovered && !bIsSelected)
	{
		return FAppStyle::Get().GetSlateColor("Colors.ForegroundHover");
	}

	return FSlateColor::UseForeground();
}

/** Check if our wrapped tree item is visible */
bool STedsOutlinerVisibilityWidget::IsVisible() const
{
	if (UE::Editor::DataStorage::ICoreProvider* DataStorage = GetDataStorage())
	{
		FVisibleInEditorColumn* HiddenInEditorColumn = DataStorage->GetColumn<FVisibleInEditorColumn>(TargetRow);
		if (HiddenInEditorColumn)
		{
			return HiddenInEditorColumn->bIsVisibleInEditor;
		}
	}

	return true;
}

bool STedsOutlinerVisibilityWidget::IsSelected() const
{
	if (UE::Editor::DataStorage::ICoreProvider* DataStorage = GetDataStorage())
	{
		return DataStorage->HasColumns<FTypedElementSelectionColumn>(TargetRow);
	}

	return false;
}

void STedsOutlinerVisibilityWidget::SetIsVisible(RowHandle InRow, const bool bVisible, UE::Editor::DataStorage::FRowHandleArray* ChangedRowsToAppendTo)
{
	using namespace UE::Editor::DataStorage;

	if (ICoreProvider* DataStorage = GetDataStorage())
	{
		// Use the hierarchy data if present
		const RowHandle ParentRow = TableViewerUtils::GetTableViewerUiRow(DataStorage, WidgetRow);
		
		if (FHierarchyViewerDataColumn* DataColumn = DataStorage->GetColumn<FHierarchyViewerDataColumn>(ParentRow))
		{
			if (TSharedPtr<IHierarchyViewerDataInterface> DataInterface = DataColumn->HierarchyData.Pin())
			{
				SetVisibility_Recursive(*DataStorage, InRow, DataInterface, bVisible, ChangedRowsToAppendTo);
			}
		}
		// The Teds Outliner doesn't use the new hierarchy data yet so we have a fallback to the tree item API
		else if (TSharedPtr<ISceneOutlinerTreeItem> RowTreeItem = GetTreeItem(InRow))
		{
			SetVisibility_Recursive(*DataStorage, RowTreeItem, bVisible, ChangedRowsToAppendTo);
		}
		
	}
}

void STedsOutlinerVisibilityWidget::CommitVisibility(UE::Editor::DataStorage::ICoreProvider& DataStorage, RowHandle Row, bool bVisible,
	UE::Editor::DataStorage::FRowHandleArray* ChangedRowsOut)
{
	if(!CanCommitVisibility(DataStorage, Row))
	{
		return;
	}
	
	// Set new visibility on the TargetRow
	FVisibleInEditorColumn* VisibileInEditorColumn = DataStorage.GetColumn<FVisibleInEditorColumn>(Row);
	if (VisibileInEditorColumn && VisibileInEditorColumn->bIsVisibleInEditor != bVisible)
	{
		VisibileInEditorColumn->bIsVisibleInEditor = bVisible;
		DataStorage.AddColumn<FVisibilityChangedTag>(Row);
		if (ChangedRowsOut)
		{
			ChangedRowsOut->Add(Row);
		}
	}
}

void STedsOutlinerVisibilityWidget::SetVisibility_Recursive(UE::Editor::DataStorage::ICoreProvider& DataStorage, RowHandle Row,
	TSharedPtr<UE::Editor::DataStorage::IHierarchyViewerDataInterface> HierarchyData, bool bVisible, 
	UE::Editor::DataStorage::FRowHandleArray* ChangedRowsOut)
{
	using namespace UE::Editor::DataStorage;

	FRowHandleArray Children;
	HierarchyData->WalkDepthFirst(DataStorage, Row, [&Children](const ICoreProvider& Context, RowHandle Owner, RowHandle Target)
	{
		Children.Add(Target);
	});

	for (RowHandle ChildRow : Children.GetRows())
	{
		CommitVisibility(DataStorage, ChildRow, bVisible, ChangedRowsOut);
	}
}

void STedsOutlinerVisibilityWidget::SetVisibility_Recursive(UE::Editor::DataStorage::ICoreProvider& DataStorage, FSceneOutlinerTreeItemPtr TreeItem, bool bVisible, 
	UE::Editor::DataStorage::FRowHandleArray* ChangedRowsOut)
{
	using namespace UE::Editor::DataStorage;
	using namespace UE::Editor::Outliner;

	if (ICompatibilityProvider* DataStorageCompat = GetDataStorageCompatibility())
	{
		if (FActorTreeItem* ActorTreeItem = TreeItem->CastTo<FActorTreeItem>())
		{
			RowHandle ActorRow = DataStorageCompat->FindRowWithCompatibleObject(ActorTreeItem->Actor);
			CommitVisibility(DataStorage, ActorRow, bVisible, ChangedRowsOut);
		}
		else if (FTedsOutlinerTreeItem* TedsItem = TreeItem->CastTo<FTedsOutlinerTreeItem>())
		{
			CommitVisibility(DataStorage, TedsItem->GetRowHandle(), bVisible, ChangedRowsOut);
		}
	}

	for (auto& ChildTreeItemPtr : TreeItem->GetChildren())
	{
		auto ChildTreeItem = ChildTreeItemPtr.Pin();
		if (ChildTreeItem.IsValid())
		{
			SetVisibility_Recursive(DataStorage, ChildTreeItem, bVisible, ChangedRowsOut);
		}
	}
}

FSceneOutlinerTreeItemPtr STedsOutlinerVisibilityWidget::GetTreeItem(RowHandle InRow) const
{
	using namespace UE::Editor::DataStorage;
	ICoreProvider* DataStorage = GetDataStorage();
	if (!DataStorage)
	{
		return nullptr;
	}

	const FSceneOutlinerColumn* TEDSOutlinerColumn = DataStorage->GetColumn<FSceneOutlinerColumn>(WidgetRow);
	if (!TEDSOutlinerColumn)
	{
		return nullptr;
	}

	TSharedPtr<ISceneOutliner> Outliner = TEDSOutlinerColumn->Outliner.IsValid() ? TEDSOutlinerColumn->Outliner.Pin() : nullptr;
	if (!Outliner.IsValid())
	{
		return nullptr;
	}

	return UE::Editor::Outliner::Helpers::GetTreeItemFromRowHandle(DataStorage, Outliner.ToSharedRef(), InRow);
}

UE::Editor::DataStorage::ICoreProvider* STedsOutlinerVisibilityWidget::GetDataStorage()
{
	using namespace UE::Editor::DataStorage;
	return GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName);
}

UE::Editor::DataStorage::ICoreProvider* STedsOutlinerVisibilityWidget::GetDataStorageUI()
{
	using namespace UE::Editor::DataStorage;
	return GetMutableDataStorageFeature<ICoreProvider>(UiFeatureName);
}

UE::Editor::DataStorage::ICompatibilityProvider* STedsOutlinerVisibilityWidget::GetDataStorageCompatibility()
{
	using namespace UE::Editor::DataStorage;
	return GetMutableDataStorageFeature<ICompatibilityProvider>(CompatibilityFeatureName);
}

void STedsOutlinerVisibilityWidget::GetSelectedRows(TArray<RowHandle>& OutSelectedRows) const
{
	using namespace UE::Editor::DataStorage;
	using namespace UE::Editor::DataStorage::Queries;
	if (ICoreProvider* DataStorage = GetDataStorage())
	{
		if (const FTypedElementSelectionColumn* SelectionColumn = DataStorage->GetColumn<FTypedElementSelectionColumn>(TargetRow))
		{
			// We want all rows selected in the same set as the row this widget is displayed for
			FName SelectionSet = SelectionColumn->SelectionSet;

			static QueryHandle AllSelectedItemsQuery =
				DataStorage->RegisterQuery(
					Select()
						.ReadOnly<FTypedElementSelectionColumn>()
					.Where()
					.Compile());
			
			DataStorage->RunQuery(AllSelectedItemsQuery,
				CreateDirectQueryCallbackBinding([&OutSelectedRows, SelectionSet](const IDirectQueryContext& Context, RowHandle Row, const FTypedElementSelectionColumn& SelectionColumn)
				{
					if (SelectionColumn.SelectionSet == SelectionSet)
					{
						OutSelectedRows.Add(Row);
					}
				}));
		}
	}
}

#undef LOCTEXT_NAMESPACE
