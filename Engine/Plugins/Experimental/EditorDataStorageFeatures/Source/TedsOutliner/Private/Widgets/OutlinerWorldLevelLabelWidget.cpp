// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/OutlinerWorldLevelLabelWidget.h"

#include "Elements/Columns/TypedElementCompatibilityColumns.h"
#include "Elements/Columns/TypedElementLabelColumns.h"
#include "Elements/Interfaces/TypedElementDataStorageUiInterface.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(OutlinerWorldLevelLabelWidget)

void UOutlinerWorldLevelLabelWidgetFactory::RegisterWidgetConstructors(
	UE::Editor::DataStorage::ICoreProvider& DataStorage,
	UE::Editor::DataStorage::IUiProvider& DataStorageUi) const
{
	using namespace UE::Editor::DataStorage;
	using namespace UE::Editor::DataStorage::Queries;

	const RowHandle LabelPurpose = DataStorageUi.FindPurpose(
		IUiProvider::FPurposeInfo("SceneOutliner", "RowLabel", "Text").GeneratePurposeID());

	DataStorageUi.RegisterWidgetFactory<FOutlinerWorldLevelLabelWidgetConstructor>(
		LabelPurpose,
		TColumn<FTypedElementLabelColumn>() && (TColumn<FWorldTag>() || TColumn<FLevelTag>()));
}

FOutlinerWorldLevelLabelWidgetConstructor::FOutlinerWorldLevelLabelWidgetConstructor()
	: Super(FOutlinerWorldLevelLabelWidgetConstructor::StaticStruct())
{
}

FOutlinerWorldLevelLabelWidgetConstructor::FOutlinerWorldLevelLabelWidgetConstructor(const UScriptStruct* InTypeInfo)
	: Super(InTypeInfo)
{
}

TSharedPtr<SWidget> FOutlinerWorldLevelLabelWidgetConstructor::CreateEditableWidget(
	UE::Editor::DataStorage::ICoreProvider* DataStorage,
	UE::Editor::DataStorage::IUiProvider* DataStorageUi,
	UE::Editor::DataStorage::RowHandle TargetRow,
	UE::Editor::DataStorage::RowHandle WidgetRow,
	const UE::Editor::DataStorage::FMetaDataView& Arguments)
{
	return CreateNonEditableWidget(DataStorage, DataStorageUi, TargetRow, WidgetRow, Arguments);
}
