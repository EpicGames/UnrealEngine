// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SceneOutlinerWidget.h"

#include "EditorActorFolders.h"
#include "ActorDesc/TedsActorDescColumns.h"
#include "ActorFolders/ActorFolderColumns.h"
#include "Columns/SceneOutlinerColumns.h"
#include "Columns/TedsLevelInstanceColumns.h"
#include "Columns/TedsOutlinerColumns.h"
#include "DataStorage/Features.h"
#include "Elements/Interfaces/TypedElementDataStorageCompatibilityInterface.h"
#include "Elements/Columns/TypedElementUIColumns.h"
#include "Elements/Columns/TypedElementFolderColumns.h"
#include "ILevelEditor.h"
#include "ISourceControlModule.h"
#include "LevelEditor.h"
#include "LevelEditorContextMenu.h"
#include "LevelEditorMenuContext.h"
#include "SceneOutlinerFilters.h"
#include "SSceneOutliner.h"
#include "TedsOutlinerContextMenuHelpers.h"
#include "Modules/ModuleManager.h"
#include "TedsOutlinerFolderHelpers.h"
#include "TedsOutlinerHelpers.h"
#include "TedsOutlinerItem.h"
#include "TedsOutlinerModule.h"
#include "TedsOutlinerMode.h"
#include "TedsOutlinerModeSettings.h"
#include "TedsOutlinerImpl.h"
#include "Columns/TedsActorComponentCompatibilityColumns.h"
#include "Columns/TedsActorWorldPartitionColumns.h"
#include "EditorActorFolders.h"
#include "Editor.h"
#include "SceneOutlinerModule.h"
#include "Engine/GameViewportClient.h"
#include "ToolMenus.h"
#include "Elements/Columns/TypedElementCompatibilityColumns.h"
#include "Elements/Columns/TypedElementLabelColumns.h"
#include "Elements/Columns/TypedElementMiscColumns.h"
#include "Elements/Columns/TedsRevisionControlColumns.h"
#include "Elements/Columns/TypedElementPackageColumns.h"
#include "Elements/Columns/TypedElementSelectionColumns.h"
#include "Elements/Framework/TypedElementSelectionSet.h"
#include "Factories/TedsEditorHierarchyFactory.h"
#include "Framework/Application/SlateApplication.h"
#include "LevelInstance/ILevelInstanceEditorModule.h"
#include "LevelInstance/LevelInstanceInterface.h"
#include "LevelInstance/LevelInstanceSubsystem.h"
#include "WorldTreeItem.h"
#include "Framework/Application/SlateApplication.h"
#include "ScopedTransaction.h"
#include "TedsAlertColumns.h"
#include "Subsystems/ActorEditorContextSubsystem.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/SBoxPanel.h"
#include "WorldPartition/ActorDescContainerInstance.h"
#include "WorldPartition/IWorldPartitionEditorModule.h"
#include "WorldPartition/DataLayer/DataLayerInstance.h"
#include "WorldPartition/DataLayer/DataLayerManager.h"
#include "WorldPartition/DataLayer/WorldDataLayers.h"

#define LOCTEXT_NAMESPACE "SceneOutlinerWidget"

static FName LevelEditorModuleName = "LevelEditor";
static FName SceneOutlinerModuleName = "SceneOutliner";
static FName TedsOutlinerModuleName = "TEDSOutliner";
static FName DoubleClickTogglesCurrentFolderOption = "FolderDoubleClickTogglesCurrentFolder";
static FName UpdateInPIEOption = "UpdateInPIE";

void UTedsOutlinerWidgetFactory::RegisterExternalFilterProvider(FName ProviderName, FExternalFilterProvider Provider)
{
	ExternalFilterProviders.Add(ProviderName, MoveTemp(Provider));
}

void UTedsOutlinerWidgetFactory::UnregisterExternalFilterProvider(FName ProviderName)
{
	ExternalFilterProviders.Remove(ProviderName);
}

void UTedsOutlinerWidgetFactory::CallExternalFilterProviders(TArray<TSharedPtr<UE::Editor::Outliner::FTedsOutlinerFilter>>& Filters, UE::Editor::DataStorage::ICoreProvider* DataStorage) const
{
	for (const TPair<FName, FExternalFilterProvider>& Provider : ExternalFilterProviders)
	{
		Provider.Value(Filters, DataStorage);
	}
}

static void ToggleAndSaveOption(
	UE::Editor::DataStorage::ICoreProvider* Storage, const UE::Editor::DataStorage::RowHandle OutlinerRow,
	const FName OutlinerId, const FName OptionKey)
{
	if (FTedsSceneOutlinerSettingsCacheColumn* SettingsCache = Storage->GetColumn<FTedsSceneOutlinerSettingsCacheColumn>(OutlinerRow))
	{
		bool& bOptionValue = SettingsCache->CachedSettings.FindOrAdd(OptionKey);
		bOptionValue = !bOptionValue;
		if (UTedsOutlinerConfig* Config = UTedsOutlinerConfig::Get())
		{
			Config->TedsOutliners.FindOrAdd(OutlinerId).OptionSettingsActive.FindOrAdd(OptionKey) = bOptionValue;
			Config->SaveEditorConfig();
		}
	}
}

static bool bShowTedsOutlinerRowHandles = false;
// Cvar to add a column to show row handles for items in Teds Outliners
static FAutoConsoleVariableRef CVarShowRowHandlesOutliner(
TEXT("TEDS.UI.ShowTedsOutlinerRowHandles"),
bShowTedsOutlinerRowHandles,
TEXT("Add a column to show row handles in the TEDS Outliner (requires TEDS.UI.SetOutlinerPurpose!=BaseOutliner)"),
FConsoleVariableDelegate::CreateLambda([](IConsoleVariable* /*CVar*/)
{
	UE::Editor::Outliner::Helpers::RefreshLevelEditorOutliners();
}));

UE::Editor::DataStorage::FQueryDescription static GetBaseOutlinerQuery()
{
	using namespace UE::Editor::DataStorage::Queries;
	// Show all rows with a label and world that are either a UObject, folder or unloaded actor
	return
		Select()
		.Where()
			.All<FTypedElementLabelColumn, FTypedElementWorldColumn>()
			.Any<FTypedElementUObjectColumn, FFolderCompatibilityColumn, FActorDescTag>()
		.Compile();
}

void UTedsOutlinerWidgetFactory::RegisterWidgetPurposes(UE::Editor::DataStorage::IUiProvider& DataStorageUi) const
{
	using namespace UE::Editor::DataStorage;

	const IUiProvider::FPurposeInfo TEDSOutlinerPurpose = IUiProvider::FPurposeInfo(
		LevelEditorModuleName, SceneOutlinerModuleName, "TedsOutliner",
		IUiProvider::EPurposeType::UniqueByName,
		LOCTEXT("TEDSOutlinerPurpose", "Widget used to generate the TEDS ISceneOutliner widget."),
		IUiProvider::FPurposeInfo(LevelEditorModuleName, SceneOutlinerModuleName, "ActorOutliner").GeneratePurposeID());

	DataStorageUi.RegisterWidgetPurpose(TEDSOutlinerPurpose);
}

void UTedsOutlinerWidgetFactory::RegisterWidgetConstructors(UE::Editor::DataStorage::ICoreProvider& DataStorage,
	UE::Editor::DataStorage::IUiProvider& DataStorageUi) const
{
	using namespace UE::Editor::DataStorage::Queries;
	
	DataStorageUi.RegisterWidgetFactory<FTedsSceneOutlinerWidgetConstructor>(
		DataStorageUi.FindPurpose(IUiProvider::FPurposeInfo(LevelEditorModuleName, SceneOutlinerModuleName, "TedsOutliner").GeneratePurposeID()));
}

FTedsSceneOutlinerWidgetConstructor::FTedsSceneOutlinerWidgetConstructor()
	: FSimpleWidgetConstructor(StaticStruct())
{
}

TSharedPtr<SWidget> FTedsSceneOutlinerWidgetConstructor::CreateWidget(UE::Editor::DataStorage::ICoreProvider* DataStorage,
	UE::Editor::DataStorage::IUiProvider* DataStorageUi,
	UE::Editor::DataStorage::RowHandle TargetRow, UE::Editor::DataStorage::RowHandle WidgetRow, const UE::Editor::DataStorage::FMetaDataView& Arguments)
{
	using namespace UE::Editor::DataStorage;
	using namespace UE::Editor::Outliner;

	// Fallback to the main SceneOutliner Tab Id
	TabIdentifier = LevelEditorTabIds::LevelEditorSceneOutliner;
	const FMetaDataEntryView OutlinerIdMeta = Arguments.FindGeneric("OutlinerIdentifier");
	if(const FString* const* IdMetaData = OutlinerIdMeta.TryGetExact<const FString*>())
	{
		TabIdentifier = FName(**IdMetaData);
	}

	WorldFilter = MakeShared<FTedsSceneOutlinerWorldFilter>(TEXT("WorldFilter"));

	FSceneOutlinerInitializationOptions InitOptions;
	GetSceneOutlinerInitializationOptions(InitOptions);

	TMap<FName, TSharedPtr<FTedsOutlinerFilter>> ShowOptionsFilters;
	GetShowOptionsFilters(DataStorage, ShowOptionsFilters);

	FTedsOutlinerParams Params(nullptr);
	Params.QueryDescription = GetQueryDescription();
	Params.ColumnDescription = GetColumnDescription();

	Helpers::ConvertLegacyFiltersToTedsFilters(InitOptions.FilterBarOptions, Params.Filters, GetLegacyFilterConversionMap(DataStorage), Params.FilterCategoryMap);
	GetFilters(DataStorage, Params.Filters);

	for (const TPair<FName, TSharedPtr<FTedsOutlinerFilter>>& Filter : ShowOptionsFilters)
	{
		Params.ShowOptionsFilters.Add(Filter.Value);
	}

	GetInitializeViewMenuExtender(Params);
	Params.bShowRowHandleColumn = ShowRowHandleColumn();
	Params.bForceShowParents = ForceShowParents();
	Params.bUseDefaultObservers = UseDefaultObservers();
	Params.bShowViewButton = ShowViewButton();
	Params.HierarchyData = GetHierarchyData(DataStorage);
	Params.SelectionSet = GetSelectionSet();

	GetOnItemDoubleClick(Params);
	GetOnCanPopulate(Params);
	GetInitializeOptionsMenuExtender(Params, DataStorage);
	GetExpansionStateBridge(Params);
	GetAdditionalObserverQueries(Params);
	GetOnCustomAddToToolbar(Params, DataStorage);

	GetOnRegisterInteractiveFilters(Params, DataStorage);

	const FTedsOutlinerModule& TedsOutlinerModule = FModuleManager::Get().LoadModuleChecked<FTedsOutlinerModule>(TedsOutlinerModuleName);
	TSharedRef<ISceneOutliner> Outliner = TedsOutlinerModule.CreateTedsOutliner(InitOptions, Params);
	DataStorage->AddColumn(WidgetRow, FSceneOutlinerColumn { .Outliner = Outliner });

	PostOutlinerCreated(DataStorage, DataStorage->LookupMappedRow(MappingDomain, FMapKey(TabIdentifier)));

	return Outliner;
}

void FTedsSceneOutlinerWidgetConstructor::PostOutlinerCreated(UE::Editor::DataStorage::ICoreProvider* DataStorage, UE::Editor::DataStorage::RowHandle OutlinerRow)
{
	using namespace UE::Editor::DataStorage;

	if (!DataStorage || !DataStorage->IsRowAvailable(OutlinerRow))
	{
		return;
	}

	FTedsSceneOutlinerSettingsCacheColumn InitSettingsColumn;
	UTedsOutlinerConfig::Initialize();
	if (UTedsOutlinerConfig* Config = UTedsOutlinerConfig::Get())
	{
		Config->LoadEditorConfig();
		if (const FTedsOutlinerModeConfig* ModeConfig = Config->TedsOutliners.Find(TabIdentifier))
		{
			InitSettingsColumn.CachedSettings = ModeConfig->OptionSettingsActive;
		}
	}

	if (InitSettingsColumn.CachedSettings.IsEmpty())
	{
		InitSettingsColumn.CachedSettings.Add(DoubleClickTogglesCurrentFolderOption, false);
		InitSettingsColumn.CachedSettings.Add(UpdateInPIEOption, true);
	}
	DataStorage->AddColumn(OutlinerRow, MoveTemp(InitSettingsColumn));

	// Refresh the Outliner when there is an update on the level instance edit mode so the interactivity of items is refreshed
	if (ILevelInstanceEditorModule* LevelInstanceEditorModule = FModuleManager::GetModulePtr<ILevelInstanceEditorModule>("LevelInstanceEditor"))
	{
		if (FSceneOutlinerColumn* OutlinerColumn = DataStorage->GetColumn<FSceneOutlinerColumn>(OutlinerRow))
		{
			if (TSharedPtr<ISceneOutliner> Outliner = OutlinerColumn->Outliner.Pin())
			{
				LevelInstanceEditorModule->OnLevelInstanceEditModeChanged().AddSPLambda(Outliner.Get(),
					[WeakOutliner = OutlinerColumn->Outliner](ILevelInstanceInterface*, ILevelInstanceEditorModule::ELevelInstanceEditMode)
					{
						if (TSharedPtr<ISceneOutliner> PinnedOutliner = WeakOutliner.Pin())
						{
							PinnedOutliner->FullRefresh();
						}
					});
			}
		}
		
	}
}

void FTedsSceneOutlinerWidgetConstructor::GetSceneOutlinerInitializationOptions(FSceneOutlinerInitializationOptions& InitOptions)
{
	GetLevelEditorSceneOutlinerInitOptions(InitOptions);
	
	InitOptions.FilterBarOptions.bUseSharedSettings = true;
	InitOptions.FilterBarOptions.bHasFilterBar = true;
	InitOptions.bShowTransient = true;

	const FSceneOutlinerModule& SceneOutlinerModule = FModuleManager::LoadModuleChecked<FSceneOutlinerModule>("SceneOutliner");
	SceneOutlinerModule.CreateActorBrowserColumns(InitOptions);

	// Similar to what is done in SLevelEditor::CreateSceneOutliner, bind a callback that adjusts 
	//  the context menu to have all the level editor context menu sections.
	TWeakPtr<ILevelEditor> WeakLevelEditor;
	if (const FLevelEditorModule* const LevelEditorModule = FModuleManager::GetModulePtr<FLevelEditorModule>(LevelEditorModuleName))
	{
		WeakLevelEditor = LevelEditorModule->GetLevelEditorInstance();
	}
	InitOptions.ModifyContextMenu.BindLambda([WeakWorldFilter = WorldFilter.ToWeakPtr(), WeakLevelEditor](FName& OutMenuName, FToolMenuContext& MenuContext)
	{
		static const FName ContextMenuName = "LevelEditor.TedsOutliner.ContextMenu";
		UToolMenus* ToolMenus = UToolMenus::Get();
		const bool bFirstRegistration = ToolMenus && !ToolMenus->IsMenuRegistered(ContextMenuName);

		FLevelEditorContextMenu::RegisterAsDerivedMenu(ContextMenuName, OutMenuName);
		OutMenuName = ContextMenuName;

		if (bFirstRegistration)
		{
			if (UToolMenu* DerivedMenu = ToolMenus->FindMenu(ContextMenuName))
			{
				DerivedMenu->AddDynamicSection(
					"TedsOutlinerSelectionSection",
					FNewToolMenuDelegate::CreateLambda([WeakWorldFilter](UToolMenu* InMenu)
					{
						TWeakObjectPtr<UWorld> World;
						if (const TSharedPtr<UE::Editor::Outliner::FTedsSceneOutlinerWorldFilter> WorldFilterPinned = WeakWorldFilter.Pin())
						{
							World = WorldFilterPinned->GetRepresentingWorld();
						}

						UE::Editor::Outliner::Helpers::BuildTedsOutlinerContextMenu(InMenu, World);
					}));
			}
		}

		if (WeakLevelEditor.IsValid())
		{
			FLevelEditorContextMenu::InitMenuContext(MenuContext, WeakLevelEditor, ELevelEditorMenuContext::SceneOutliner);
		}
	});
}

UE::Editor::DataStorage::FQueryDescription FTedsSceneOutlinerWidgetConstructor::GetQueryDescription() const
{
	using namespace UE::Editor::DataStorage::Queries;
	
	FQueryDescription QueryDescription = GetBaseOutlinerQuery();
	
	// This isn't a part of the base query description since that is used for observers as well, and if we did we would end up with an observer for
	// OnAdd<FHideRowFromUITag> with a condition .None<FHideRowFromUITag>() which would never trigger
	QueryDescription.ConditionTypes.Add(FQueryDescription::EOperatorType::SimpleNone);
	QueryDescription.ConditionOperators.AddZeroed_GetRef().Type = FHideRowFromUITag::StaticStruct();
	
	return QueryDescription;
}

UE::Editor::Outliner::FTedsOutlinerColumnDescription FTedsSceneOutlinerWidgetConstructor::GetColumnDescription() const
{
	using namespace UE::Editor::Outliner;
	
	FTedsOutlinerModule& TedsOutlinerModule = FModuleManager::Get().LoadModuleChecked<FTedsOutlinerModule>(TedsOutlinerModuleName);
	UE::Editor::Outliner::FTedsOutlinerColumnDescription ColumnDescription = TedsOutlinerModule.GetLevelEditorTedsOutlinerColumnDescription();
	if(WorldFilter)
	{
		if(const TWeakObjectPtr<UWorld> World = WorldFilter->GetRepresentingWorld(); World.IsValid())
		{
			if (const ISourceControlModule& SourceControlModule = ISourceControlModule::Get(); SourceControlModule.IsEnabled())
			{
				if (!World->PersistentLevel->IsUsingExternalActors())
				{
					FTedsOutlinerColumnParams& SourceControlColumn = ColumnDescription.FindOrAddColumnParams(FTypedElementPackageReference::StaticStruct());
					SourceControlColumn.InitialVisibility = ESceneOutlinerColumnVisibility::Invisible;
				}
			}
			if (!World->IsPartitionedWorld())
			{
				FTedsOutlinerColumnParams& UnsavedColumn = ColumnDescription.FindOrAddColumnParams(FTedsPrimaryPackageObjectTag::StaticStruct());
				UnsavedColumn.InitialVisibility = ESceneOutlinerColumnVisibility::Invisible;
			}
		}
	}
	
	return ColumnDescription;
}

void FTedsSceneOutlinerWidgetConstructor::GetFilters(UE::Editor::DataStorage::ICoreProvider* DataStorage, TArray<TSharedPtr<UE::Editor::Outliner::FTedsOutlinerFilter>>& Filters) const
{
	using namespace UE::Editor::Outliner;
	
	if(WorldFilter)
	{
		Filters.Add(WorldFilter);
	}

	if (DataStorage)
	{
		if (const UTedsOutlinerWidgetFactory* Factory = DataStorage->FindFactory<UTedsOutlinerWidgetFactory>())
		{
			Factory->CallExternalFilterProviders(Filters, DataStorage);
		}
	}
	
	AddFavoriteFilters(DataStorage, Filters);
	AddAlertFilters(DataStorage, Filters);
	AddSCCFilters(DataStorage, Filters);
}

void FTedsSceneOutlinerWidgetConstructor::GetShowOptionsFilters(UE::Editor::DataStorage::ICoreProvider* DataStorage, TMap<FName, TSharedPtr<UE::Editor::Outliner::FTedsOutlinerFilter>>& Filters) const
{
	using namespace UE::Editor::Outliner;

	auto IsLevelALevelInstance = [](const FTypedElementLevelColumn& LevelColumn, const FTypedElementWorldColumn& WorldColumn)
	{
		if (const ULevel* Level = LevelColumn.Level.Get())
		{
			if (const UWorld* World = WorldColumn.World.Get())
			{
				if (const ULevelInstanceSubsystem* LevelInstanceSubsystem = World->GetSubsystem<ULevelInstanceSubsystem>())
				{
					const ILevelInstanceInterface* LevelInstance = LevelInstanceSubsystem->GetOwningLevelInstance(Level);
					return LevelInstance != nullptr;
				}
			}
		}
		
		return false;
	};
	
	// Hide Temporary Actors
	const FName HideTemporaryName = TEXT("HideTemporary");
	Filters.Add(HideTemporaryName, MakeShared<FTedsOutlinerFilter>(
		HideTemporaryName,
		LOCTEXT("ToggleHideTemporaryObjects", "Hide Temporary Objects"),
			LOCTEXT("ToggleHideTemporaryObjectsToolTip", "When enabled, hides temporary/run-time objects."),
		NAME_None,
		nullptr,
		[DataStorage, IsLevelALevelInstance](TConstQueryContext<SingleRowInfo, CurrentTableInfo> Context, const FTypedElementWorldColumn& WorldColumn)
	 	{
			if (DataStorage)
			{
				if (const FTypedElementUObjectColumn* ObjectColumn = DataStorage->GetColumn<FTypedElementUObjectColumn>(Context.GetCurrentRow()))
				{
					if (const UObject* Object = ObjectColumn->Object.Get())
					{
						if (Object->HasAnyFlags(RF_Transient))
						{
							if (Context.CurrentTableHasColumns<FPieObjectTag>())
							{
								return true;
							}

							if (const FTypedElementLevelColumn* LevelColumn = DataStorage->GetColumn<FTypedElementLevelColumn>(Context.GetCurrentRow()))
							{
								return IsLevelALevelInstance(*LevelColumn, WorldColumn);
							}
							
							return false;
						}
					}
				}
			}
			return true;
	 	}));

	// Hide Level Instance Content
	const FName HideLevelInstanceContentName = TEXT("HideLevelInstanceContent");
	Filters.Add(HideLevelInstanceContentName, MakeShared<FTedsOutlinerFilter>(
		HideLevelInstanceContentName,
		LOCTEXT("ToggleHideLevelInstanceContent", "Hide Level Instance Content"),
		LOCTEXT("ToggleHideLevelInstancesToolTip", "When enabled, hides all level instance content."),
		NAME_None,
		nullptr,
		[DataStorage, IsLevelALevelInstance](TConstQueryContext<SingleRowInfo, CurrentTableInfo> Context, const FTypedElementWorldColumn& WorldColumn)
		 {
			if (DataStorage)
			{
				if (const FTypedElementLevelColumn* LevelColumn = DataStorage->GetColumn<FTypedElementLevelColumn>(Context.GetCurrentRow()))
				{
					return !IsLevelALevelInstance(*LevelColumn, WorldColumn);
				}
				if (const FFolderCompatibilityColumn* FolderColumn = DataStorage->GetColumn<FFolderCompatibilityColumn>(Context.GetCurrentRow()))
				{
					const ILevelInstanceInterface* LevelInstance = Cast<ILevelInstanceInterface>(FolderColumn->Folder.GetRootObjectPtr());
					return LevelInstance == nullptr;
				}
			}

			return true;
		 }));

	// Only Selected
	const FName ShowOnlySelectedName = TEXT("ShowOnlySelected");
	Filters.Add(ShowOnlySelectedName, MakeShared<FTedsOutlinerFilter>(
		ShowOnlySelectedName,
		LOCTEXT("ToggleShowOnlySelected", "Only Selected"),
		LOCTEXT("ToggleShowOnlySelectedToolTip", "When enabled, only displays objects that are currently selected."),
		NAME_None,
		nullptr,
		DataStorage->RegisterQuery(
			Select()
			.Where()
			.All<FTypedElementSelectionColumn>()
			.Compile())
		));

	// Only in Current Level
	const FName ShowOnlyCurrentLevelName = TEXT("ShowOnlyCurrentLevel");
	Filters.Add(ShowOnlyCurrentLevelName, MakeShared<FTedsOutlinerFilter>(
		ShowOnlyCurrentLevelName,
		LOCTEXT("ToggleShowOnlyCurrentLevel", "Only in Current Level"),
		LOCTEXT("ToggleShowOnlyCurrentLevelToolTip", "When enabled, only shows objects that are in the Current Level."),
		NAME_None,
		nullptr,
		[DataStorage](TConstQueryContext<SingleRowInfo, CurrentTableInfo> Context)
		 {
			if (DataStorage)
			{
				if (Context.CurrentTableHasColumns<FEditorDataStorageLevelTag>())
				{
					if (const FTypedElementUObjectColumn* ObjectColumn = DataStorage->GetColumn<FTypedElementUObjectColumn>(Context.GetCurrentRow()))
					{
						if (UObject* Object = ObjectColumn->Object.Get())
						{
							if (const ULevel* Level = Cast<ULevel>(Object))
							{
								return Level->IsCurrentLevel();
							}
						}
					}
				}
				if (const FTypedElementLevelColumn* LevelColumn = DataStorage->GetColumn<FTypedElementLevelColumn>(Context.GetCurrentRow()))
				{
					if (const ULevel* Level = LevelColumn->Level.Get())
					{
						return Level->IsCurrentLevel();
					}
				}
			}

			return false;
		}));

	// Only in any Current Data Layers
	const FName ShowOnlyInCurrentDataLayerName = TEXT("ShowOnlyInCurrentDataLayer");
	Filters.Add(ShowOnlyInCurrentDataLayerName, MakeShared<FTedsSceneOutlinerCurrentDataLayerFilter>(ShowOnlyInCurrentDataLayerName));

	// Only in Current Content Bundle
	const FName ShowOnlyInCurrentContentBundleName = TEXT("ShowOnlyInCurrentContentBundle");
	Filters.Add(ShowOnlyInCurrentContentBundleName, MakeShared<FTedsSceneOutlinerCurrentContentBundleFilter>(ShowOnlyInCurrentContentBundleName));

	// Hide Actor Components
	const FName HideComponentsFilterName = TEXT("HideComponentsFilter");
	Filters.Add(HideComponentsFilterName, MakeShared<FTedsOutlinerFilter>(
		HideComponentsFilterName,
		LOCTEXT("ToggleHideActorComponents", "Hide Actor Components"),
		LOCTEXT("ToggleHideActorComponentsToolTip", "When enabled, hides components belonging to actors."),
		NAME_None,
		nullptr,
		DataStorage->RegisterQuery(
			Select()
			.Where()
			.None<FActorComponentTypeTag>()
			.Compile()), 
		true,
		true
		));

	// Hide Unloaded Actors
	const FName HideUnloadedActorsFilterName = TEXT("HideUnloadedActorsFilter");
    Filters.Add(HideUnloadedActorsFilterName, MakeShared<FTedsOutlinerFilter>(
    	HideUnloadedActorsFilterName,
    	LOCTEXT("ToggleHideUnloadedActors", "Hide Unloaded Actors"),
    	LOCTEXT("ToggleHideUnloadedActorsToolTip", "When enabled, hides all unloaded world partition actors."),
    	NAME_None,
    	nullptr,
    	[](TConstQueryContext<CurrentTableInfo> Context)
    	{
			return !Context.CurrentTableHasColumns<FActorDescTag>();
    	}));

	// Hide Empty Folders
	const FName HideEmptyFoldersFilterName = TEXT("HideEmptyFoldersFilter");
	const FHierarchyHandle Hierarchy = DataStorage->FindHierarchyByName(UTedsEditorHierarchyFactory::EditorObjectHierarchyName);
	const UScriptStruct* FolderParentTag = DataStorage->GetParentTagType(Hierarchy);

	Filters.Add(HideEmptyFoldersFilterName, MakeShared<FTedsOutlinerFilter>(
		HideEmptyFoldersFilterName,
		LOCTEXT("ToggleHideEmptyFolders", "Hide Empty Folders"),
		LOCTEXT("ToggleHideEmptyFoldersToolTip", "When enabled, hides all empty folders."),
		NAME_None,
		nullptr,
		[FolderParentTag](TConstQueryContext<CurrentTableInfo> Context)
		 {
			return !Context.CurrentTableHasColumns<FFolderCompatibilityColumn>() || Context.CurrentTableHasColumns({FolderParentTag});
		 }));
}

void FTedsSceneOutlinerWidgetConstructor::GetInitializeViewMenuExtender(UE::Editor::Outliner::FTedsOutlinerParams& OutlinerParams)
{
	TSharedPtr<UE::Editor::Outliner::FTedsSceneOutlinerWorldFilter> LocalWorldFilter = WorldFilter;
	
	OutlinerParams.OnInitializeViewMenuExtender =
		UE::Editor::Outliner::FOnExtendViewMenu::CreateLambda([LocalWorldFilter](TSharedPtr<FExtender> Extender)
	{
		Extender->AddMenuExtension(SceneOutliner::ExtensionHooks::Show, EExtensionHook::After, nullptr, FMenuExtensionDelegate::CreateLambda([LocalWorldFilter](FMenuBuilder& MenuBuilder)
		{
			MenuBuilder.BeginSection("World", LOCTEXT("ShowWorldHeading", "World"));
				
			auto SetUserChosenWorld = [LocalWorldFilter](const TWeakObjectPtr<UWorld>& InWorld)
			{
				if (LocalWorldFilter)
				{
					LocalWorldFilter->SetUserChosenWorld(InWorld);
				}
			};

			auto IsCurrentUserChosenWorld = [LocalWorldFilter](const TWeakObjectPtr<UWorld>& InWorld)
			{
				if (LocalWorldFilter)
				{
					const TWeakObjectPtr<UWorld> UserChosenWorld = LocalWorldFilter->GetUserChosenWorld();
					return (UserChosenWorld == InWorld) || (InWorld.IsExplicitlyNull() && !UserChosenWorld.IsValid());
				}
				return false;
			};
		
			MenuBuilder.AddSubMenu(
				LOCTEXT("ChooseWorldSubMenu", "Choose World"),
				LOCTEXT("ChooseWorldSubMenuToolTip", "Choose the world to display in the outliner."),
				FNewMenuDelegate::CreateLambda([SetUserChosenWorld, IsCurrentUserChosenWorld](FMenuBuilder& MenuBuilder)
				{
					MenuBuilder.BeginSection("Worlds", LOCTEXT("WorldsHeading", "Worlds"));
					{
						MenuBuilder.AddMenuEntry(
							LOCTEXT("AutoWorld", "Auto"),
							LOCTEXT("AutoWorldToolTip", "Automatically pick the world to display based on context."),
							FSlateIcon(),
							FUIAction(
								FExecuteAction::CreateLambda([SetUserChosenWorld]()
								{
									SetUserChosenWorld(TWeakObjectPtr<UWorld>());
								}),
								FCanExecuteAction(),
								FIsActionChecked::CreateLambda([IsCurrentUserChosenWorld]()
								{
									return IsCurrentUserChosenWorld(TWeakObjectPtr<UWorld>());
								})
							),
							NAME_None,
							EUserInterfaceActionType::RadioButton
						);

						for (const FWorldContext& Context : GEngine->GetWorldContexts())
						{
							UWorld* World = Context.World();
							if (World && (World->WorldType == EWorldType::PIE || Context.WorldType == EWorldType::Editor))
							{
								MenuBuilder.AddMenuEntry(
									SceneOutliner::GetWorldDescription(World),
									LOCTEXT("ChooseWorldToolTip", "Display actors for this world."),
									FSlateIcon(),
									FUIAction(
										FExecuteAction::CreateLambda([SetUserChosenWorld, World]()
										{
											SetUserChosenWorld(World);
										}),
										FCanExecuteAction(),
										FIsActionChecked::CreateLambda([IsCurrentUserChosenWorld, World]()
										{
											return IsCurrentUserChosenWorld(World);
										})),
									NAME_None,
									EUserInterfaceActionType::RadioButton
								);
							}
						}
					}
					MenuBuilder.EndSection();
				}));
		
			MenuBuilder.EndSection();
		}));
	});
}

void FTedsSceneOutlinerWidgetConstructor::GetOnItemDoubleClick(UE::Editor::Outliner::FTedsOutlinerParams& OutlinerParams)
{
	using namespace UE::Editor::DataStorage;
	using namespace UE::Editor::Outliner;

	OutlinerParams.OnItemDoubleClick =
		FOnOutlinerItemDoubleClick::CreateLambda(
		[](ICoreProvider* Storage, RowHandle OutlinerRow, RowHandle Row) -> FReply
		{
			if (!Storage)
			{
				return FReply::Unhandled();
			}
			
			// Folder double-click: toggle expansion or toggle current folder
			if (const FFolderCompatibilityColumn* FolderColumn = Storage->GetColumn<FFolderCompatibilityColumn>(Row))
			{
				if (const FTedsSceneOutlinerSettingsCacheColumn* CachedSettingsColumn = Storage->GetColumn<FTedsSceneOutlinerSettingsCacheColumn>(OutlinerRow))
				{
					if (const bool* CachedFolderDoubleClick = CachedSettingsColumn->CachedSettings.Find(DoubleClickTogglesCurrentFolderOption))
					{
						if (*CachedFolderDoubleClick)
						{
							if (const FTypedElementWorldColumn* WorldColumn = Storage->GetColumn<FTypedElementWorldColumn>(Row))
							{
								if (UWorld* World = WorldColumn->World.Get())
								{
									const FScopedTransaction Transaction(LOCTEXT("ToggleCurrentActorFolder", "Toggle Current Actor Folder"));
									const FFolder& Folder = FolderColumn->Folder;
									if (FActorFolders::Get().GetActorEditorContextFolder(*World) == Folder)
									{
										FActorFolders::Get().SetActorEditorContextFolder(*World, FFolder::GetWorldRootFolder(World));
									}
									else
									{
										FActorFolders::Get().SetActorEditorContextFolder(*World, Folder);
									}
								}
							}
						}
						else
						{
							if (const FSceneOutlinerColumn* OutlinerColumn = Storage->GetColumn<FSceneOutlinerColumn>(OutlinerRow);
								OutlinerColumn && OutlinerColumn->Outliner.IsValid())
							{
								if (TSharedPtr<ISceneOutliner> Outliner = OutlinerColumn->Outliner.Pin())
								{
									if (const FSceneOutlinerTreeItemPtr TreeItem = Outliner->GetTreeItem(Row))
									{
										Outliner->SetItemExpansion(TreeItem, !Outliner->GetTree().IsItemExpanded(TreeItem));
									}
								}
							}
						}
					}
				}
				
				return FReply::Handled();
			}

			// Alt+double-click: LevelInstance enter/exit edit
			if (FSlateApplication::Get().GetModifierKeys().IsAltDown())
			{
				if (const FTypedElementUObjectColumn* ObjectColumn = Storage->GetColumn<FTypedElementUObjectColumn>(Row))
				{
					if (UObject* Object = ObjectColumn->Object.Get(); ILevelInstanceInterface* LevelInstance = Cast<ILevelInstanceInterface>(Object))
					{
						if (LevelInstance->CanEnterEdit())
						{
							LevelInstance->EnterEdit();
							return FReply::Handled();
						}
						if (LevelInstance->CanExitEdit())
						{
							LevelInstance->ExitEdit();
							return FReply::Handled();
						}
					}
				}
			}
			return FReply::Unhandled();
		});
}

void FTedsSceneOutlinerWidgetConstructor::GetOnCanPopulate(UE::Editor::Outliner::FTedsOutlinerParams& OutlinerParams)
{
	using namespace UE::Editor::DataStorage;
	using namespace UE::Editor::Outliner;
	
	OutlinerParams.OnCanPopulate = FOnCanPopulate::CreateLambda(
	[](ICoreProvider* Storage, RowHandle OutlinerRow)
	{
		if (Storage)
		{
			if (const FTedsSceneOutlinerSettingsCacheColumn* SettingsCache = Storage->GetColumn<FTedsSceneOutlinerSettingsCacheColumn>(OutlinerRow))
			{
				if (const bool* bUpdateInPIE = SettingsCache->CachedSettings.Find(UpdateInPIEOption); bUpdateInPIE && !*bUpdateInPIE)
				{
					if (const FTypedElementWorldColumn* WorldColumn = Storage->GetColumn<FTypedElementWorldColumn>(OutlinerRow))
					{
						if (const UWorld* World = WorldColumn->World.Get())
						{
							if (const UGameViewportClient* GameViewport = World->GetGameViewport())
							{
								return !GameViewport->Viewport || !GameViewport->Viewport->HasFocus();
							}
						}
					}
				}
			}
		}
		return true;
	});
}

void FTedsSceneOutlinerWidgetConstructor::GetInitializeOptionsMenuExtender(UE::Editor::Outliner::FTedsOutlinerParams& OutlinerParams, UE::Editor::DataStorage::ICoreProvider* DataStorage)
{
	using namespace UE::Editor::Outliner;

	OutlinerParams.OnExtendOptionsMenu = FOnExtendOptionsMenu::CreateLambda(
		[OutlinerId = TabIdentifier, Storage = DataStorage](FMenuBuilder& MenuBuilder)
		{
			const RowHandle OutlinerRow = Storage->LookupMappedRow(MappingDomain, FMapKeyView(OutlinerId));
			MenuBuilder.AddMenuEntry(
				LOCTEXT("FolderDoubleClickToggleCurrentFolderLabel", "Double Click toggles Current Folder"),
				LOCTEXT("FolderDoubleClickToggleCurrentFolderTooltip", "When enabled, double clicking on a folder will toggle its Current Folder state instead of its expansion."),
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateLambda([OutlinerId, OutlinerRow, Storage]()
					{
						ToggleAndSaveOption(Storage, OutlinerRow, OutlinerId, DoubleClickTogglesCurrentFolderOption);
					}),
					FCanExecuteAction(),
					FIsActionChecked::CreateLambda([Storage, OutlinerRow]()
					{
						if (const FTedsSceneOutlinerSettingsCacheColumn* SettingsCache = Storage->GetColumn<FTedsSceneOutlinerSettingsCacheColumn>(OutlinerRow))
						{
							const bool* bDoubleClickFolderOption = SettingsCache->CachedSettings.Find(DoubleClickTogglesCurrentFolderOption);
							return bDoubleClickFolderOption && *bDoubleClickFolderOption;
						}
						return false;
					})
				),
				NAME_None,
				EUserInterfaceActionType::ToggleButton
			);

			MenuBuilder.AddMenuEntry(
				LOCTEXT("ShouldUpdateContentWhileInPIEFocusedLabel", "Update In PIE"),
				LOCTEXT("ShouldUpdateContentWhileInPIEFocusedTooltip", "When enabled, the Outliner will update in PIE."),
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateLambda([OutlinerId, OutlinerRow, Storage]()
					{
						ToggleAndSaveOption(Storage, OutlinerRow, OutlinerId, UpdateInPIEOption);
					}),
					FCanExecuteAction(),
					FIsActionChecked::CreateLambda([Storage, OutlinerRow]()
					{
						if (const FTedsSceneOutlinerSettingsCacheColumn* SettingsCache = Storage->GetColumn<FTedsSceneOutlinerSettingsCacheColumn>(OutlinerRow))
						{
							const bool* bUpdateInPIEOption = SettingsCache->CachedSettings.Find(UpdateInPIEOption);
							return bUpdateInPIEOption && *bUpdateInPIEOption;
						}
						return true;
					}),
					FIsActionButtonVisible::CreateLambda([]()
					{
						const IWorldPartitionEditorModule* WorldPartitionModule = FModuleManager::GetModulePtr<IWorldPartitionEditorModule>("WorldPartitionEditor");
						return WorldPartitionModule ? !WorldPartitionModule->GetDisablePIE() : true;
					})
				),
				NAME_None,
				EUserInterfaceActionType::ToggleButton
			);
		});
}

void FTedsSceneOutlinerWidgetConstructor::GetExpansionStateBridge(UE::Editor::Outliner::FTedsOutlinerParams& OutlinerParams)
{
	using namespace UE::Editor::DataStorage;
	
	OutlinerParams.ExpansionStateBridge.SetExpansionState = ([](RowHandle Row, bool bIsExpanded)
		{
			if (ICoreProvider* Storage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName))
			{
				FFolderCompatibilityColumn* FolderColumn = Storage->GetColumn<FFolderCompatibilityColumn>(Row);
				FTypedElementWorldColumn* WorldColumn = Storage->GetColumn<FTypedElementWorldColumn>(Row);
				// Currently only folders handle expansion state via TEDS
				if (FolderColumn && WorldColumn && WorldColumn->World.IsValid())
				{
					// Since folders are eventually going to use TEDS as the source of truth, we don't need sync procesors and tags. Just directly
					// set the value via the API for now and eventually we can just directly add/remove the tag
					FActorFolders::Get().SetIsFolderExpanded(*WorldColumn->World.Get(), FolderColumn->Folder, bIsExpanded);
				}
			}
		});
	
	OutlinerParams.ExpansionStateBridge.GetExpansionState = ([](RowHandle Row) -> bool
		{
			if (ICoreProvider* Storage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName))
			{
				// Folders can directly check FExpandedInUITag right now
				if (Storage->HasColumns<FFolderCompatibilityColumn>(Row))
				{
					return Storage->HasColumns<FExpandedInUITag>(Row);
				}
				// Actor have to check their default expansion state in the AActor class
				else if (Storage->HasColumns<FTypedElementActorTag>(Row))
				{
					if (FTypedElementUObjectColumn* ObjectColumn = Storage->GetColumn<FTypedElementUObjectColumn>(Row))
					{
						if (AActor* Actor = Cast<AActor>(ObjectColumn->Object.Get()))
						{
							return Actor->bDefaultOutlinerExpansionState;
						}
					}
				}
				// Unloaded actors are collapsed by default
				else if (Storage->HasColumns<FActorDescTag>(Row))
				{
					return false;
				}
			}
			
			// All other types are treated as expanded by default
			return true;
		});
}

void FTedsSceneOutlinerWidgetConstructor::GetAdditionalObserverQueries(UE::Editor::Outliner::FTedsOutlinerParams& OutlinerParams)
{
	using namespace UE::Editor::DataStorage;
	
	FQueryDescription OnAddObserver(GetBaseOutlinerQuery());
	OnAddObserver.Callback.Name = TEXT("Hide row from Outliner on FHideRowFromUITag addition");
	OnAddObserver.Callback.Type = EQueryCallbackType::ObserveAdd;
	OnAddObserver.Callback.ExecutionMode = EExecutionMode::GameThread;
	OnAddObserver.Callback.MonitoredType = FHideRowFromUITag::StaticStruct();
	
	// Flip the logic here, we want to REMOVE rows from the Outliner when the FHideRowFromUITag is ADDED
	OutlinerParams.RemoveObserverQuery = MoveTemp(OnAddObserver);
	
	FQueryDescription OnRemoveObserver(GetBaseOutlinerQuery());
	OnRemoveObserver.Callback.Name = TEXT("Show row in Outliner on FHideRowFromUITag removal");
	OnRemoveObserver.Callback.Type = EQueryCallbackType::ObserveRemove;
	OnRemoveObserver.Callback.ExecutionMode = EExecutionMode::GameThread;
	OnRemoveObserver.Callback.MonitoredType = FHideRowFromUITag::StaticStruct();
	
	// Flip the logic here, we want to ADD rows to the Outliner when the FHideRowFromUITag is REMOVED
	OutlinerParams.AddObserverQuery = MoveTemp(OnRemoveObserver);
}

void FTedsSceneOutlinerWidgetConstructor::GetOnCustomAddToToolbar(UE::Editor::Outliner::FTedsOutlinerParams& OutlinerParams, UE::Editor::DataStorage::ICoreProvider* DataStorage)
{
	using namespace UE::Editor::DataStorage;
	using namespace UE::Editor::Outliner;

	OutlinerParams.OnCustomAddToToolbar = FOnCustomAddToToolbar::CreateLambda(
		[DataStorage](TSharedPtr<SHorizontalBox> Toolbar, RowHandle OutlinerRow)
		{
			Toolbar->AddSlot()
				.VAlign(VAlign_Center)
				.AutoWidth()
				.Padding(4.f, 0.f, 0.f, 0.f)
				[
					SNew(SButton)
					.ButtonStyle(FAppStyle::Get(), "SimpleButton")
					.ToolTipText(LOCTEXT("CreateFolderToolTip", "Create a new folder containing the current selection"))
					.OnClicked(FOnClicked::CreateLambda([DataStorage, OutlinerRow]() -> FReply
					{
						if (!DataStorage)
						{
							return FReply::Handled();
						}

						const FSceneOutlinerColumn* OutlinerColumn = DataStorage->GetColumn<FSceneOutlinerColumn>(OutlinerRow);
						const FTypedElementWorldColumn* WorldColumn = DataStorage->GetColumn<FTypedElementWorldColumn>(OutlinerRow);
						if (!OutlinerColumn || !WorldColumn)
						{
							return FReply::Handled();
						}

						TSharedPtr<SSceneOutliner> Outliner = StaticCastSharedPtr<SSceneOutliner>(OutlinerColumn->Outliner.Pin());
						UWorld* World = WorldColumn->World.Get();
						if (!Outliner.IsValid() || !World)
						{
							return FReply::Handled();
						}

						// Snapshot folder rows in the current selection so they can be moved under the new folder.
						const TArray<FSceneOutlinerTreeItemPtr> SelectedItems = Outliner->GetTree().GetSelectedItems();
						TArray<FFolder> PreviouslySelectedFolders;
						for (const FSceneOutlinerTreeItemPtr& Item : SelectedItems)
						{
							if (const FTedsOutlinerTreeItem* TedsItem = Item->CastTo<FTedsOutlinerTreeItem>())
							{
								if (const FFolderCompatibilityColumn* FolderCol = DataStorage->GetColumn<FFolderCompatibilityColumn>(TedsItem->GetRowHandle()))
								{
									if (!FolderCol->Folder.IsNone())
									{
										PreviouslySelectedFolders.Add(FolderCol->Folder);
									}
								}
							}
						}

						const FFolder NewFolder = Helpers::CreateFolderForSelection(*DataStorage, *Outliner, *World, SelectedItems);

						if (!NewFolder.IsNone() && !PreviouslySelectedFolders.IsEmpty())
						{
							for (const FFolder& SelectedFolder : PreviouslySelectedFolders)
							{
								if (SelectedFolder.GetRootObject() == NewFolder.GetRootObject())
								{
									const FFolder NewPath = FActorFolders::Get().GetFolderName(*World, NewFolder, SelectedFolder.GetLeafName());
									FActorFolders::Get().RenameFolderInWorld(*World, SelectedFolder, NewPath);
								}
							}
						}

						return FReply::Handled();
					}))
					[
						SNew(SImage)
						.ColorAndOpacity(FSlateColor::UseForeground())
						.Image(FAppStyle::Get().GetBrush("SceneOutliner.NewFolderIcon"))
					]
				];
		});
}


bool FTedsSceneOutlinerWidgetConstructor::ShowRowHandleColumn() const
{
	return bShowTedsOutlinerRowHandles;
}

bool FTedsSceneOutlinerWidgetConstructor::ForceShowParents() const
{
	return true;
}

bool FTedsSceneOutlinerWidgetConstructor::UseDefaultObservers() const
{
	return true;
}

bool FTedsSceneOutlinerWidgetConstructor::ShowViewButton() const
{
	return true;
}

TSharedPtr<UE::Editor::Outliner::ITedsOutlinerHierarchyDataInterface> FTedsSceneOutlinerWidgetConstructor::GetHierarchyData(UE::Editor::DataStorage::ICoreProvider* DataStorage) const
{
	using namespace UE::Editor::Outliner;
	
	const TArray<FHierarchyHandle> Hierarchies{
		DataStorage->FindHierarchyByName(UTedsEditorHierarchyFactory::EditorObjectHierarchyName)
		};
	
	return MakeShared<FTedsOutlinerMultiHierarchyInterface>(MakeShared<FHierarchyViewerMultiData>(*DataStorage, Hierarchies));
}

UTypedElementSelectionSet* FTedsSceneOutlinerWidgetConstructor::GetSelectionSet() const
{
	TWeakPtr<ILevelEditor> LevelEditor = FModuleManager::GetModuleChecked<FLevelEditorModule>(LevelEditorModuleName).GetLevelEditorInstance();
	if (TSharedPtr<ILevelEditor> LevelEditorPinned = LevelEditor.Pin())
	{
		return LevelEditorPinned->GetMutableElementSelectionSet();
	}
	return nullptr;
}

TMap<FName, TVariant<UE::Editor::DataStorage::QueryHandle, UE::Editor::DataStorage::Queries::TConstQueryFunction<bool>>>
	FTedsSceneOutlinerWidgetConstructor::GetLegacyFilterConversionMap(UE::Editor::DataStorage::ICoreProvider* DataStorage) const
{
	using namespace UE::Editor::DataStorage::Queries;
	if (DataStorage)
	{
		return
		{
			{
				FName("UnsavedAssetsFilter"),
				TVariant<QueryHandle, TConstQueryFunction<bool>>( TInPlaceType<QueryHandle>(),
				DataStorage->RegisterQuery(
				Select()
				.Where()
				.All<FTedsPackageDirtyTag>()
				.Compile()))
			},
			{
				FName("UncontrolledAssetsFilter"),
				TVariant<QueryHandle, TConstQueryFunction<bool>>( TInPlaceType<TConstQueryFunction<bool>>(),
					BuildConstQueryFunction<bool>([DataStorage](TConstQueryContext<RowBatchInfo> Context, TResult<bool>& Result,
						TConstBatch<FTypedElementPackageReference> PackageRefColumns)
					{
						Context.ForEachRow([&Result, DataStorage](RowHandle Row, const FTypedElementPackageReference& PackageRefColumn)
						{
							bool bIsUncontrolled = false;
							if (DataStorage)
							{
								if (const FSccStateColumn* SccState = DataStorage->GetColumn<FSccStateColumn>(PackageRefColumn.Row))
								{
									bIsUncontrolled = SccState->bIsExternallyEdited;
								}
							}
							Result.Add(Row, bIsUncontrolled);
						}, PackageRefColumns);
					})
				)
			},
		};
	}
	return {};
}


void FTedsSceneOutlinerWidgetConstructor::GetLevelEditorSceneOutlinerInitOptions(FSceneOutlinerInitializationOptions& OutInitOptions) const
{
	TWeakPtr<ILevelEditor> LevelEditor = FModuleManager::GetModuleChecked<FLevelEditorModule>(LevelEditorModuleName).GetLevelEditorInstance();
	if (TSharedPtr<ILevelEditor> LevelEditorPinned = LevelEditor.Pin())
	{
		LevelEditorPinned->GetSceneOutlinerInitializationOptions(TabIdentifier, OutInitOptions);
	}
}

void FTedsSceneOutlinerWidgetConstructor::AddFavoriteFilters(UE::Editor::DataStorage::ICoreProvider* DataStorage, TArray<TSharedPtr<UE::Editor::Outliner::FTedsOutlinerFilter>>& Filters) const
{
	using namespace UE::Editor::DataStorage::Queries;
	
	const TSharedRef<FFilterCategory> FavoriteFiltersCategory = MakeShared<FFilterCategory>(LOCTEXT("FavoriteFilters", "Favorites"), LOCTEXT("FavoriteFiltersTooltip", "Filter by Favorite state"));

	// Create the filter
	Filters.Add(MakeShared<UE::Editor::Outliner::FTedsOutlinerFilter>(
		"FilterByFavorites",
		LOCTEXT("FavoriteFilterDisplayName", "Favorite"),
		LOCTEXT("FavoriteFilterDisplayNameToolTip", "Filter by Favorites"),
		FName("Icons.Star"),
		FavoriteFiltersCategory,
		DataStorage->RegisterQuery(
			Select()
			.Where()
			.All<FFavoriteTag>()
			.Compile())
		));
}

void FTedsSceneOutlinerWidgetConstructor::AddAlertFilters(UE::Editor::DataStorage::ICoreProvider* DataStorage, TArray<TSharedPtr<UE::Editor::Outliner::FTedsOutlinerFilter>>& Filters) const
{
	using namespace UE::Editor::DataStorage::Queries;
	
	const TSharedPtr<FFilterCategory> AlertFilterCategory = MakeShared<FFilterCategory>(LOCTEXT("AlertFilterCategory", "Alerts"),
		LOCTEXT("AlertFilterCategoryTooltip", "Filters for items in the Outliner that have any alerts (warning or errors)"));

	// Add a filter for warning alert
	Filters.Add(MakeShared<UE::Editor::Outliner::FTedsOutlinerFilter>(
		"Warning",
		LOCTEXT("WarningFilterName", "Warning"),
		LOCTEXT("WarningFilterTooltip", "Only show items that have a warning."),
		NAME_None,
		AlertFilterCategory,
		[](TConstQueryContext<RowBatchInfo> Context, TResult<bool>& Result, TConstBatch<FTedsAlertColumn> AlertColumns)
		{
			Context.ForEachRow([&Result](RowHandle Row, const FTedsAlertColumn& AlertColumn)
			{
				Result.Add(Row, AlertColumn.AlertType == FTedsAlertColumnType::Warning);
			}, AlertColumns);
		}));

	// Add a filter for error alert
	Filters.Add(MakeShared<UE::Editor::Outliner::FTedsOutlinerFilter>(
		"Error",
		LOCTEXT("ErrorFilterName", "Error"),
		LOCTEXT("ErrorFilterTooltip", "Only show items that have an error."),
		NAME_None,
		AlertFilterCategory,
		[](TConstQueryContext<RowBatchInfo> Context, TResult<bool>& Result, TConstBatch<FTedsAlertColumn> AlertColumns)
		{
			Context.ForEachRow([&Result](RowHandle Row, const FTedsAlertColumn& AlertColumn)
			{
				Result.Add(Row, AlertColumn.AlertType == FTedsAlertColumnType::Error);
			}, AlertColumns);
		}));

	// Add a filter for items that have any alert
	Filters.Add(MakeShared<UE::Editor::Outliner::FTedsOutlinerFilter>(
		"Alert",
		LOCTEXT("AlertFilterName", "Any Alerts"),
		LOCTEXT("AlertFilterTooltip", "Only show items that have any alerts (warning or error)."),
		NAME_None,
		AlertFilterCategory,
		DataStorage->RegisterQuery(
			Select()
			.Where()
			.All<FTedsAlertColumn>()
			.Compile())
		));
}

void FTedsSceneOutlinerWidgetConstructor::AddSCCFilters(UE::Editor::DataStorage::ICoreProvider* DataStorage, TArray<TSharedPtr<UE::Editor::Outliner::FTedsOutlinerFilter>>& Filters) const
{
	using namespace UE::Editor::DataStorage::Queries;
	
	// Only show the filters if we are populating TEDS with revision control data
	IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("TEDS.RevisionControl.AutoPopulateState"));
	
	if(!CVar || !CVar->GetBool())
	{
		return;
	}

	const ICoreProvider* Storage = GetDataStorageFeature<ICoreProvider>(StorageFeatureName);
	if (!Storage)
	{
		return;
	}

	// SCC Category
	FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>("LevelEditor");
	const TSharedPtr<FFilterCategory> SCCFilterCategory = LevelEditorModule.GetOutlinerFilterCategory(FLevelEditorOutlinerBuiltInCategories::SourceControl());

	// Add a filter for items that are checked out
	Filters.Add(MakeShared<UE::Editor::Outliner::FTedsOutlinerFilter>(
		"CheckedOut",
		LOCTEXT("CheckedOutFilterName", "Checked Out"),
		LOCTEXT("CheckedOutFilterTooltip", "Only show items that you have checked out or pending for add."),
		NAME_None,
		SCCFilterCategory,
		BuildConstQueryFunction<bool>([DataStorage](TConstQueryContext<RowBatchInfo> Context, TResult<bool>& Result,
			TConstBatch<FTypedElementPackageReference> PackageRefColumns)
		{
			Context.ForEachRow([&Result, DataStorage](RowHandle Row, const FTypedElementPackageReference& PackageRefColumn)
			{
				bool bIsLocked = false;
				if (DataStorage)
				{
					if (const FSccStateColumn* SccState = DataStorage->GetColumn<FSccStateColumn>(PackageRefColumn.Row))
					{
						bIsLocked = SccState->bIsLocked;
					}
				}
				Result.Add(Row, bIsLocked);
			}, PackageRefColumns);
		})));
	
	// Add a filter for items that are not at head revision
	Filters.Add(MakeShared<UE::Editor::Outliner::FTedsOutlinerFilter>(
		"NotCurrent",
		LOCTEXT("NotCurrentFilterName", "Not at Head Revision"),
		LOCTEXT("NotCurrentFilterTooltip", "Only show items that have a new revision available."),
		NAME_None,
		SCCFilterCategory,
		BuildConstQueryFunction<bool>([DataStorage](TConstQueryContext<RowBatchInfo> Context, TResult<bool>& Result,
			TConstBatch<FTypedElementPackageReference> PackageRefColumns)
		{
			Context.ForEachRow([&Result, DataStorage](RowHandle Row, const FTypedElementPackageReference& PackageRefColumn)
			{
				bool bIsNotCurrent = false;
				if (DataStorage)
				{
					if (const FSccStateColumn* SccState = DataStorage->GetColumn<FSccStateColumn>(PackageRefColumn.Row))
					{
						bIsNotCurrent = SccState->bIsNotCurrent;
					}
				}
				Result.Add(Row, bIsNotCurrent);
			}, PackageRefColumns);
		})));
	
	// Add a filter for items that are checked out by others
	Filters.Add(MakeShared<UE::Editor::Outliner::FTedsOutlinerFilter>(
		"CheckedOutByOther",
		LOCTEXT("CheckedOutByOthersFilterName", "Checked Out by Other"),
		LOCTEXT("CheckedOutByOthersFilterTooltip", "Only show items that are checked out by someone else."),
		NAME_None,
		SCCFilterCategory,
		BuildConstQueryFunction<bool>([DataStorage](TConstQueryContext<RowBatchInfo> Context, TResult<bool>& Result,
			TConstBatch<FTypedElementPackageReference> PackageRefColumns)
		{
			Context.ForEachRow([&Result, DataStorage](RowHandle Row, const FTypedElementPackageReference& PackageRefColumn)
			{
				bool bIsExternallyEdited = false;
				if (DataStorage)
				{
					if (const FSccStateColumn* SccState = DataStorage->GetColumn<FSccStateColumn>(PackageRefColumn.Row))
					{
						bIsExternallyEdited = SccState->bIsExternallyEdited;
					}
				}
				Result.Add(Row, bIsExternallyEdited);
			}, PackageRefColumns);
		})));
}

FTedsSceneOutlinerActorEditorContextSubsystemFilter::FTedsSceneOutlinerActorEditorContextSubsystemFilter(const FName& InFilterName, const FText InFilterDisplayName,
	const FText InFilterTooltip, const UE::Editor::DataStorage::Queries::TConstQueryFunction<bool>& InFilterQuery)
	: FTedsOutlinerFilter(
		InFilterName,
		InFilterDisplayName,
		InFilterTooltip,
		NAME_None,
		nullptr,
		InFilterQuery)
{
	using namespace UE::Editor::DataStorage;

	WorldPartitionModule = FModuleManager::GetModulePtr<IWorldPartitionEditorModule>("WorldPartitionEditor");
	DataStorage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName);
}

FTedsSceneOutlinerActorEditorContextSubsystemFilter::~FTedsSceneOutlinerActorEditorContextSubsystemFilter()
{
	if (OnActorEditorContextSubsystemChangedDelegate.IsValid())
	{
		if (UActorEditorContextSubsystem* ActorEditorContextSubsystem = UActorEditorContextSubsystem::Get())
		{
			ActorEditorContextSubsystem->OnActorEditorContextSubsystemChanged().Remove(OnActorEditorContextSubsystemChangedDelegate);
		}
	}
}

void FTedsSceneOutlinerActorEditorContextSubsystemFilter::ActiveStateChanged(bool bActive)
{
	if (UActorEditorContextSubsystem* ActorEditorContextSubsystem = UActorEditorContextSubsystem::Get())
	{
		if (bActive)
		{
			OnActorEditorContextSubsystemChangedDelegate = ActorEditorContextSubsystem->OnActorEditorContextSubsystemChanged()
				.AddRaw(this, &FTedsSceneOutlinerActorEditorContextSubsystemFilter::OnActorEditorContextSubsystemChanged);
		}
		else
		{
			ActorEditorContextSubsystem->OnActorEditorContextSubsystemChanged().Remove(OnActorEditorContextSubsystemChangedDelegate);
			OnActorEditorContextSubsystemChangedDelegate.Reset();
		}
	}
	
	FTedsOutlinerFilter::ActiveStateChanged(bActive);
}

void FTedsSceneOutlinerActorEditorContextSubsystemFilter::OnActorEditorContextSubsystemChanged() const
{
	if (const TSharedPtr<UE::Editor::Outliner::FTedsOutlinerImpl> TedsOutlinerImplPin = TedsOutlinerImpl.Pin();
		TedsOutlinerImplPin.IsValid() && WorldPartitionModule)
	{
		TedsOutlinerImplPin->FullRefresh();
	}
}

FTedsSceneOutlinerCurrentDataLayerFilter::FTedsSceneOutlinerCurrentDataLayerFilter(const FName& InFilterName)
	: FTedsSceneOutlinerActorEditorContextSubsystemFilter(
	InFilterName,
	LOCTEXT("ToggleShowOnlyCurrentDataLayers", "Only in any Current Data Layers"),
	LOCTEXT("ToggleShowOnlyCurrentDataLayersToolTip", "When enabled, only shows Actors that are in any Current Data Layers."),
	UE::Editor::DataStorage::Queries::BuildConstQueryFunction<bool>(
	[this](UE::Editor::DataStorage::Queries::TConstQueryContext<UE::Editor::DataStorage::Queries::SingleRowInfo,
		UE::Editor::DataStorage::Queries::CurrentTableInfo> Context, const FTypedElementWorldColumn& WorldColumn)
		{
			using namespace UE::Editor::DataStorage;

			const AWorldDataLayers* WorldDataLayers = WorldColumn.World.IsValid() ? WorldColumn.World->GetCurrentLevel()->GetWorldDataLayers() : nullptr;
			if (!DataStorage || !WorldDataLayers || WorldDataLayers->GetActorEditorContextDataLayers().IsEmpty())
			{
			   return true;
			}

			if (Context.CurrentTableHasColumns<FWorldPartitionDataLayerColumn>())
			{
				if (const FWorldPartitionHandleColumn* HandleColumn = DataStorage->GetColumn<FWorldPartitionHandleColumn>(Context.GetCurrentRow());
					HandleColumn && HandleColumn->Handle.IsValid())
				{
				   if (const FWorldPartitionActorDescInstance* const ActorDescInstance = *HandleColumn->Handle)
				   {
					   const UDataLayerManager* DataLayerManager = UDataLayerManager::GetDataLayerManager(WorldColumn.World.Get());
					   for (const UDataLayerInstance* const DataLayerInstance : DataLayerManager->GetDataLayerInstances(ActorDescInstance->GetDataLayerInstanceNames().ToArray()))
					   {
						   if (DataLayerInstance->IsInActorEditorContext())
						   {
							   return true;
						   }
					   }
					   return false;
				   }
				}

				if (const FTypedElementUObjectColumn* ObjectColumn = DataStorage->GetColumn<FTypedElementUObjectColumn>(Context.GetCurrentRow()))
				{
				   if (const AActor* Actor = Cast<AActor>(ObjectColumn->Object.Get()))
				   {
					   for (const UDataLayerInstance* DataLayerInstance : Actor->GetDataLayerInstances())
					   {
						   if (DataLayerInstance->IsInActorEditorContext())
						   {
							   return true;
						   }
					   }
					   return false;
				   }
				}
			}

			return false;
		}))
{}

void FTedsSceneOutlinerWidgetConstructor::GetOnRegisterInteractiveFilters(UE::Editor::Outliner::FTedsOutlinerParams& OutlinerParams, UE::Editor::DataStorage::ICoreProvider* DataStorage)
{
	using namespace UE::Editor::DataStorage;
	using namespace UE::Editor::Outliner;

	OutlinerParams.OnRegisterInteractiveFilters = FOnRegisterInteractiveFilters::CreateLambda(
		[DataStorage](SSceneOutliner& Outliner)
		{
			using namespace UE::Editor::DataStorage::Queries;

			// Inside-LI rule: a row contained in a non-editing LI is non-interactive. Rows outside any LI defer to the next filter.
			Outliner.AddInteractiveFilter(MakeShared<TSceneOutlinerPredicateFilter<FTedsOutlinerTreeItem>>(
				FTedsOutlinerTreeItem::FFilterPredicate::CreateLambda([](const RowHandle)
					{
						return true;
					}),
				FSceneOutlinerFilter::EDefaultBehaviour::Pass,
				FTedsOutlinerTreeItem::FInteractivePredicate::CreateLambda([DataStorage](const RowHandle InRow) -> bool
				{
					if (!DataStorage)
					{
						return true;
					}

					const RowHandle ContainingLevelInstanceRow = Helpers::LevelInstance::FindContainingLevelInstanceRow(DataStorage, InRow);
					if (ContainingLevelInstanceRow != InvalidRowHandle
						&& !DataStorage->HasColumns<FLevelInstanceEditingColumn>(ContainingLevelInstanceRow))
					{
						return false;
					}
					return true;
				})
			));

		
			static QueryHandle LevelInstanceEditingCountQuery = DataStorage->RegisterQuery(
				Count()
				.Where()
					.All<FLevelInstanceEditingColumn>()
				.Compile());

			// Outside-LI rule: while a LI is being edited, a row that is not inside any LI is non-interactive unless it belongs to the edit hierarchy.
			Outliner.AddInteractiveFilter(MakeShared<TSceneOutlinerPredicateFilter<FTedsOutlinerTreeItem>>(
				FTedsOutlinerTreeItem::FFilterPredicate::CreateLambda([](const RowHandle)
					{
						return true;
					}),
				FSceneOutlinerFilter::EDefaultBehaviour::Pass,
				FTedsOutlinerTreeItem::FInteractivePredicate::CreateLambda([DataStorage](const RowHandle InRow) -> bool
				{
					if (!DataStorage)
					{
						return true;
					}

					const FQueryResult EditingCountResult = DataStorage->RunQuery(LevelInstanceEditingCountQuery);
					if (EditingCountResult.Count == 0)
					{
						return true;
					}

					if (Helpers::LevelInstance::FindContainingLevelInstanceRow(DataStorage, InRow) != InvalidRowHandle)
					{
						return true;
					}

					if (DataStorage->GetColumn<FFolderCompatibilityColumn>(InRow))
					{
						return false;
					}
					if (DataStorage->HasColumns<FTypedElementActorTag>(InRow))
					{
						// The LI Actor should remain editable
						if (DataStorage->HasColumns<FLevelInstanceTag>(InRow)
							&& DataStorage->HasColumns<FLevelInstanceEditingColumn>(InRow))
						{
							return true;
						}
						return Helpers::LevelInstance::IsInEditingLevelInstanceHierarchy(DataStorage, InRow);
					}
					if (DataStorage->HasColumns<FActorDescTag>(InRow))
					{
						return false;
					}
					return true;
				})
			));
		});
}

FTedsSceneOutlinerCurrentContentBundleFilter::FTedsSceneOutlinerCurrentContentBundleFilter(const FName& InFilterName)
	: FTedsSceneOutlinerActorEditorContextSubsystemFilter(
	InFilterName,
	LOCTEXT("ToggleShowOnlyCurrentContentBundle", "Only in Current Content Bundle"),
	LOCTEXT("ToggleShowOnlyCurrentContentBundleToolTip", "When enabled, only shows Actors that are in the Current Content Bundle."),
	UE::Editor::DataStorage::Queries::BuildConstQueryFunction<bool>(
	[this](UE::Editor::DataStorage::Queries::TConstQueryContext<UE::Editor::DataStorage::Queries::SingleRowInfo,
		UE::Editor::DataStorage::Queries::CurrentTableInfo> Context, const FTypedElementWorldColumn& WorldColumn)
		{
			using namespace UE::Editor::DataStorage;
	  		
			if (WorldPartitionModule && WorldPartitionModule->IsEditingContentBundle() )
			{
				// Only Unloaded Actors and Actors can have content bundles
				if (Context.CurrentTableHasColumns<FWorldPartitionContentBundleColumn>())
				{
					if (const FWorldPartitionHandleColumn* HandleColumn = DataStorage->GetColumn<FWorldPartitionHandleColumn>(Context.GetCurrentRow());
						HandleColumn && HandleColumn->Handle.IsValid())
					{
						if (const FWorldPartitionActorDescInstance* const ActorDescInstance = *HandleColumn->Handle)
						{
							const FGuid& Guid = ActorDescInstance->GetContentBundleGuid();
							return Guid.IsValid() && WorldPartitionModule->IsEditingContentBundle(Guid);
						}
					}
					if (const FTypedElementUObjectColumn* ObjColumn = DataStorage->GetColumn<FTypedElementUObjectColumn>(Context.GetCurrentRow()))
					{
						if (const AActor* Actor = Cast<AActor>(ObjColumn->Object.Get()))
						{
							const FGuid& Guid = Actor->GetContentBundleGuid();
							return Guid.IsValid() && WorldPartitionModule->IsEditingContentBundle(Guid);
						}
					}
				}
				return false;
			}
			return true;
		}))
{}

#undef LOCTEXT_NAMESPACE