// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SocketWidget.h"

#include "Columns/TedsActorSocketColumns.h"
#include "Elements/Columns/TypedElementCompatibilityColumns.h"
#include "Elements/Framework/TypedElementAttributeBinding.h"
#include "TedsOutlinerHelpers.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/SBoxPanel.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SocketWidget)

#define LOCTEXT_NAMESPACE "SocketWidget"

void USocketWidgetFactory::RegisterWidgetConstructors(UE::Editor::DataStorage::ICoreProvider& DataStorage,
	UE::Editor::DataStorage::IUiProvider& DataStorageUi) const
{
	using namespace UE::Editor::DataStorage::Queries;

	DataStorageUi.RegisterWidgetFactory<FSocketWidgetConstructor>(
	DataStorageUi.FindPurpose(DataStorageUi.GetGeneralWidgetPurposeID()),
		TColumn<FTedsActorSocketColumn>());
}

FSocketWidgetConstructor::FSocketWidgetConstructor()
	: FSimpleWidgetConstructor(StaticStruct())
{
}

TSharedPtr<SWidget> FSocketWidgetConstructor::CreateWidget(
	UE::Editor::DataStorage::ICoreProvider* DataStorage,
	UE::Editor::DataStorage::IUiProvider* DataStorageUi,
	UE::Editor::DataStorage::RowHandle TargetRow,
	UE::Editor::DataStorage::RowHandle WidgetRow,
	const UE::Editor::DataStorage::FMetaDataView& Arguments)
{
	using namespace UE::Editor::DataStorage;
	if (!DataStorage->IsRowAvailable(TargetRow))
	{
		return SNew(STextBlock)
			.Text(LOCTEXT("MissingRowReferenceColumn", "Unable to retrieve row reference."));
	}

	FAttributeBinder Binder(TargetRow, DataStorage);

	return SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(8, 0, 0, 0)
			[
				SNew(STextBlock)
					.ColorAndOpacity(FSlateColor::UseSubduedForeground())
					.Text(Binder.BindText(&FTedsActorSocketColumn::AttachedSocket))
					.HighlightText(UE::Editor::Outliner::Helpers::GetHighlightTextAttribute(DataStorage, WidgetRow))
			];
}

FText FSocketWidgetConstructor::CreateWidgetDisplayNameText(UE::Editor::DataStorage::ICoreProvider* DataStorage,
	UE::Editor::DataStorage::RowHandle Row) const
{
	return LOCTEXT("SocketHeaderDisplayName", "Socket");
}

#undef LOCTEXT_NAMESPACE
