// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/TypeInfoWidget.h"

#include "SceneOutlinerHelpers.h"
#include "Elements/Columns/TypedElementCompatibilityColumns.h"
#include "Elements/Columns/TypedElementMiscColumns.h"
#include "Elements/Columns/TypedElementSlateWidgetColumns.h"
#include "Elements/Columns/TypedElementTypeInfoColumns.h"
#include "Styling/SlateIconFinder.h"
#include "TedsOutlinerHelpers.h"
#include "TedsTableViewerUtils.h"
#include "Elements/Framework/TypedElementAttributeBinding.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Images/SImage.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TypeInfoWidget)

#define LOCTEXT_NAMESPACE "TypeInfoWidget"

namespace UE::Editor::DataStorage
{
	FText FTypeInfoWidgetSorter::GetShortName() const
	{
		return LOCTEXT("TypeInfoWidgetSorter", "Type Info Sorter");
	}

	FColumnSorterInterface::ESortType FTypeInfoWidgetSorter::GetSortType() const
	{
		return ESortType::HybridSort;
	}

	FString GetTypeSortString(const ICoreProvider& Storage, RowHandle Row)
	{
		if (const FTypedElementTypeInfoDisplayOverrideColumn* Override = Storage.GetColumn<FTypedElementTypeInfoDisplayOverrideColumn>(Row))
		{
			return Override->TypeDisplayName.ToString();
		}
		if (const FTypedElementClassTypeInfoColumn* TypeInfo = Storage.GetColumn<FTypedElementClassTypeInfoColumn>(Row))
		{
			if (TObjectPtr<const UClass> TypeInfoPtr = TypeInfo->TypeInfo.Get())
			{
				return TypeInfoPtr->GetName();
			}
		}
		return FString("");
	}

	int32 FTypeInfoWidgetSorter::Compare(const ICoreProvider& Storage, RowHandle Left, RowHandle Right) const
	{
		return GetTypeSortString(Storage, Left).Compare(GetTypeSortString(Storage, Right), ESearchCase::IgnoreCase);
	}

	FPrefixInfo FTypeInfoWidgetSorter::CalculatePrefix(const ICoreProvider& Storage, RowHandle Row, uint32 ByteIndex) const
	{
		const FString TypeName = GetTypeSortString(Storage, Row);
		return CreateSortPrefix(ByteIndex, TSortStringView(FSortCaseInsensitive{}, TypeName));
	}

	void FTypeInfoWidgetSearcher::GetSearchableString(FString& SearchableString, const ICoreProvider& Storage, RowHandle Row) const
	{
		SearchableString = GetTypeSortString(Storage, Row);
	}

}


void UTypeInfoWidgetFactory::RegisterWidgetConstructors(UE::Editor::DataStorage::ICoreProvider& DataStorage,
	UE::Editor::DataStorage::IUiProvider& DataStorageUi) const
{
	using namespace UE::Editor::DataStorage::Queries;
	
	DataStorageUi.RegisterWidgetFactory<FTypeInfoWidgetConstructor>(DataStorageUi.FindPurpose(DataStorageUi.GetGeneralWidgetPurposeID()),
		TColumn<FTypedElementClassTypeInfoColumn>() || TColumn<FTypedElementTypeInfoDisplayOverrideColumn>());

}

FTypeInfoWidgetConstructor::FTypeInfoWidgetConstructor()
	: Super(FTypeInfoWidgetConstructor::StaticStruct())
{
}

FTypeInfoWidgetConstructor::FTypeInfoWidgetConstructor(const UScriptStruct* InTypeInfo)
	: Super(InTypeInfo)
{
	
}

TSharedPtr<SWidget> FTypeInfoWidgetConstructor::CreateWidget(UE::Editor::DataStorage::ICoreProvider* DataStorage,
	UE::Editor::DataStorage::IUiProvider* DataStorageUi, UE::Editor::DataStorage::RowHandle TargetRow, UE::Editor::DataStorage::RowHandle WidgetRow,
	const UE::Editor::DataStorage::FMetaDataView& Arguments)
{
	bool bUseIcon = false;
	
	// Check if the caller provided metadata to use an icon widget
	UE::Editor::DataStorage::FMetaDataEntryView MetaDataEntryView = Arguments.FindGeneric("TypeInfoWidget_bUseIcon");
	if(MetaDataEntryView.IsSet())
	{
		check(MetaDataEntryView.IsType<bool>());

		bUseIcon = *MetaDataEntryView.TryGetExact<bool>();
	}
	
	if(bUseIcon)
	{
		return SNew(SImage)
					.DesiredSizeOverride(FVector2D(16.f, 16.f))
					.ColorAndOpacity(FSlateColor::UseForeground())
					// Icon doesn't really change post creation so this doesn't need to be an attribute
					.Image(UE::Editor::DataStorage::TableViewerUtils::GetIconForRow(DataStorage, TargetRow));
	}
	else
	{
		return SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
					.AutoWidth()
					.VAlign(VAlign_Center)
					.Padding(8, 0, 0, 0)
					[
						CreateTextWidget(DataStorage, TargetRow, WidgetRow)
					];
	}
}

TArray<TSharedPtr<const UE::Editor::DataStorage::FColumnSorterInterface>> FTypeInfoWidgetConstructor::ConstructColumnSorters(
	UE::Editor::DataStorage::ICoreProvider* DataStorage,
	UE::Editor::DataStorage::IUiProvider* DataStorageUi,
	const UE::Editor::DataStorage::FMetaDataView& Arguments)
{
	using namespace UE::Editor::DataStorage;
	return TArray<TSharedPtr<const FColumnSorterInterface>>(
		{
			MakeShared<FTypeInfoWidgetSorter>()
		});
}

TArray<TSharedPtr<const UE::Editor::DataStorage::FColumnSearcherInterface>> FTypeInfoWidgetConstructor::ConstructColumnSearchers(
	UE::Editor::DataStorage::ICoreProvider* DataStorage,
	UE::Editor::DataStorage::IUiProvider* DataStorageUi,
	const UE::Editor::DataStorage::FMetaDataView& Arguments)
{
	using namespace UE::Editor::DataStorage;
	return TArray<TSharedPtr<const FColumnSearcherInterface>>(
		{
			MakeShared<FTypeInfoWidgetSearcher>()
		});
}

TSharedRef<SWidget> FTypeInfoWidgetConstructor::CreateTextWidget(UE::Editor::DataStorage::ICoreProvider* DataStorage,
	UE::Editor::DataStorage::RowHandle TargetRow, UE::Editor::DataStorage::RowHandle WidgetRow)
{
	using namespace UE::Editor::DataStorage;
	// Check if we have a hyperlink for this object
	if (const FTypedElementUObjectColumn* ObjectColumn = DataStorage->GetColumn<FTypedElementUObjectColumn>(TargetRow))
	{
		if (TSharedPtr<SWidget> HyperlinkWidget = SceneOutliner::FSceneOutlinerHelpers::GetClassHyperlink(ObjectColumn->Object.Get()))
		{
			return HyperlinkWidget.ToSharedRef();
		}
	}
	
	// If not, we simply show a text block with the type
	TSharedPtr<STextBlock> TextBlock = SNew(STextBlock)
		.ColorAndOpacity(FSlateColor::UseSubduedForeground())
		.Text_Lambda([DataStorage, TargetRow]()
			{
				if (const FTypedElementTypeInfoDisplayOverrideColumn* Override = DataStorage->GetColumn<FTypedElementTypeInfoDisplayOverrideColumn>(TargetRow))
				{
					return Override->TypeDisplayName;
				}
				if (const FTypedElementClassTypeInfoColumn* TypeInfo = DataStorage->GetColumn<FTypedElementClassTypeInfoColumn>(TargetRow))
				{
					if (TObjectPtr<const UClass> TypeInfoPtr = TypeInfo->TypeInfo.Get())
					{
						return TypeInfoPtr->GetDisplayNameText();
					}
				}
				return FText::GetEmpty();
			})
		.HighlightText(UE::Editor::Outliner::Helpers::GetHighlightTextAttribute(DataStorage, WidgetRow));
	
	return TextBlock.ToSharedRef();
}

#undef LOCTEXT_NAMESPACE
