// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/VisibilityWidget.h"

#include "DataStorage/Features.h"
#include "Elements/Columns/TypedElementVisibilityColumns.h"
#include "Elements/Interfaces/TypedElementDataStorageInterface.h"
#include "Elements/Framework/TypedElementAttributeBinding.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Images/SImage.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(VisibilityWidget)

#define LOCTEXT_NAMESPACE "VisibilityWidget"

//
// Cell Factory
//

FVisibilityWidgetConstructor::FVisibilityWidgetConstructor()
	: FSimpleWidgetConstructor(StaticStruct())
{
}

void UVisibilityWidgetFactory::RegisterWidgetConstructors(UE::Editor::DataStorage::ICoreProvider& DataStorage,
	UE::Editor::DataStorage::IUiProvider& DataStorageUi) const
{
	using namespace UE::Editor::DataStorage::Queries;
	DataStorageUi.RegisterWidgetFactory<FVisibilityWidgetConstructor>( DataStorageUi.FindPurpose(DataStorageUi.GetGeneralWidgetPurposeID()), 
		TColumn<FVisibleInEditorColumn>());
}

TSharedPtr<SWidget> FVisibilityWidgetConstructor::CreateWidget(UE::Editor::DataStorage::ICoreProvider* DataStorage,
	UE::Editor::DataStorage::IUiProvider* DataStorageUi,
	UE::Editor::DataStorage::RowHandle TargetRow, UE::Editor::DataStorage::RowHandle WidgetRow, const UE::Editor::DataStorage::FMetaDataView& Arguments)
{
	UE::Editor::DataStorage::FAttributeBinder Binder(TargetRow, DataStorage);

	static const FName NAME_VisibleBrush = TEXT("Level.VisibleIcon16x");
	static const FName NAME_NotVisibleBrush = TEXT("Level.NotVisibleIcon16x");
	
	return SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		[
			SNew(SImage)
				.ColorAndOpacity(FSlateColor::UseForeground())
				.Image(Binder.BindData(&FVisibleInEditorColumn::bIsVisibleInEditor, [](bool IsVisible)
					{
						if (IsVisible)
						{
							return FAppStyle::Get().GetBrush(NAME_VisibleBrush);
						}
						return FAppStyle::Get().GetBrush(NAME_NotVisibleBrush);
					}))
		];
}

#undef LOCTEXT_NAMESPACE
