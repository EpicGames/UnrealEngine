// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Columns/SceneOutlinerColumns.h"
#include "Elements/Interfaces/TypedElementDataStorageInterface.h"
#include "SceneOutlinerPublicTypes.h"
#include "Templates/SharedPointer.h"

#include "TedsOutlinerColumns.generated.h"

namespace UE::Editor::Outliner
{
	struct FTedsOutlinerColumnDescription;
}

class ISceneOutliner;
struct ISceneOutlinerTreeItem;
class UToolMenu;
class SSceneOutliner;

namespace UE::Editor::Outliner
{
	// Delegate that refreshes columns on the Teds Outliner it is bound to when executed
	DECLARE_DELEGATE_OneParam(FOnRefreshColumns, const UE::Editor::Outliner::FTedsOutlinerColumnDescription&)

	// Delegate called when the selection in the Outliner changes. SelectInfo is Direct when the change originated
	// outside the outliner/through code (ex: viewport), and OnMouseClick/OnKeyPress/etc when from the Outliner.
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnTedsOutlinerSelectionChanged, ESelectInfo::Type)

	// Delegate broadcast to register Select/Rename/ScrollIntoView actions on a row's TreeItem the next time
	// it is added to the tree. Item actions are a bitmask of SceneOutliner::ENewItemAction values.
	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnRegisterPendingItemActions, DataStorage::RowHandle, uint8)

	// Delegate to get the TreeItemID for a row
	DECLARE_DELEGATE_RetVal_OneParam(FSceneOutlinerTreeItemID, FTreeItemIDDealiaser, DataStorage::RowHandle);

	// Mapping domain name for Outliners (kept for backward compatibility; OutlinerMappingDomain in SceneOutlinerColumns.h is the canonical source)
	inline static const FName MappingDomain = "TedsOutliner";
}

DECLARE_DELEGATE_TwoParams(FTedsOutlinerContextMenuDelegate, UToolMenu* Menu, SSceneOutliner& SceneOutliner)

/** Column used to allow context menu to be extended for an item in the outliner */
USTRUCT()
struct FTedsOutlinerContextMenuColumn final : public FEditorDataStorageColumn
{
	GENERATED_BODY()

	TSharedPtr<FTedsOutlinerContextMenuDelegate> OnCreateContextMenu;
};

/** Column used to refresh information about the columns being viewed in the Teds Outliner */
USTRUCT()
struct FTedsOutlinerColumnRefreshEventColumn final : public FEditorDataStorageColumn
{
	GENERATED_BODY()

	// Call this delegate to refresh the columns in the Teds Outliner
	TSharedPtr<UE::Editor::Outliner::FOnRefreshColumns> OnRefreshColumns;
};

/** Column used to store a dealiaser from outliner item -> row handle */
USTRUCT()
struct FTedsOutlinerDealiaserColumn final : public FEditorDataStorageColumn
{
	GENERATED_BODY()

	UE::Editor::Outliner::FTreeItemIDDealiaser Dealiaser;
};

/** Column with a delegate that is broadcast when the outliner's selection changes */
USTRUCT()
struct FTedsOutlinerSelectionChangeColumn final : public FEditorDataStorageColumn
{
	GENERATED_BODY()

	TSharedPtr<UE::Editor::Outliner::FOnTedsOutlinerSelectionChanged> OnSelectionChanged;
};

/** Column with a delegate that is broadcast to register pending item actions (select/rename/scroll) for a row */
USTRUCT()
struct FTedsOutlinerPendingItemActionsColumn final : public FEditorDataStorageColumn
{
	GENERATED_BODY()

	TSharedPtr<UE::Editor::Outliner::FOnRegisterPendingItemActions> OnRegisterPendingItemActions;
};

/** Column with a map containing boolean settings for a SceneOutliner */
USTRUCT()
struct FTedsSceneOutlinerSettingsCacheColumn final : public FEditorDataStorageColumn
{
	GENERATED_BODY()

	TMap<FName, bool> CachedSettings;
};

