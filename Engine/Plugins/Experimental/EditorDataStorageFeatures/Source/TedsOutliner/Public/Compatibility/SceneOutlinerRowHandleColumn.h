// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SceneOutlinerFwd.h"
#include "ISceneOutliner.h"
#include "ISceneOutlinerColumn.h"
#include "DataStorage/Handles.h"

namespace UE::Editor::DataStorage
{
	class FTedsTableViewerColumn;
}

/**
 * This is a custom column for the TEDS Outliner/Table Viewer to display row handles for items. It is a special cased column instead of going through
 * the TEDS UI layer because row handles are not information stored in a TEDS column
 */
class FSceneOutlinerRowHandleColumn : public ISceneOutlinerColumn
{
public:

	DECLARE_DELEGATE_RetVal_OneParam(UE::Editor::DataStorage::RowHandle, FGetRowHandle, const ISceneOutlinerTreeItem&);
	
	TEDSOUTLINER_API explicit FSceneOutlinerRowHandleColumn(ISceneOutliner& SceneOutliner);
	TEDSOUTLINER_API explicit FSceneOutlinerRowHandleColumn(ISceneOutliner& SceneOutliner, const FGetRowHandle& InGetRowHandle);
	
	virtual ~FSceneOutlinerRowHandleColumn() = default;

	TEDSOUTLINER_API static FName GetID();
	
	//////////////////////////////////////////////////////////////////////////
	// Begin ISceneOutlinerColumn Implementation
	virtual FName GetColumnID() override;
	virtual SHeaderRow::FColumn::FArguments ConstructHeaderRowColumn() override;
	virtual const TSharedRef<SWidget> ConstructRowWidget(FSceneOutlinerTreeItemRef TreeItem, const STableRow<FSceneOutlinerTreeItemPtr>& Row) override;
	virtual void PopulateSearchStrings( const ISceneOutlinerTreeItem& Item, TArray< FString >& OutSearchStrings ) const override;
	virtual bool SupportsSorting() const override { return true; }
	virtual void SortItems(TArray<FSceneOutlinerTreeItemPtr>& OutItems, const EColumnSortMode::Type SortMode) const override;
	// End ISceneOutlinerColumn Implementation
	//////////////////////////////////////////////////////////////////////////

private:

	void CreateWidgetConstructor();
	
private:
	TWeakPtr<ISceneOutliner> WeakSceneOutliner;
	FGetRowHandle GetRowHandle;

	// The Table Viewer column we are going to use internally to create the widget
	TSharedPtr<UE::Editor::DataStorage::FTedsTableViewerColumn> TableViewerColumn;
};
