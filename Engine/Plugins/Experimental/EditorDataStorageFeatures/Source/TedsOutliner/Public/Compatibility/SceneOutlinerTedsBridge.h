// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SceneOutlinerStandaloneTypes.h"
#include "Containers/ContainersFwd.h"
#include "Elements/Interfaces/TypedElementDataStorageFactory.h"
#include "Elements/Interfaces/TypedElementDataStorageInterface.h"
#include "Elements/Interfaces/TypedElementDataStorageUiInterface.h"
#include "Templates/SharedPointer.h"
#include "UObject/ObjectMacros.h"

#include "SceneOutlinerTedsBridge.generated.h"

class ISceneOutliner;

namespace UE::Editor::DataStorage
{
	class ICoreProvider;
	class IUiProvider;
	class ICompatibilityProvider;
} // namespace UE::Editor::DataStorage

UCLASS()
class USceneOutlinerTedsBridgeFactory : public UEditorDataStorageFactory
{
	GENERATED_BODY()

public:
	~USceneOutlinerTedsBridgeFactory() override = default;

	virtual void PreRegister(UE::Editor::DataStorage::ICoreProvider& DataStorage) override;
	virtual void RegisterTables(UE::Editor::DataStorage::ICoreProvider& DataStorage) override;
	virtual void RegisterWidgetPurposes(UE::Editor::DataStorage::IUiProvider& DataStorageUi) const override;

	FName FindOutlinerColumnFromTedsColumns(TConstArrayView<TWeakObjectPtr<const UScriptStruct>> TEDSColumns) const;
	void RemoveTedsColumnMappingToOutlinerColumn(TWeakObjectPtr<const UScriptStruct> TEDSColumnToRemove);
	void AddTedsColumnMappingToOutlinerColumn(TWeakObjectPtr<const UScriptStruct> TEDSColumnToAdd, const FName& OutlinerColumnId);
protected:

	TMap<TWeakObjectPtr<const UScriptStruct>, FName> TEDSToOutlinerDefaultColumnMapping;
};
