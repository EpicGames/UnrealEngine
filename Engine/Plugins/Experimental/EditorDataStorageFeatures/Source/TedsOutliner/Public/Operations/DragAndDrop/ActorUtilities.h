// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataStorage/Handles.h"

class AActor;
class ULevel;
struct FAssetData;
namespace UE::Editor::DataStorage
{
class ICoreProvider;
}

#define UE_API TEDSOUTLINER_API

namespace UE::Editor::DataStorage::Operations::Utilities
{

struct FActorLevelPair
{
	AActor* Actor = nullptr;
	ULevel* Level = nullptr;
};
	
/** Returns the actor assigned to the given row. */
UE_API AActor* GetActorFromRow(const ICoreProvider& Storage, RowHandle Row);

/**
 * Resolves the owning ULevel of an arbitrary drop target row.
 */
UE_API ULevel* GetTargetLevel(const ICoreProvider& Storage, RowHandle TargetRow);

/** Returns the actor and level assigned to the target column of the input row. */
UE_API bool GetTargetActorOrLevel(FActorLevelPair& OutResult, const ICoreProvider& Storage, RowHandle InputRow, FText* OutError);
	
/** Returns the actor and level assigned to the target column of the input row. */
UE_API bool GetTargetActorOrLevelWithAssetValidation(FActorLevelPair& OutResult, const ICoreProvider& Storage, RowHandle InputRow,
	const FAssetData& AssetData, FText* OutError);	

}

#undef UE_API
