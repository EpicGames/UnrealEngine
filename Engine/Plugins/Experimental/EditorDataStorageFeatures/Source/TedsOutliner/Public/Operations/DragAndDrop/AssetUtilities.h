// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataStorage/Handles.h"

class UClass;
class UObject;
struct FAssetData;
namespace UE::Editor::DataStorage
{
class ICoreProvider;
}

#define UE_API TEDSOUTLINER_API

namespace UE::Editor::DataStorage::Operations::Utilities
{

/** Returns the asset class assigned to the source column's row of the given input row. */
UE_API UClass* GetSourceAssetClass(const ICoreProvider& Storage, RowHandle InputRow);

/** Returns the asset data assigned to the source column's row of the given input row. */
UE_API const FAssetData* GetSourceAssetData(const ICoreProvider& Storage, RowHandle InputRow);
	
/** Tests if the given asset data is allowed to be referenced by the target object. */
UE_API bool AssetDataPassesFilter(TNotNull<const UObject*> TargetObject, const FAssetData& AssetData, FText* OutError);

/** Returns the target object. */
UE_API UObject* GetTargetObject(const ICoreProvider& Storage, RowHandle InputRow, FText* OutError);

/** Returns the target object and tests if it passes the asset filter. */
UE_API UObject* GetTargetObjectWithAssetValidation(const ICoreProvider& Storage, RowHandle InputRow, const FAssetData& AssetData, FText* OutError);

/** Calls the LevelEditorDragDropHandler pre-place callback. */
UE_API bool PrePlaceActorValidation(TNotNull<const UObject*> TargetObject, const FAssetData& AssetData);
/** Calls the LevelEditorDragDropHandler post-place callback. */
UE_API void PostPlaceActorValidation(TNotNull<const UObject*> TargetObject, const FAssetData& AssetData);
	
}

#undef UE_API
