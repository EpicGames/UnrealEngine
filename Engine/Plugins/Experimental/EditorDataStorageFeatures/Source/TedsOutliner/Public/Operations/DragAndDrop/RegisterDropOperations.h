// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"

namespace UE::Editor::DataStorage
{
class ICoreProvider;
}

#define UE_API TEDSOUTLINER_API

namespace UE::Editor::DataStorage::Operations
{
	UE_API void RegisterReparentActor(ICoreProvider& Storage);
	UE_API void RegisterReparentFolder(ICoreProvider& Storage);

	inline void RegisterDropOperations(ICoreProvider& Storage)
	{
		RegisterReparentActor(Storage);
		RegisterReparentFolder(Storage);
	}
}

#undef UE_API
