// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"

namespace UE::Editor::DataStorage
{
	/**
	 * Builds a byte-sortable natural-order key from a label string.
	 *
	 * The returned FString has the property that byte-wise lexicographic comparison produces natural ordering (e.g. "Actor2" sorts before "Actor10"). 
	 * This makes the key safe to use both as a HybridSort prefix (via TSortStringView) and in the Compare() tiebreaker, guaranteeing the two paths agree.
	 *
	 */
	TEDSOUTLINER_API FString BuildNaturalSortKey(FStringView Label);

	/**
	 * Byte-wise compare of two BuildNaturalSortKey outputs. Unlike FString::Compare and
	 * FStringView::Compare (both null-terminated via Strcmp/Strncmp),
	 * this respects the full string length and correctly handles the embedded 0x0000 TCHARs emitted by BuildNaturalSortKey (hi-bytes of the length/zeros fields).
	 *
	 * Returns <0 if A sorts before B, >0 if after, 0 if equal. 
	 * On prefix-equal inputs, the shorter key sorts first, matching UE::ComparisonUtility::CompareNaturalOrder.
	 */
	TEDSOUTLINER_API int32 CompareNaturalSortKeys(FStringView A, FStringView B);
}
