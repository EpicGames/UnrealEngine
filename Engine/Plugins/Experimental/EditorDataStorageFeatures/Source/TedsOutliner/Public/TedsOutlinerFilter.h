// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EDITOR

#include <type_traits>
#include "Filters/FilterBase.h"
#include "SceneOutlinerPublicTypes.h"
#include "TedsFilter.h"
#include "Elements/Framework/TypedElementQueryBuilder.h"

namespace UE::Editor::Outliner
{
	using namespace UE::Editor::DataStorage::Queries;
	class FTedsOutlinerImpl;
		
	class FTedsOutlinerFilter : public FTedsFilterBase<SceneOutliner::FilterBarType>
	{
		friend class FTedsOutlinerImpl;
		friend class FTedsOutlinerMode;
		
	public:
		/**
		 * Constructor for a TEDS Outliner Filter
		 * @param InFilterName FName Identifier of the Filter
		 * @param InFilterDisplayName Display Name to show on the Add Filter Menu and toggle button
		 * @param InFilterToolTip Tooltip to describe the function of the Filter
		 * @param InFilterIconName Name of an icon in the App Style Set to display in the Add Filter Menu
		 * @param InCategory Group to place the filter in the Add Filter Menu
		 * @param InFilterQuery QueryHandle of a registered query to filter for (must be the same syntax as the Outliner Query)
		 * @param bInteractiveFilter Whether this filter is visible in the Add Filter Menu and toggleable or is hidden and enabled by default
		 * @param bActiveByDefault Whether this filter should be active by default and turned on when the SceneOutliner is Init
		 */
		TEDSOUTLINER_API FTedsOutlinerFilter(const FName& InFilterName, const FText& InFilterDisplayName, const FText& InFilterToolTip, const FName& InFilterIconName, 
			const TSharedPtr<FFilterCategory>& InCategory, const QueryHandle& InFilterQuery, const bool bInteractiveFilter = true, const bool bActiveByDefault = false);

		/**
		 * Constructor for a TEDS Outliner Filter
		 * @param InFilterName FName Identifier of the Filter
		 * @param InFilterDisplayName Display Name to show on the Add Filter Menu and toggle button
		 * @param InFilterQuery QueryHandle of a registered query to filter for (must be the same syntax as the Outliner Query)
		 * @param bInteractiveFilter Whether this filter is visible in the Add Filter Menu and toggleable or is hidden and enabled by default
		 * @param bActiveByDefault Whether this filter should be active by default and turned on when the SceneOutliner is Init
		 */
		TEDSOUTLINER_API FTedsOutlinerFilter(const FName& InFilterName, const FText& InFilterDisplayName, const QueryHandle& InFilterQuery,
			const bool bInteractiveFilter = true, const bool bActiveByDefault = false);

		/**
		 * Constructor for a TEDS Outliner Filter
		 * @param InFilterName FName Identifier of the Filter
		 * @param InFilterDisplayName Display Name to show on the Add Filter Menu and toggle button
		 * @param InFilterToolTip Tooltip to describe the function of the Filter
		 * @param InFilterIconName Name of an icon in the App Style Set to display in the Add Filter Menu
		 * @param InCategory Group to place the filter in the Add Filter Menu
		 * @param InFilterQuery QueryFunction to filter for, Batching should be used for improved performance if possible
		 * @param bInteractiveFilter Whether this filter is visible in the Add Filter Menu and toggleable or is hidden and enabled by default
		 * @param bActiveByDefault Whether this filter should be active by default and turned on when the SceneOutliner is Init
		 */
		TEDSOUTLINER_API FTedsOutlinerFilter(const FName& InFilterName, const FText& InFilterDisplayName, const FText& InFilterToolTip, const FName& InFilterIconName,
			const TSharedPtr<FFilterCategory>& InCategory, const TConstQueryFunction<bool>& InFilterQuery, const bool bInteractiveFilter = true, const bool bActiveByDefault = false);

		/**
		 * Constructor for a TEDS Outliner Filter
		 * @param InFilterName FName Identifier of the Filter
		 * @param InFilterDisplayName Display Name to show on the Add Filter Menu and toggle button
		 * @param InFilterQuery QueryFunction to filter for, Batching should be used for improved performance if possible
		 * @param bInteractiveFilter Whether this filter is visible in the Add Filter Menu and toggleable or is hidden and enabled by default
		 * @param bActiveByDefault Whether this filter should be active by default and turned on when the SceneOutliner is Init
		 */
		TEDSOUTLINER_API FTedsOutlinerFilter(const FName& InFilterName, const FText& InFilterDisplayName, const TConstQueryFunction<bool>& InFilterQuery,
			const bool bInteractiveFilter = true, const bool bActiveByDefault = false);
		
		/**
		 * Constructor for a TEDS Outliner Filter
		 * Require that the template type cannot be one of the pre-defined FilterQuery types so it will pass to the defined
		 * constructor instead of this one.
		 * @param InFilterName FName Identifier of the Filter
		 * @param InFilterDisplayName Display Name to show on the Add Filter Menu and toggle button
		 * @param InFilterToolTip Tooltip to describe the function of the Filter
		 * @param InFilterIconName Name of an icon in the App Style Set to display in the Add Filter Menu
		 * @param InCategory Group to place the filter in the Add Filter Menu
		 * @param InFilterQuery Function callback that checks the value of a column(s) and returns a bool of whether that row passes the filter
		 * @param bInteractiveFilter Whether this filter is visible in the Add Filter Menu and toggleable or is hidden and enabled by default
		 * @param bActiveByDefault Whether this filter should be active by default and turned on when the SceneOutliner is Init
		 */
		template< typename FilterFunction UE_REQUIRES(
			!std::is_same_v<std::remove_cvref_t<FilterFunction>, QueryHandle> &&
			!std::is_same_v<std::remove_cvref_t<FilterFunction>, Queries::TConstQueryFunction<bool>>) >
		FTedsOutlinerFilter(const FName& InFilterName, const FText& InFilterDisplayName, const FText& InFilterToolTip, const FName& InFilterIconName,
			const TSharedPtr<FFilterCategory>& InCategory, FilterFunction&& InFilterQuery, const bool bInteractiveFilter = true, const bool bActiveByDefault = false)
			: FTedsOutlinerFilter(
				InFilterName, 
				InFilterDisplayName, 
				InFilterToolTip, 
				InFilterIconName, 
				InCategory,
				Queries::BuildConstQueryFunction<bool>(Forward<FilterFunction>(InFilterQuery)),
				bInteractiveFilter,
				bActiveByDefault)
		{}

		/**
		 * Constructor for a TEDS Outliner Filter
		 * Require that the template type cannot be one of the pre-defined FilterQuery types so it will pass to the defined
		 * constructor instead of this one.
		 * @param InFilterName FName Identifier of the Filter
		 * @param InFilterDisplayName Display Name to show on the Add Filter Menu and toggle button
		 * @param InFilterQuery Function callback that checks the value of a column(s) and returns a bool of whether that row passes the filter
		 * @param bInteractiveFilter Whether this filter is visible in the Add Filter Menu and toggleable or is hidden and enabled by default
		 * @param bActiveByDefault Whether this filter should be active by default and turned on when the SceneOutliner is Init
		 */
		template< typename FilterFunction UE_REQUIRES(
			!std::is_same_v<std::remove_cvref_t<FilterFunction>, QueryHandle> &&
			!std::is_same_v<std::remove_cvref_t<FilterFunction>, Queries::TConstQueryFunction<bool>>) >
		FTedsOutlinerFilter(const FName& InFilterName, const FText& InFilterDisplayName, FilterFunction&& InFilterQuery,
			const bool bInteractiveFilter = true, const bool bActiveByDefault = false)
			: FTedsOutlinerFilter(
				InFilterName, 
				InFilterDisplayName, 
				FText::FromString(InFilterName.ToString()), 
				FName(), 
				nullptr,
				Queries::BuildConstQueryFunction<bool>(Forward<FilterFunction>(InFilterQuery)),
				bInteractiveFilter,
				bActiveByDefault)
		{}

		/**
		 * Constructor for a TEDS Outliner Class Filter
		 * @param InClass UClass to filter for on the TypeInfo Column
		 * @param InCategory Group to place the filter in the Add Filter Menu
		 * @param bInteractiveFilter Whether this filter is visible in the Add Filter Menu and toggleable or is hidden and enabled by default
		 * @param bActiveByDefault Whether this filter should be active by default and turned on when the SceneOutliner is Init
		 */
		TEDSOUTLINER_API FTedsOutlinerFilter(const UClass* InClass, const TSharedPtr<FFilterCategory>& InCategory = nullptr,
			const bool bInteractiveFilter = true, const bool bActiveByDefault = false);

		bool IsInteractiveFilter() const { return bInteractiveFilter; }

		/** Notification that the filter became active or inactive */
		virtual TEDSOUTLINER_API void ActiveStateChanged(bool bActive) override;
		
		/** Returns whether the specified Item passes the Filter's restrictions */
		virtual TEDSOUTLINER_API bool PassesFilter(SceneOutliner::FilterBarType InItem ) const override;

	protected:
		/** Sets a SceneOutlinerImpl to be used by this Filter */
		void SetSceneOutlinerImpl(const TSharedPtr<FTedsOutlinerImpl>& InTedsOutlinerImpl);
		
		const bool bInteractiveFilter;
		
		TWeakPtr<FTedsOutlinerImpl> TedsOutlinerImpl;
	};
} // namespace UE::Editor::Outliner

#endif
