// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SceneOutlinerFwd.h"
#include "EditorActorFolders.h"

class AActor;
class SSceneOutliner;
class UWorld;

namespace UE::Editor::DataStorage
{
	class ICoreProvider;
}

// Helper functions for TEDS Outliner folder
namespace UE::Editor::Outliner::Helpers
{
	// Selects descendant actors via a single batched GEditor selection operation.
	TEDSOUTLINER_API void SelectDescendantActors(SSceneOutliner& Outliner, const TArray<FSceneOutlinerTreeItemPtr>& Roots, bool bSelectImmediateChildrenOnly);

	// Creates a new folder
	TEDSOUTLINER_API FFolder CreateFolder(DataStorage::ICoreProvider& Storage, const SSceneOutliner& Outliner, UWorld& World, const FSceneOutlinerTreeItemPtr& ParentFolder = nullptr);

	// Creates a new folder for the current selection
	TEDSOUTLINER_API FFolder CreateFolderForSelection(DataStorage::ICoreProvider& Storage, const SSceneOutliner& Outliner, UWorld& World, const TArray<FSceneOutlinerTreeItemPtr>& SelectedFolderItems);
}
