// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataStorage/Handles.h"
#include "Elements/Interfaces/TypedElementDataStorageInterface.h"
#include "Filters/FilterBase.h"
#include "SceneOutlinerFwd.h"
#include "Columns/TedsOutlinerColumns.h"
#include "Elements/Interfaces/TypedElementDataStorageCompatibilityInterface.h"

namespace UE::Editor::Outliner
{
	class FTedsOutlinerFilter;
}

class FTedsOutlinerColumnDescription;

// Helper functions for Teds Outliner related functionality
namespace UE::Editor::Outliner::Helpers
{
	// Get the internal Outliner tree item from the row handle from the specific Outliner
	TEDSOUTLINER_API FSceneOutlinerTreeItemPtr GetTreeItemFromRowHandle(const DataStorage::ICoreProvider* Storage, TSharedRef<ISceneOutliner> InOutliner, DataStorage::RowHandle InRowHandle);

	// Register a dealiaser for the given outliner, overriding any previously registered ones.
	// Returns false if the registration failed, mostly commonly because the Outliner doesn't have a unique OutlinerIdentifier specificed
	TEDSOUTLINER_API bool RegisterOutlinerDealiaser(DataStorage::ICoreProvider* Storage, TSharedRef<ISceneOutliner> InOutliner, const FTreeItemIDDealiaser& InDealiaser);

	// Get the name of the Teds table that all Outliners are stored in
	TEDSOUTLINER_API FName GetTedsOutlinerTableName();
	
	// Refresh all the Outliners currently open in the level editor
	TEDSOUTLINER_API void RefreshLevelEditorOutliners();

	// Orders a given array of Columns within the same priority group and relations defined by the
	// ColumnDescription and returns evaluated priorities to the OutOrderedColumnMap. Returns true if the ordering was successful
	bool TEDSOUTLINER_API OrderColumnGroup(const TArray<TWeakObjectPtr<const UScriptStruct>>& InColumnGroup,
    	const FTedsOutlinerColumnDescription& InColumnDescription, const uint8 InDefaultStartingPriorityIndex,
    	TMap<TWeakObjectPtr<const UScriptStruct>, uint8>& OutOrderedColumnMap);

	// Orders a given array of Columns using priority groups and relations defined by the
	// ColumnDescription and returns evaluated priorities to the OutOrderedColumnMap. Returns true if the ordering was successful
	bool TEDSOUTLINER_API OrderColumns(const TArray<TWeakObjectPtr<const UScriptStruct>>& InColumns,
		const FTedsOutlinerColumnDescription& InColumnDescription, TMap<TWeakObjectPtr<const UScriptStruct>, uint8>& OutOrderedColumnMap);

	// Returns a TAttribute<FText> bound to FHighlightTextColumn on the parent table viewer row.
	// Returns an empty FText attribute if the column is not present (safe no-op in non-outliner contexts).
	TAttribute<FText> TEDSOUTLINER_API GetHighlightTextAttribute(DataStorage::ICoreProvider* DataStorage, DataStorage::RowHandle WidgetRow);
	
	// Get the name of the Outliner column corresponding to the given TEDS column (if any)
	FName FindOutlinerColumnFromTedsColumns(const DataStorage::ICoreProvider* Storage, TConstArrayView<TWeakObjectPtr<const UScriptStruct>> TEDSColumns);

	// Remove a Teds Column from the TEDSToOutlinerDefaultColumnMapping
	void RemoveTedsColumnMappingToOutlinerColumn(DataStorage::ICoreProvider* Storage, TWeakObjectPtr<const UScriptStruct> TEDSColumnToRemove);
	// Add a Teds Column to the TEDSToOutlinerDefaultColumnMapping
	void AddTedsColumnMappingToOutlinerColumn(DataStorage::ICoreProvider* Storage, TWeakObjectPtr<const UScriptStruct> TEDSColumnToAdd, const FName& OutlinerColumnId);
	
	// Helper function to check if the given QueryHandle is valid to be used in a filter (is not an observer).
    bool CheckValidFilterQueryHandle(const DataStorage::QueryHandle& InQueryHandle);

	// Helper function to convert legacy filters to Teds counterparts. All class filters will be automatically converted, any custom filters
	// will need to have a mapped filter function counterpart to be converted successfully. OutFilterCategoryMap is populated with the additional
	// categories for class filters that belonged to multiple categories in the legacy system.
	void TEDSOUTLINER_API ConvertLegacyFiltersToTedsFilters(FSceneOutlinerFilterBarOptions& LegacyFilters, TArray<TSharedPtr<FTedsOutlinerFilter>>& TedsFilters,
		const TMap<FName, TVariant<DataStorage::QueryHandle, DataStorage::Queries::TConstQueryFunction<bool>>>& FilterConversionMap,
		TMap<FName, TArray<TSharedPtr<FFilterCategory>>>& OutTypeFilterCategoryMap);

	// Helper function to get the Outliner Display color for a text/icon item label widget
	TAttribute<FSlateColor> GetLabelWidgetForegroundColor(TNotNull<DataStorage::ICoreProvider*> DataStorage, DataStorage::RowHandle DataRow, 
		DataStorage::RowHandle WidgetRow);

	// Helper function to resolve the row handle from a SceneOutlinerTreeItem
	DataStorage::RowHandle GetRowHandleFromOutlinerItem(DataStorage::ICoreProvider& DataStorage, const DataStorage::ICompatibilityProvider& Compatibility, const DataStorage::RowHandle OutlinerRow, const ISceneOutlinerTreeItem& Item);


	namespace LevelInstance
	{
		/**
		 * Walk the EditorObjectHierarchy ancestors of InRow until a row carrying FLevelInstanceTag is found
		 */
		TEDSOUTLINER_API DataStorage::RowHandle FindContainingLevelInstanceRow(const DataStorage::ICoreProvider* DataStorage, DataStorage::RowHandle InRow);

		/**
		 * Returns true if any EditorObjectHierarchy ancestor of InRow carries FLevelInstanceEditingColumn
		 */
		TEDSOUTLINER_API bool IsInEditingLevelInstanceHierarchy(const DataStorage::ICoreProvider* DataStorage, DataStorage::RowHandle InRow);
	}
}

