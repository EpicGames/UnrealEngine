// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HierarchyViewerIntefaces.h"
#include "ISceneOutliner.h"
#include "DataStorage/Handles.h"
#include "Elements/Columns/TypedElementHiearchyColumns.h"

#define UE_API TEDSOUTLINER_API

namespace UE::Editor::Outliner
{
	/* Base class for all hierarchy data interfaces used by the Teds Outliner to display hierarchies in the UI.
	 * Teds Outliner hierarchies have the additional step of registering queries to detect changes to hierarchies when compared to the hierarchy viewer
	 */
	class ITedsOutlinerHierarchyDataInterface : public DataStorage::IHierarchyViewerDataInterface
	{
	public:
		virtual ~ITedsOutlinerHierarchyDataInterface() override = default;
		
		/** 
		 * Register any queries required by this data interface
		 * @param Storage Reference to the editor data storage interface
		 * @param OutlinerQueryDescription The query description used to gather rows in the outliner, can be merged with any local queries 
		 *								   to filter down on the rows observed to only those in the outliner
		 * @param Outliner Ptr to the Scene Outliner owning this interface
		 * @param bUsingQueryConditionsSyntax Whether the owning outliner query description is using the new query condition syntax or not
		 *									  Useful for query merge which requires both input queries to be using the same syntax
		 */
		UE_API virtual void RegisterQueries(DataStorage::ICoreProvider& Storage, const DataStorage::FQueryDescription& OutlinerQueryDescription, 
			TWeakPtr<ISceneOutliner> Outliner, bool bUsingQueryConditionsSyntax) = 0;
		
		// Unregister any queries required by this data interface
		UE_API virtual void UnregisterQueries(DataStorage::ICoreProvider& Storage) = 0;
		
		using FParentIterationCallback = TFunction<bool(const DataStorage::ICoreProvider& Context, DataStorage::RowHandle Parent)>;
		
		// Iterate over the immediate parents of InRow. In the simple case this is just one parent, but if this is a complex hierarchy combining multiple
		// individual ones there could be multiple immediate parents
		UE_API virtual void ForEachImmediateParent(const DataStorage::ICoreProvider& Storage, DataStorage::RowHandle InRow, FParentIterationCallback Callback);
	};
	
	// Hierarchy Data interface that can be used to combine multiple hierarchies into one view
	class FTedsOutlinerMultiHierarchyInterface : public ITedsOutlinerHierarchyDataInterface
	{
	public:
		// @see FHierarchyViewerMultiData, which is used to specify the ordered list of hierarchies
		UE_API explicit FTedsOutlinerMultiHierarchyInterface(TSharedRef<DataStorage::FHierarchyViewerMultiData> InHierarchyData);
		
		UE_API virtual DataStorage::RowHandle GetParent(const DataStorage::ICoreProvider& Storage, DataStorage::RowHandle InRow) const override;
		
		UE_API virtual void WalkDepthFirst(const DataStorage::ICoreProvider& Storage, DataStorage::RowHandle InRow, 
			DataStorage::ICoreProvider::FHierarchyIterationCallback VisitFn,
			DataStorage::ICoreProvider::ETraversalOrder TraversalOrder = DataStorage::ICoreProvider::ETraversalOrder::PreOrder) const override;
		
		UE_API virtual void RegisterQueries(DataStorage::ICoreProvider& Storage, const DataStorage::FQueryDescription& OutlinerQueryDescription,
			TWeakPtr<ISceneOutliner> Outliner, bool bUsingQueryConditionsSyntax) override;
		
		UE_API virtual void UnregisterQueries(DataStorage::ICoreProvider& Storage) override;
		
		UE_API virtual void ForEachImmediateParent(const DataStorage::ICoreProvider& Storage, DataStorage::RowHandle InRow, FParentIterationCallback Callback) override;
		
	protected:
		TSharedRef<DataStorage::FHierarchyViewerMultiData> HierarchyData;
		TArray<DataStorage::QueryHandle> HierarchyChangeQueries;
	};
	
	/**
	  * @section Legacy Hierarchy Interfaces
	  * These classes exist for backward compatibility with existing uses of the Teds Outliner which involve custom hierarchies instead of the new
	  * TEDS hierarchies feature. It is NOT recommended to use these anymore, and instead to use the hierarchy API on ICoreProvider to set your hierarchy
	  * alongside the new data interfaces above. 
	  */
	
	// Legacy struct storing information on how hierarchies are handled in the TEDS Outliner
    struct FTedsOutlinerHierarchyData
    {
    	/** A delegate used to get the parent row handle for a given row */
    	DECLARE_DELEGATE_RetVal_OneParam(DataStorage::RowHandle, FGetParentRowHandle, const void* /* InColumnData */);
    
    	/** A delegate used to get the children row handles for a given row */
    	DECLARE_DELEGATE_RetVal_OneParam(TArrayView<DataStorage::RowHandle>, FGetChildrenRowsHandles, void* /* InColumnData */);
    	
    	/** A delegate used to set the parent row handle for a given row */
    	DECLARE_DELEGATE_TwoParams(FSetParentRowHandle, void* /* InColumnData */, DataStorage::RowHandle /* InParentRowHandle */);
    
    	FTedsOutlinerHierarchyData(const UScriptStruct* InHierarchyColumn, const FGetParentRowHandle& InGetParent, const FSetParentRowHandle& InSetParent, const FGetChildrenRowsHandles& InGetChildren)
    		: HierarchyColumn(InHierarchyColumn)
    		, GetParent(InGetParent)
    		, GetChildren(InGetChildren)
    		, SetParent(InSetParent)
    	{
    	
    	}
    
    	// The column that contains the parent row handle for rows
    	const UScriptStruct* HierarchyColumn;
    
    	// Function to get parent row handle
    	FGetParentRowHandle GetParent;
    
    	// Function to get the children row handle
    	FGetChildrenRowsHandles GetChildren;
    
    	// Function to set the parent row handle
    	FSetParentRowHandle SetParent;
    	
    	// Get the default hierarchy data for the TEDS Outliner that uses FTableRowParentColumn to get the parent
    	static FTedsOutlinerHierarchyData GetDefaultHierarchyData();
    };
	
	// Legacy data interface for displaying custom hierarchies in the Teds Outliner
	class FTedsOutlinerLegacyHierarchyInterface : public ITedsOutlinerHierarchyDataInterface
	{
	public:
		UE_API explicit FTedsOutlinerLegacyHierarchyInterface(FTedsOutlinerHierarchyData InHierarchyData);
		UE_API virtual DataStorage::RowHandle GetParent(const DataStorage::ICoreProvider& Storage, DataStorage::RowHandle InRow) const override;
		UE_API virtual void WalkDepthFirst(const DataStorage::ICoreProvider& Storage, DataStorage::RowHandle InRow, DataStorage::ICoreProvider::FHierarchyIterationCallback VisitFn,
											DataStorage::ICoreProvider::ETraversalOrder TraversalOrder = DataStorage::ICoreProvider::ETraversalOrder::PreOrder) const override;
		UE_API virtual void RegisterQueries(DataStorage::ICoreProvider& Storage, const DataStorage::FQueryDescription& OutlinerQueryDescription, TWeakPtr<ISceneOutliner> Outliner, bool bUsingQueryConditionsSyntax) override;
		UE_API virtual void UnregisterQueries(DataStorage::ICoreProvider& Storage) override;
		
	protected:
		FTedsOutlinerHierarchyData HierarchyData;
		DataStorage::QueryHandle HierarchyChangeQuery = DataStorage::InvalidQueryHandle;
		DataStorage::QueryHandle GetChildrenQuery = DataStorage::InvalidQueryHandle;
	};
}

#undef UE_API