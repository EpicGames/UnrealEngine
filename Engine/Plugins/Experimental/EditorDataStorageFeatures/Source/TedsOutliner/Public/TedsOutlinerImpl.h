// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/Interfaces/TypedElementDataStorageInterface.h"
#include "ISceneOutlinerHierarchy.h"
#include "ISceneOutlinerMode.h"
#include "Columns/TedsOutlinerColumns.h"
#include "Elements/Columns/TypedElementHiearchyColumns.h"
#include "Elements/Framework/TypedElementRowHandleArray.h"
#include "Elements/Framework/TypedElementRowHandleArrayView.h"
#include "Elements/Interfaces/TypedElementDataStorageUiInterface.h"
#include "Elements/Framework/TypedElementQueryBuilder.h"
#include "Elements/Framework/TypedElementSelectionSet.h"
#include "Framework/MultiBox/MultiBoxExtender.h"
#include "SceneOutlinerPublicTypes.h"
#include "TedsOutlinerFilter.h"
#include "TedsOutlinerHierarchyInterfaces.h"

#define UE_API TEDSOUTLINER_API

namespace UE::Editor::DataStorage
{
	namespace QueryStack
	{
		class FRowOrderInversionNode;
		class FRowSortNode;
		class FRowHandleSortNode;
		class FRowQueryResultsNode;
		class IQueryNode;
		class FRowMonitorNode;
		class FRowMergeNode;
		class FRowFilterNode;
		class IRowNode;
		class FExplicitUpdateExecutor;
	}

	class IUiProvider;
	class ICompatibilityProvider;
} // namespace UE::Editor::DataStorage

namespace UE::Editor::Outliner
{
	class FRowMapNode;
}

struct FTypedElementWidgetConstructor;
class SWidget;
class SHorizontalBox;

namespace UE::Editor::Outliner
{
	DECLARE_DELEGATE_OneParam(FOnExtendViewMenu, TSharedPtr<FExtender>);
	DECLARE_DELEGATE_RetVal_ThreeParams(FReply, FOnOutlinerItemDoubleClick, ICoreProvider*, DataStorage::RowHandle /* Outliner */, DataStorage::RowHandle /* Item */);
	DECLARE_DELEGATE_OneParam(FOnExtendOptionsMenu, FMenuBuilder&);
	DECLARE_DELEGATE_RetVal_TwoParams(bool, FOnCanPopulate, ICoreProvider*, DataStorage::RowHandle /* Outliner */);
	DECLARE_DELEGATE_TwoParams(FOnCustomAddToToolbar, TSharedPtr<SHorizontalBox> /* Toolbar */, DataStorage::RowHandle /* Outliner */);
	DECLARE_DELEGATE_OneParam(FOnRegisterInteractiveFilters, SSceneOutliner&);

struct FTedsOutlinerColumnParams
{
	// Defines the priority groups, where Left priority is ordered first and Right priority is ordered last
	// ||  Left  |  Middle  |  Right  ||
	enum class TEDSOUTLINER_API EColumnPriorityGroup
	{
		Middle = 0, // Initial Priority value of 80
		Left = 1 << 0, // Initial Priority value of 0
		Right = 1 << 1 // Initial Priority value of 160
	};

	// Defines possible relations between columns so they can specify the desired ordering
	struct FColumnPriorityRelation
	{
		enum class TEDSOUTLINER_API EColumnPriorityRelation
		{
			None = 0,
			Before = 1 << 0, // This column comes before the specified RelatedColumn
			After = 1 << 1 // This column comes after the specified RelatedColumn
		};
		
		explicit TEDSOUTLINER_API FColumnPriorityRelation(const EColumnPriorityRelation InPriorityRelation = EColumnPriorityRelation::None,
			const TWeakObjectPtr<const UScriptStruct>& InRelatedColumn = nullptr);

		// When determining priority, all columns with Before relations are sorted before columns with After Columns
		EColumnPriorityRelation PriorityRelation;
		TWeakObjectPtr<const UScriptStruct> RelatedColumn;
	};

	explicit TEDSOUTLINER_API FTedsOutlinerColumnParams(
		const ESceneOutlinerColumnVisibility InInitialVisibility = ESceneOutlinerColumnVisibility::Visible,
		const EColumnPriorityGroup InPriorityGroup = EColumnPriorityGroup::Middle,
		const FColumnPriorityRelation& InPriorityRelation = FColumnPriorityRelation());

	explicit TEDSOUTLINER_API FTedsOutlinerColumnParams(
		const EColumnPriorityGroup InPriorityGroup,
		const FColumnPriorityRelation& InPriorityRelation = FColumnPriorityRelation());

	// Visibility State of this column when the Outliner is Initialized (can change the visibility via the Header right-click menu)
	ESceneOutlinerColumnVisibility InitialVisibility;
	// Priority Group this column belongs to, it will sort by relations in this group and have an adjusted priority value based on its group
	EColumnPriorityGroup PriorityGroup;
	// Specifies if this column should appear before or after another column (must be in the same priority group). Without this, order
	// is defined by what Column is passed the Column Array first
	FColumnPriorityRelation PriorityRelation;
};
	
struct FTedsOutlinerColumnDescription
{
	TEDSOUTLINER_API FTedsOutlinerColumnDescription(
		const TArray<TWeakObjectPtr<const UScriptStruct>>& InColumns = {},
		const TMap<TWeakObjectPtr<const UScriptStruct>, FTedsOutlinerColumnParams>& InColumnParams = {},
		const TArray<TSharedPtr<FColumnMetaData>>& InColumnMetaData = {},
		const FMetaData& InGenericMetaData = FMetaData());

	TEDSOUTLINER_API FTedsOutlinerColumnDescription(
		const TArray<TWeakObjectPtr<const UScriptStruct>>& InColumns, const FMetaData& InGenericMetaData);

	const TArray<TWeakObjectPtr<const UScriptStruct>>& GetColumns() const;

	// Returns GetColumns() with ExcludedColumns filtered out.
	TEDSOUTLINER_API TArray<TWeakObjectPtr<const UScriptStruct>> GetEffectiveColumns() const;

	const FTedsOutlinerColumnParams* FindColumnParams(const TWeakObjectPtr<const UScriptStruct>& InColumn) const;
	FTedsOutlinerColumnParams& FindOrAddColumnParams(const TWeakObjectPtr<const UScriptStruct>& InColumn);
	const TArray<TSharedPtr<FColumnMetaData>>& GetColumnMetaData() const;
	FMetaData GetGenericMetaData() const;

	// Mark columns as excluded.
	TEDSOUTLINER_API bool ExcludeColumns(TConstArrayView<TWeakObjectPtr<const UScriptStruct>> InColumns);

	// Re-include columns that were previously excluded.
	TEDSOUTLINER_API bool IncludeColumns(TConstArrayView<TWeakObjectPtr<const UScriptStruct>> InColumns);

	const TSet<TWeakObjectPtr<const UScriptStruct>>& GetExcludedColumns() const;

protected:
	// Columns to display in the Outliner
	TArray<TWeakObjectPtr<const UScriptStruct>> Columns;
	// Customization Parameters for a given column
	TMap<TWeakObjectPtr<const UScriptStruct>, FTedsOutlinerColumnParams> ColumnParams;
	// Metadata for Columns (ReadOnly/ReadWrite), by default the constructor will populate this with ReadOnly Metadata for all columns
	TArray<TSharedPtr<FColumnMetaData>> ColumnMetaData;
	// Generic meta data for the TEDS (i.e. GenericMetaData.AddOrSetMutableData("bAllowSorting", false))
	FMetaData GenericMetaData;
	// Columns currently suppressed. Honored by FTedsOutlinerImpl::RegenerateColumns; the Columns
	// array and ColumnParams map are never mutated when a column is excluded.
	TSet<TWeakObjectPtr<const UScriptStruct>> ExcludedColumns;
};
	
UE_EXPERIMENTAL(5.8, "This struct is a temporary bridge for expansion state introp until we have persistent expansion state in TEDS")
struct FTedsOutlinerExpansionStateBridge
{
	TFunction<bool(RowHandle)> GetExpansionState;
	TFunction<void(RowHandle, bool)> SetExpansionState;
};

struct FTedsOutlinerParams
{
	TEDSOUTLINER_API FTedsOutlinerParams(SSceneOutliner* InSceneOutliner);

	SSceneOutliner* SceneOutliner = nullptr;

	// The query description that will be used to populate rows in the TEDS Outliner
	DataStorage::FQueryDescription QueryDescription;

	// The columns to display in the TEDS Outliner and any additional metadata
	FTedsOutlinerColumnDescription ColumnDescription;
	
	// TEDS Filters that utilize QueryDescriptions or QueryFunctions in this TEDS Outliner
	TArray<TSharedPtr<FTedsOutlinerFilter>> Filters;

	// Additional categories for type filters. The same filter instance will appear under all categories in 
	// the "Add Filter" menu, sharing its active state.
	TMap<FName, TArray<TSharedPtr<FFilterCategory>>> FilterCategoryMap;

	// TEDS Filters that populate the Show section of the settings menu: "Hide Components", etc.
	TArray<TSharedPtr<FTedsOutlinerFilter>> ShowOptionsFilters;

	// Delegate for extending the View Menu with additional options
	FOnExtendViewMenu OnInitializeViewMenuExtender;

	// Delegate for adding entries into the "Options" section of the View Menu
	FOnExtendOptionsMenu OnExtendOptionsMenu;

	// Delegate called before the default double-click behavior. Return true if handled.
	FOnOutlinerItemDoubleClick OnItemDoubleClick;
	
	// Expansion state bridge (temporary)
	FTedsOutlinerExpansionStateBridge ExpansionStateBridge;

	// Delegate used by the SSceneOutliner to decide whether to populate
	FOnCanPopulate OnCanPopulate;

	// Delegate fired by the mode's CustomAddToToolbar - extender adds widgets to the outliner toolbar.
	FOnCustomAddToToolbar OnCustomAddToToolbar;

	// Delegate invoked once after the outliner is constructed so callers can register interactive filters via SSceneOutliner::AddInteractiveFilter.
	FOnRegisterInteractiveFilters OnRegisterInteractiveFilters;

	// If true, this Outliner will include a column for row handle
	bool bShowRowHandleColumn;

	// If true, parent nodes will remain visible if a child passes all filters even if the parent fails a filter
	bool bForceShowParents;

	// If true, the Teds Outliner will create observers to track addition/removal of rows and update the Outliner
	bool bUseDefaultObservers;
	
	// If true, the Teds Outliner will create a view button (settings cog) pre-populated with hierarchy actions
	bool bShowViewButton = false;

	// If specified, this is how the TEDS Outliner will handle hierarchies. If not specified - there will be no hierarchies shown as a
	// parent-child relation in the tree view
	TSharedPtr<ITedsOutlinerHierarchyDataInterface> HierarchyData;

	// The selection set to use for this Outliner, nullptr = don't propagate tree selection through TypedElementSelectionInterface
	UTypedElementSelectionSet* SelectionSet;

	// The purpose to use when generating widgets for row/column pairs through TEDS UI
	DataStorage::IUiProvider::FPurposeID CellWidgetPurpose;

	// The purpose to use when generating widgets for column headers through TEDS UI
	DataStorage::IUiProvider::FPurposeID HeaderWidgetPurpose;

	// The purpose to use when generating widgets for the "Item Label" column through TEDS UI
	DataStorage::IUiProvider::FPurposeID LabelWidgetPurpose;
	
	// Additional queries that the TEDS Outliner will use to add or remove certain rows when the query is run.
	// NOTE: The callback function for these queries will be overriden by the Outliner.
	TOptional<FQueryDescription> AddObserverQuery;
	TOptional<FQueryDescription> RemoveObserverQuery;
};


// This class is meant to be a model to hold functionality to create a "table viewer" in TEDS that can be
// attached to any view/UI.
// TEDS-Outliner TODO: This class still has a few outliner implementation details leaking in that should be removed
class FTedsOutlinerImpl : public TSharedFromThis<FTedsOutlinerImpl>
{

public:
	// Helper function to create a label widget for a given row
	static UE_API TSharedRef<SWidget> CreateLabelWidget(
		DataStorage::ICoreProvider& Storage,
		DataStorage::IUiProvider& StorageUi,
		const DataStorage::IUiProvider::FPurposeID& Purpose,
		DataStorage::RowHandle TargetRow,
		const DataStorage::RowHandle OutlinerWidgetRow,
		ISceneOutlinerTreeItem& TreeItem,
		const STableRow<FSceneOutlinerTreeItemPtr>& RowItem,
		bool bInteractable = true);

	UE_API FTedsOutlinerImpl(const FTedsOutlinerParams& InParams, ISceneOutlinerMode* InMode, bool bInHybridMode);
	UE_API virtual ~FTedsOutlinerImpl();

	UE_API void Init();
	UE_API void FullRefresh() const;
	UE_API void OnMonitoredRowsAdded(DataStorage::FRowHandleArrayView InRows);
	UE_API void OnMonitoredRowsRemoved(DataStorage::FRowHandleArrayView InRows);

	// TEDS construct getters
	UE_API DataStorage::ICoreProvider* GetStorage() const;
	UE_API DataStorage::IUiProvider* GetStorageUI() const;
	UE_API DataStorage::ICompatibilityProvider* GetStorageCompatibility() const;

	UE_API UTypedElementSelectionSet* GetSelectionSet() const;

	// Delegate fired when the selection changes, only if SelectionSet is set
	UE_API FOnTedsOutlinerSelectionChanged& OnSelectionChanged();

	// Set the selection type to broadcast with the next OnSelectionChanged event
	UE_API void SetPendingSelectionType(ESelectInfo::Type InSelectionType);

	// Delegate fired when the hierarchy changes due to item addition/removal/move
	UE_API ISceneOutlinerHierarchy::FHierarchyChangedEvent& OnHierarchyChanged();

	// Delegate to check if a certain outliner item is compatible with this TEDS Outliner Impl - set by the system using FTedsOutlinerImpl
	DECLARE_DELEGATE_RetVal_OneParam(bool, FIsItemCompatible, const ISceneOutlinerTreeItem&)
	UE_API FIsItemCompatible& IsItemCompatible();

	// Update the selection to the input rows, only if SelectionSet is set
	// This will use the TypedElementSelectionInterface to select/deselect elements
	UE_API void SetSelection(const TArray<DataStorage::RowHandle>& InSelectedRows);

	// Helper function to create a label widget for a given row
	UE_API TSharedRef<SWidget> CreateLabelWidgetForItem(DataStorage::RowHandle TargetRow, ISceneOutlinerTreeItem& TreeItem, const STableRow<FSceneOutlinerTreeItemPtr>& RowItem, bool bIsInteractable = true) const;

	// Add an external query description to the Outliner
	UE_API void AddExternalQuery(const FName& QueryName, const QueryHandle& InQueryHandle);
	UE_API void RemoveExternalQuery(const FName& QueryName);

	// Append all external queries into the given query description
	UE_API void AppendExternalQueries(DataStorage::FQueryDescription& OutQuery);

	// Add an external query function to the Outliner
	UE_API void AddExternalQueryFunction(const FName& QueryName, const TConstQueryFunction<bool>& InQueryFunction);
	UE_API void RemoveExternalQueryFunction(const FName& QueryName);

	// Add an external class query function to the Outliner (uses OR) 
	UE_API void AddClassQueryFunction(const FName& ClassName, const TConstQueryFunction<bool>& InClassQueryFunction);
	UE_API void RemoveClassQueryFunction(const FName& ClassName);
	
	// Outliner specific functionality
	UE_API void CreateItemsFromQuery(TArray<FSceneOutlinerTreeItemPtr>& OutItems, ISceneOutlinerMode* InMode);
	UE_API void CreateChildren(const FSceneOutlinerTreeItemPtr& Item, TArray<FSceneOutlinerTreeItemPtr>& OutChildren) const;

	// Get the parent row for a given row
	UE_API DataStorage::RowHandle GetParentRow(DataStorage::RowHandle InRowHandle);

	UE_API bool ShouldForceShowParentRows() const;

	// Refresh the primary query node and its filters; use instead of RecompileQueries if the queries have not changed 
	UE_API void RefreshQueries() const;

	// Recompile all queries used by this table viewer
	UE_API void RecompileQueries();

	// Unregister all queries used by this table viewer
	UE_API void UnregisterQueries();
	
	UE_API void Tick();

	// Sorting delegates
	bool IsSorting() const;
	void OnSort(FName ColumnName, TSharedPtr<const DataStorage::FColumnSorterInterface> ColumnSorter, EColumnSortMode::Type Direction, 
		EColumnSortPriority::Type Priority);
	void SortItems(TArray<FSceneOutlinerTreeItemPtr>& Items, EColumnSortMode::Type Direction) const;

	UE_API DataStorage::RowHandle GetOutlinerRowHandle() const;

	// Exclude and include one or more columns from this outliner.
	UE_API void IncludeExcludeColumns(TConstArrayView<TWeakObjectPtr<const UScriptStruct>> IncludedColumns, TConstArrayView<TWeakObjectPtr<const UScriptStruct>> ExcludedColumns);

	UE_API int32 GetTotalRowCount() const;
	
	UE_API const FTedsOutlinerExpansionStateBridge& GetExpansionStateBridge() const;

protected:
	
	UE_API void ClearSelection() const;
	UE_API void PostDataStorageUpdate();

	UE_API void CreateFilterQueries();

	// Check if this row can be displayed in this table viewer
	UE_API bool CanDisplayRow(DataStorage::RowHandle ItemRowHandle) const;

	// Register this Teds Outliner with TEDS
	UE_API void RegisterTedsOutliner();

	UE_API void ClearColumns() const;
	UE_API void RegenerateColumns(const FTedsOutlinerColumnDescription& ColumnDescription);
	
private:
	void OnSelectionChangedHandler(const UTypedElementSelectionSet* InSelectionSet);
	RowHandle GetRowHandleFromOutlinerItem(const ISceneOutlinerTreeItem& Item) const;

protected:
	// TEDS Storage Constructs
	DataStorage::ICoreProvider* Storage{ nullptr };
	DataStorage::IUiProvider* StorageUi{ nullptr };
	DataStorage::ICompatibilityProvider* StorageCompatibility{ nullptr };

	FTedsOutlinerParams CreationParams;

	// The purpose to use when generating widgets for row/column pairs through TEDS UI
	DataStorage::IUiProvider::FPurposeID CellWidgetPurpose;
	
	// The purpose to use when generating widgets for headers pairs through TEDS UI
	DataStorage::IUiProvider::FPurposeID HeaderWidgetPurpose;

	// The purpose to use when generating widgets for the "Item Label" column through TEDS UI
	DataStorage::IUiProvider::FPurposeID LabelWidgetPurpose;
	
	// Initial query provided by user
	DataStorage::FQueryDescription InitialQueryDescription;

	// External query descriptions that are currently active (e.g Filters)
	TMap<FName, TSharedPtr<DataStorage::QueryStack::IQueryNode>> ExternalQueries;
	
	// External query functions that are currently active (e.g Filters)
    TMap<FName, DataStorage::Queries::TConstQueryFunction<bool>> ExternalQueryFunctions;

	// External query functions used to filter by class, these are the only filters that use OR instead of AND
	TMap<FName, DataStorage::Queries::TConstQueryFunction<bool>> ClassFilters;

	// Array to store all filter nodes created from the active query functions
	TArray<TSharedPtr<DataStorage::QueryStack::FRowFilterNode>> FilterNodes;

	// Array to store all filter nodes created from the active class filters
	TArray<TSharedPtr<DataStorage::QueryStack::FRowFilterNode>> ClassFilterNodes;

	// Optional Hierarchy Data
	TSharedPtr<ITedsOutlinerHierarchyDataInterface> HierarchyData;
	// The query stack node responsible for collecting all rows that match the initial query on FullRefresh()
	TSharedPtr<DataStorage::QueryStack::FRowQueryResultsNode> InitialRowCollectorNode;
	// The query stack node responsible for collecting all rows that match the composite query on FullRefresh()
	TSharedPtr<DataStorage::QueryStack::FRowQueryResultsNode> CombinedRowCollectorNode;

	// The query stack node responsible for combining class filters - utilizes the unique merge approach (OR)
	TSharedPtr<DataStorage::QueryStack::IRowNode> CombinedClassFilterRowNode;

	// The query stack node responsible for combining all filters and queries as the 'final' node responsible
	// for getting all desired rows - utilizes the repeating merge approach (AND)
	TSharedPtr<DataStorage::QueryStack::IRowNode> FinalCombinedRowNode;

	// The query stack node responsible for tracking and row addition/removals that happen in between refreshes
	TSharedPtr<DataStorage::QueryStack::FRowMonitorNode> RowMonitorNode;

	// A query stack node that keeps a copy of our rows sorted by row handle for faster search/lookup
	TSharedPtr<DataStorage::QueryStack::FRowHandleSortNode> RowHandleSortNode;

	// A query stack node that keeps a copy of our rows sorted by the actual column the user wants to sort by
	TSharedPtr<DataStorage::QueryStack::FRowSortNode> RowPrimarySortingNode;

	// Inverts the rows from the query stack when needed by the primary sort.
	TSharedPtr<DataStorage::QueryStack::FRowOrderInversionNode> RowPrimaryInversionNode;

	// A map of the row handle -> index in the sort node for lookup
	TSharedPtr<FRowMapNode> RowSortMapNode;

	// The executor used to update the query stack stack.
	TUniquePtr<DataStorage::QueryStack::FExplicitUpdateExecutor> QueryStackExecutor;
	
	// A query stack node that keeps a sorted copy of the initial query results (i.e without any filters or search) for fast lookup
	TSharedPtr<DataStorage::QueryStack::FRowHandleSortNode> InitialQueryResultsSortedNode;
	
	// The executor used to update the node storing a copy of the initial query results. This is updated on demand and not every tick
	TUniquePtr<DataStorage::QueryStack::FExplicitUpdateExecutor> InitialQueryResultsSortedNodeExecutor;

	// Query to get all selected rows, track selection added, track selection removed
	DataStorage::QueryHandle SelectedRowsQuery = DataStorage::InvalidQueryHandle;
	DataStorage::QueryHandle SelectionAddedQuery = DataStorage::InvalidQueryHandle;
	DataStorage::QueryHandle SelectionRemovedQuery = DataStorage::InvalidQueryHandle;

	// Query to track when a row's label gets changed
	DataStorage::QueryHandle LabelUpdateQuery = DataStorage::InvalidQueryHandle;
	
	// Additional observer queries
	DataStorage::QueryHandle AdditionalAddObserverQuery = DataStorage::InvalidQueryHandle;
	DataStorage::QueryHandle AdditionalRemoveObserverQuery = DataStorage::InvalidQueryHandle;

	// Handle to keep track of the lifetime of the OnChanged delegate handler for the SelectionSet
	// This handler is responsible for setting bSelectionDirty
	FDelegateHandle SelectionSetChangedDelegateHandle;
	
	// The UTypedElementSelectionSet to use for propagating selection via TypedElementSelectionInterface
	TObjectPtr<UTypedElementSelectionSet> SelectionSet;
	
	bool bSelectionDirty = false;
	ESelectInfo::Type PendingSelectionType = ESelectInfo::Direct;

	// If true, parent nodes will remain visible if a child passes all filters even if the parent fails a filter
	bool bForceShowParents = true;

	// Rows that are force-shown as parents in the tree because bForceShowParents=true, but do not themselves match the active filter (not in FinalCombinedRowNode).
	FRowHandleArray ForceShownParentRows;
	
	// Ticker for selection updates so we don't fire the delegate multiple times in one frame for multi select
	FTSTicker::FDelegateHandle TickerHandle;

	// Handle for OnTextFilteredChanged delegate, so we can remove it upon destruction
	FDelegateHandle OnTextFilteredChangedHandle;
	
	FOnTedsOutlinerSelectionChanged OnTedsOutlinerSelectionChanged;

	// Scene Outliner specific constructors
	ISceneOutlinerMode* SceneOutlinerMode;
	SSceneOutliner* SceneOutliner;

	// Event fired when the hierarchy changes (addition/removal/move)
	ISceneOutlinerHierarchy::FHierarchyChangedEvent HierarchyChangedEvent;

	// Delegate to check if an item is compatible with this table viewer
	FIsItemCompatible IsItemCompatibleWithTeds;

	// Whether the row query is using the FConditions syntax (TColumn<FMyColumn>()) or the old syntax (.All().Any().None())
	bool bUsingQueryConditionsSyntax = false;

	// Addition and Label Updates are deferred because they access data storage implicitly
	// Deletion is currently not deferred to work nicely with object lifecycles in some cases - but can be once everything goes through the query stack
	TSet<DataStorage::RowHandle> RowsPendingLabelUpdate;

	// The Row Handle associated with this outliner
	DataStorage::RowHandle OutlinerRowHandle = DataStorage::InvalidRowHandle;

	// The columns currently being displayed by the Teds Outliner
	TArray<FName> AddedColumns;

	// Whether this is a hybrid outliner, i.e it is displaying TEDS rows and some non-TEDS items
	bool bHybridMode;
	
	// Expansion state bridge (temporary)
	FTedsOutlinerExpansionStateBridge ExpansionStateBridge;
};
} // namespace UE::Editor::Outliner

#undef UE_API
