// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/Interfaces/TypedElementDataStorageInterface.h"
#include "TedsOutlinerMode.h"

namespace UE::Editor::Outliner
{
class FTedsOutlinerImpl;

/*
 * A generic item in the TEDS driven Outliner, that uses a DataStorage::RowHandle to uniquely identify the object it is
 * looking at. Functionality should be added through TEDS queries instead of having a different TreeItem type for each
 * type of object you are looking at (i.e. Actor vs Folder)
 * Inherits from ISceneOutlinerItem - which determines what type of item you are looking at. E.G FActorTreeItem for actors
 */
struct FTedsOutlinerTreeItem : ISceneOutlinerTreeItem
{
public:
	
	DECLARE_DELEGATE_RetVal_OneParam(bool, FFilterPredicate, const DataStorage::RowHandle);
	DECLARE_DELEGATE_RetVal_OneParam(bool, FInteractivePredicate, const DataStorage::RowHandle);

	bool Filter(FFilterPredicate Pred) const
	{
		return Pred.Execute(RowHandle);
	}

	bool GetInteractiveState(FInteractivePredicate Pred) const
	{
		return Pred.Execute(RowHandle);
	}

	TEDSOUTLINER_API FTedsOutlinerTreeItem(const DataStorage::RowHandle& InRowHandle, const TWeakPtr<const FTedsOutlinerImpl>& InTedsOutlinerImpl);

	/* Begin ISceneOutlinerTreeItem Implementation */
	TEDSOUTLINER_API virtual bool IsValid() const override;
	TEDSOUTLINER_API virtual FSceneOutlinerTreeItemID GetID() const override;
	TEDSOUTLINER_API virtual FFolder::FRootObject GetRootObject() const override;
	TEDSOUTLINER_API virtual FString GetDisplayString() const override;
	TEDSOUTLINER_API virtual bool CanInteract() const override;
	TEDSOUTLINER_API virtual TSharedRef<SWidget> GenerateLabelWidget(ISceneOutliner& Outliner, const STableRow<FSceneOutlinerTreeItemPtr>& InRow) override;
	/** Generate a context menu for this item. Only called if *only* this item is selected. */
	TEDSOUTLINER_API virtual void GenerateContextMenu(UToolMenu* Menu, SSceneOutliner& Outliner);
	TEDSOUTLINER_API virtual void OnExpansionChanged() override;
	/* End ISceneOutlinerTreeItem Implementation */

	TEDSOUTLINER_API static const FSceneOutlinerTreeItemType Type;

	TEDSOUTLINER_API DataStorage::RowHandle GetRowHandle() const;

private:
	const DataStorage::RowHandle RowHandle;
	const TWeakPtr<const FTedsOutlinerImpl> TedsOutlinerImpl;
};
} // namespace UE::Editor::Outliner
