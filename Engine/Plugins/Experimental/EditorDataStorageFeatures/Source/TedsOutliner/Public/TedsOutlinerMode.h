// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ISceneOutlinerMode.h"
#include "TedsOutlinerImpl.h"
#include "Elements/Interfaces/TypedElementDataStorageInterface.h"
#include "Elements/Columns/TypedElementHiearchyColumns.h"
#include "Framework/MultiBox/MultiBoxExtender.h"

#include "TedsOutlinerMode.generated.h"

#define UE_API TEDSOUTLINER_API

struct FTedsOutlinerModeConfig;

namespace UE::Editor::Outliner
{
/*
 * TEDS driven Outliner mode where the Outliner is populated using the results of the RowHandleQueries passed in.
 * See CreateGenericTEDSOutliner() for example usage
 * Inherits from ISceneOutlinerMode - which contains all actions that depend on the type of item you are viewing in the Outliner
 */
class FTedsOutlinerMode : public ISceneOutlinerMode
{
public:
	UE_API explicit FTedsOutlinerMode(const FTedsOutlinerParams& InParams);
	UE_API virtual ~FTedsOutlinerMode() override;

	/* ISceneOutlinerMode interface */
	UE_API virtual void Rebuild() override;
	UE_API virtual void SynchronizeSelection() override;
	UE_API virtual void OnItemSelectionChanged(FSceneOutlinerTreeItemPtr Item, ESelectInfo::Type SelectionType, const FSceneOutlinerItemSelection& Selection) override;
	UE_API virtual bool CanInteract(const ISceneOutlinerTreeItem& Item) const override;
	virtual ESelectionMode::Type GetSelectionMode() const override { return ESelectionMode::Multi; }
	virtual bool CanSupportDragAndDrop() const override { return true; } // TODO: Can we check this from TEDS somehow (if the user requests a drag column?)
	UE_API virtual TSharedPtr<FDragDropOperation> CreateDragDropOperation(const FPointerEvent& MouseEvent, const TArray<FSceneOutlinerTreeItemPtr>& InTreeItems) const override;
	UE_API virtual bool ParseDragDrop(FSceneOutlinerDragDropPayload& OutPayload, const FDragDropOperation& Operation) const override;
	UE_API virtual FSceneOutlinerDragValidationInfo ValidateDrop(const ISceneOutlinerTreeItem& DropTarget, const FSceneOutlinerDragDropPayload& Payload) const override;
	UE_API virtual void OnDrop(ISceneOutlinerTreeItem& DropTarget, const FSceneOutlinerDragDropPayload& Payload, const FSceneOutlinerDragValidationInfo& ValidationInfo) const override;
	UE_API virtual TSharedPtr<SWidget> CreateContextMenu() override;
	UE_API virtual void OnItemDoubleClick(FSceneOutlinerTreeItemPtr Item) override;
	UE_API virtual void Tick() override;
	UE_API virtual bool CanPopulate() const override;
	UE_API virtual bool ShowViewButton() const override;
	UE_API virtual void InitializeViewMenuExtender(TSharedPtr<FExtender> Extender) override;
	UE_API virtual bool ShowFilterOptions() const override;
	UE_API virtual void OnItemAdded(FSceneOutlinerTreeItemPtr Item) override;
	UE_API virtual void OnItemRemoved(FSceneOutlinerTreeItemPtr Item) override;
	UE_API virtual FText GetStatusText() const override;
	UE_API virtual FSlateColor GetStatusTextColor() const override;
	UE_API virtual bool ShowStatusBar() const override;
	UE_API virtual void BindCommands(const TSharedRef<FUICommandList>& OutCommandList) override;
	UE_API virtual bool CanRenameItem(const ISceneOutlinerTreeItem& Item) const override;
	UE_API virtual bool CanDelete() const override;
	UE_API virtual bool CanCut() const override;
	UE_API virtual bool CanCopy() const override;
	UE_API virtual bool CanPaste() const override;
	UE_API virtual FFolder::FRootObject GetRootObject() const override;
	UE_API virtual bool CanCustomizeToolbar() const override;
	UE_API virtual void CustomAddToToolbar(TSharedPtr<class SHorizontalBox> Toolbar) override;
	/* end ISceneOutlinerMode interface */

	UE_API int32 GetFilteredRowCount() const;

protected:

	UE_API virtual TUniquePtr<ISceneOutlinerHierarchy> CreateHierarchy() override;
	// Called by TedsOutlinerImpl when the selection in TEDS changes
	UE_API void OnSelectionChanged(ESelectInfo::Type SelectionType);

	/** Functions to expose default SceneOutliner Options to the UI */
	void OnToggleAlwaysFrameSelection();
	bool ShouldAlwaysFrameSelection() const;

	void OnToggleCollapseOutlinerTreeOnNewSelection();
	bool CollapseOutlinerTreeOnNewSelection() const;

private:
	/**
	 * Get a mutable version of the TedsOutliner config for setting values.
	 * @returns		The config for this TedsOutliner.
	 * @note		If OutlinerIdentifier is not set for this outliner, it is not possible to store settings.
	 */
	struct FTedsOutlinerModeConfig* GetMutableConfig();

	/**
	 * Get a const version of the TedsOutliner config for getting values.
	 * @returns		The config for this TedsOutliner.
	 * @note		If OutlinerIdentifier is not set for this outliner, it is not possible to store settings.
	 */
	const FTedsOutlinerModeConfig* GetConstConfig() const;

	/** Save the config for this TedsOutliner */
	void SaveConfig();

	// Bound to the rename command.
	bool CanExecuteRename();
	void OnExecuteRename();

	/**
	 * Register Select/Rename/ScrollIntoView actions to apply to a row's TreeItem the next time it is added to the tree.
	 * NewItemActions (what OnItemAdded writes to) gets cleared at SSceneOutliner::Populate whenever PendingOperations is empty. 
	 * For TEDS, the new row surfaces via RowMonitor on the next TEDS tick so a Populate pass with empty PendingOperations can fire in between, wiping the entry before SSceneOutliner::AddItemToTree ever sees it.
	 * RegisterPendingItemActions bypasses that by storing the mask on the TEDS side and consuming it in FTedsOutlinerMode::OnItemAdded after the race window. 
	 */
	void RegisterPendingItemActions(DataStorage::RowHandle Row, uint8 ItemActions);

protected:
	bool bShowViewButton = false;
	
	/** Should the outliner scroll to the item on selection */
	bool bAlwaysFrameSelection;
	/** Should Outliner Tree get collapsed on selection, except for the item that was just selected */
	bool bCollapseOutlinerTreeOnNewSelection;
	
	// Delegate for extending the View Menu with options (World picker, frame on selection, etc.)
	FOnExtendViewMenu OnInitializeViewMenuExtender;
	// Delegate for injecting entries into the "Options" section of the View Menu
	FOnExtendOptionsMenu OnExtendOptionsMenu;
	// Delegate called before the default double-click behavior. Return true if handled.
	FOnOutlinerItemDoubleClick OnItemDoubleClickDelegate;
	// Delegate called in CanPopulate if there is logic needed to disable populating (i.e during PIE)
	FOnCanPopulate OnCanPopulateDelegate;
	// Delegate that adds widgets to the outliner toolbar.
	FOnCustomAddToToolbar OnCustomAddToToolbarDelegate;
	// The actual model for the TEDS Outliner
	TSharedPtr<FTedsOutlinerImpl> TedsOutlinerImpl;

	// TypedElement Selection Options use during CanInteract
	FTypedElementSelectionOptions CanInteractSelectionOptions;

	// Number of items currently visible in the tree (reduced by text filter)
	int32 TedsOutlinerItemCount = 0;

	// Per-row Select/Rename/ScrollIntoView actions to apply when the matching TreeItem first appears in the tree.
	TMap<DataStorage::RowHandle, uint8> PendingItemActionsByRow;
};
} // namespace UE::Editor::Outliner

// Class to hold the owning scene outliner for a menu
// TEDS-Outliner TODO: Once menus go through TEDS UI this can be done using the FTableViewerColumn on the widget row instead
UCLASS()
class UTedsOutlinerMenuContext : public UObject
{
	GENERATED_BODY()
public:
	SSceneOutliner* OwningSceneOutliner = nullptr;
};

#undef UE_API
