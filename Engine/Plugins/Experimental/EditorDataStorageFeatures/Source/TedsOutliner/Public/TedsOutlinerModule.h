// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataStorage/Handles.h"
#include "Modules/ModuleInterface.h"
#include "Templates/SharedPointer.h"
#include "Delegates/IDelegateInstance.h"
#include "DataStorage/Queries/Description.h"
#include "TedsOutlinerFilter.h"
#include "Elements/Framework/TypedElementQueryContext.h"

namespace UE::Editor::DataStorage
{
	class IUiProvider;
}

struct FTypedElementWorldColumn;
class ISceneOutliner;
struct FSceneOutlinerInitializationOptions;
class SDockTab;
class SWidget;
class FSpawnTabArgs;

namespace UE::Editor::Outliner
{
	struct FTedsOutlinerParams;
	struct FTedsOutlinerColumnDescription;

/**
 * Implements the Scene Outliner module.
 */
class FTedsOutlinerModule
	: public IModuleInterface
{
public:

	FTedsOutlinerModule();

	/**
	 * Creates a TEDS-Outliner widget
	 *
	 * @param	InInitOptions						Programmer-driven configuration for the scene outliner
	 * @param	InInitTedsOptions				Programmer-driven configuration for the TEDS queries that drive the outliner
	 *
	 * @return	New scene outliner widget
	 */
	virtual TSharedRef<ISceneOutliner> CreateTedsOutliner(
		const FSceneOutlinerInitializationOptions& InInitOptions, const FTedsOutlinerParams& InInitTedsOptions) const;

	// IModuleInterface interface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	
	// Get the column description the default table viewer uses
	virtual FTedsOutlinerColumnDescription GetLevelEditorTedsOutlinerColumnDescription();

	// The name of the tab the default table viewer is opened in
	FName GetTedsOutlinerTabName();

private:
	
	void RegisterLevelEditorTedsOutlinerTab();
	void UnregisterLevelEditorTedsOutlinerTab();
	TSharedRef<SDockTab> OpenLevelEditorTedsOutliner(const FSpawnTabArgs& SpawnTabArgs);
	TSharedRef<SWidget> CreateLevelEditorTedsOutliner();
	
private:
	FName TedsOutlinerTabName;
	FDelegateHandle LevelEditorTabManagerChangedHandle;
	FDelegateHandle TedsInitializedHandle;
	FDelegateHandle EditorInitializedHandle;
	DataStorage::ICoreProvider* Storage = nullptr;
	DataStorage::IUiProvider* StorageUi = nullptr;
};

class FTedsSceneOutlinerWorldFilter : public FTedsOutlinerFilter
{
public:
	FTedsSceneOutlinerWorldFilter(const FName& InFilterName, const TWeakObjectPtr<UWorld>& InSpecifiedWorldToDisplay = nullptr);

	~FTedsSceneOutlinerWorldFilter();
	
	void BuildWorldFilter();
	
	void QueryFunction(TConstQueryContext<RowBatchInfo> Context, TResult<bool>& Result, TConstBatch<FTypedElementWorldColumn> WorldColumns);

	void SetUserChosenWorld(const TWeakObjectPtr<UWorld>& InWorld);
	
	TWeakObjectPtr<UWorld> GetRepresentingWorld() const;

	TWeakObjectPtr<UWorld> GetUserChosenWorld() const;
	
protected:
	/** The world which we are currently filtering for */
	TWeakObjectPtr<UWorld> RepresentingWorld;
	
	/** The world that has been manually picked by the user */
	TWeakObjectPtr<UWorld> UserChosenWorld;

	/** If this mode was created to display a specific world, don't allow it to be reassigned */
	const TWeakObjectPtr<UWorld> SpecifiedWorldToDisplay;
};
} // namespace UE::Editor::Outliner
