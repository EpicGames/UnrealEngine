// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/Interfaces/TypedElementDataStorageFactory.h"
#include "Elements/Interfaces/TypedElementDataStorageInterface.h"
#include "Elements/Interfaces/TypedElementDataStorageUiInterface.h"
#include "Layers/Columns/LayersColumns.h"

#include "OutlinerActorLayersWidget.generated.h"

#define UE_API TEDSOUTLINER_API

namespace UE::Editor::DataStorage
{
	class FOutlinerActorLayersWidgetSearcher final : public TColumnSearcherInterface<Layers::FActorLayersColumn>
	{
	protected:
		virtual void GetSearchableString(FString& SearchableString, const ICoreProvider& Storage, const Layers::FActorLayersColumn& Column) const override;
	};
}

UCLASS()
class UOutlinerActorLayersWidgetFactory : public UEditorDataStorageFactory
{
	GENERATED_BODY()

public:
	~UOutlinerActorLayersWidgetFactory() override = default;

	void RegisterWidgetConstructors(UE::Editor::DataStorage::ICoreProvider& DataStorage,
		UE::Editor::DataStorage::IUiProvider& DataStorageUi) const override;
};

USTRUCT()
struct FOutlinerActorLayersWidgetConstructor : public FSimpleWidgetConstructor
{
	GENERATED_BODY()

public:
	UE_API FOutlinerActorLayersWidgetConstructor();
	~FOutlinerActorLayersWidgetConstructor() override = default;

	UE_API virtual FText CreateWidgetDisplayNameText(
		UE::Editor::DataStorage::ICoreProvider* DataStorage,
		UE::Editor::DataStorage::RowHandle Row = UE::Editor::DataStorage::InvalidRowHandle) const override;

	UE_API virtual TSharedPtr<SWidget> CreateWidget(
		UE::Editor::DataStorage::ICoreProvider* DataStorage,
		UE::Editor::DataStorage::IUiProvider* DataStorageUi,
		UE::Editor::DataStorage::RowHandle TargetRow,
		UE::Editor::DataStorage::RowHandle WidgetRow,
		const UE::Editor::DataStorage::FMetaDataView& Arguments) override;

	UE_API virtual TArray<TSharedPtr<const UE::Editor::DataStorage::FColumnSearcherInterface>> ConstructColumnSearchers(
		UE::Editor::DataStorage::ICoreProvider* DataStorage,
		UE::Editor::DataStorage::IUiProvider* DataStorageUi,
		const UE::Editor::DataStorage::FMetaDataView& Arguments) override;
};

#undef UE_API
