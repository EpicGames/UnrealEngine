// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/Interfaces/TypedElementDataStorageFactory.h"
#include "Elements/Interfaces/TypedElementDataStorageUiInterface.h"
#include "Types/SlateEnums.h"

#include "OutlinerCDOTextWidget.generated.h"

struct FSlateBrush;
struct FTypedElementClassTypeInfoColumn;
class  SInlineEditableTextBlock;
class  STextBlock;

UCLASS()
class UOutlinerCDOTextWidgetFactory : public UEditorDataStorageFactory
{
	GENERATED_BODY()

public:
	virtual ~UOutlinerCDOTextWidgetFactory() override = default;

	void RegisterWidgetConstructors(UE::Editor::DataStorage::ICoreProvider& DataStorage,
		UE::Editor::DataStorage::IUiProvider& DataStorageUi) const override;
};

USTRUCT()
struct FOutlinerCDOTextWidgetConstructor : public FSimpleWidgetConstructor
{
	GENERATED_BODY()

public:
	TEDSOUTLINER_API FOutlinerCDOTextWidgetConstructor();
	TEDSOUTLINER_API FOutlinerCDOTextWidgetConstructor(const UScriptStruct* InTypeInfo);
	virtual ~FOutlinerCDOTextWidgetConstructor() override = default;

	TEDSOUTLINER_API virtual TSharedPtr<SWidget> CreateWidget(
		UE::Editor::DataStorage::ICoreProvider* DataStorage,
		UE::Editor::DataStorage::IUiProvider* DataStorageUi,
		UE::Editor::DataStorage::RowHandle TargetRow,
		UE::Editor::DataStorage::RowHandle WidgetRow,
		const UE::Editor::DataStorage::FMetaDataView& Arguments) override;
};
