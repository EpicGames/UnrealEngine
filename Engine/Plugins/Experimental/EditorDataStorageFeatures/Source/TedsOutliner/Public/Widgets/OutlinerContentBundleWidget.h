// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/Interfaces/TypedElementDataStorageFactory.h"
#include "Elements/Interfaces/TypedElementDataStorageInterface.h"
#include "Elements/Interfaces/TypedElementDataStorageUiInterface.h"

#include "OutlinerContentBundleWidget.generated.h"

#define UE_API TEDSOUTLINER_API

UCLASS()
class UOutlinerContentBundleWidgetFactory : public UEditorDataStorageFactory
{
	GENERATED_BODY()

public:
	~UOutlinerContentBundleWidgetFactory() override = default;

	void RegisterWidgetConstructors(UE::Editor::DataStorage::ICoreProvider& DataStorage,
		UE::Editor::DataStorage::IUiProvider& DataStorageUi) const override;
};

USTRUCT()
struct FOutlinerContentBundleWidgetConstructor : public FSimpleWidgetConstructor
{
	GENERATED_BODY()

public:
	UE_API FOutlinerContentBundleWidgetConstructor();
	~FOutlinerContentBundleWidgetConstructor() override = default;

	UE_API virtual FText CreateWidgetDisplayNameText(
		UE::Editor::DataStorage::ICoreProvider* DataStorage,
		UE::Editor::DataStorage::RowHandle Row = UE::Editor::DataStorage::InvalidRowHandle) const override;

	UE_API virtual TSharedPtr<SWidget> CreateWidget(
		UE::Editor::DataStorage::ICoreProvider* DataStorage,
		UE::Editor::DataStorage::IUiProvider* DataStorageUi,
		UE::Editor::DataStorage::RowHandle TargetRow,
		UE::Editor::DataStorage::RowHandle WidgetRow,
		const UE::Editor::DataStorage::FMetaDataView& Arguments) override;
};

#undef UE_API
