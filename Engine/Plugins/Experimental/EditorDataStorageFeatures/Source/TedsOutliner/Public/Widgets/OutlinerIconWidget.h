// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/Interfaces/TypedElementDataStorageFactory.h"
#include "Elements/Interfaces/TypedElementDataStorageUiInterface.h"
#include "UObject/OverridableManager.h"

#include "OutlinerIconWidget.generated.h"

struct FSlateBrush;
struct FTypedElementClassTypeInfoColumn;

UCLASS()
class UOutlinerIconWidgetFactory : public UEditorDataStorageFactory
{
	GENERATED_BODY()

public:
	virtual void RegisterWidgetConstructors(UE::Editor::DataStorage::ICoreProvider& DataStorage,
		UE::Editor::DataStorage::IUiProvider& DataStorageUi) const override;
};

struct FOutlinerIconWidgetLogic
{
	virtual ~FOutlinerIconWidgetLogic() = default;
	
	TEDSOUTLINER_API virtual const FSlateBrush* GetOverrideBadgeFirstLayer(const UE::Editor::DataStorage::ICoreProvider& Storage,
		UE::Editor::DataStorage::RowHandle Row) const;
	TEDSOUTLINER_API virtual const FSlateBrush* GetOverrideBadgeSecondLayer(const UE::Editor::DataStorage::ICoreProvider& Storage,
		UE::Editor::DataStorage::RowHandle Row) const;
	TEDSOUTLINER_API virtual FText GetOverrideTooltip(const UE::Editor::DataStorage::ICoreProvider& Storage,
		UE::Editor::DataStorage::RowHandle Row) const;
	
	TEDSOUTLINER_API virtual EOverriddenState GetOverriddenState(const UE::Editor::DataStorage::ICoreProvider& Storage,
		UE::Editor::DataStorage::RowHandle Row) const;
};

USTRUCT()
struct FOutlinerIconWidgetConstructor : public FSimpleWidgetConstructor
{
	GENERATED_BODY()

public:
	TEDSOUTLINER_API FOutlinerIconWidgetConstructor(const UScriptStruct* InTypeInfo = StaticStruct());

	TEDSOUTLINER_API virtual TSharedPtr<SWidget> CreateWidget(
		UE::Editor::DataStorage::ICoreProvider* DataStorage,
		UE::Editor::DataStorage::IUiProvider* DataStorageUi,
		UE::Editor::DataStorage::RowHandle TargetRow,
		UE::Editor::DataStorage::RowHandle WidgetRow,
		const UE::Editor::DataStorage::FMetaDataView& Arguments) override;

protected:
	virtual const FOutlinerIconWidgetLogic& GetLogic() const;
};
