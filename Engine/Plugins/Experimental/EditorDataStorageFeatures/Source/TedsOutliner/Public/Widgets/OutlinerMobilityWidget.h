// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/Interfaces/TypedElementDataStorageFactory.h"
#include "Elements/Interfaces/TypedElementDataStorageInterface.h"
#include "Elements/Interfaces/TypedElementDataStorageUiInterface.h"
#include "Columns/TedsActorMobilityColumns.h"

#include "OutlinerMobilityWidget.generated.h"

#define UE_API TEDSOUTLINER_API

struct FTedsActorMobilityColumn;

namespace UE::Editor::DataStorage
{
	class FMobilityWidgetSearcher final : public TColumnSearcherInterface<FTedsActorMobilityColumn>
	{
	protected:
		virtual void GetSearchableString(FString& SearchableString, const ICoreProvider& Storage, const FTedsActorMobilityColumn& Column) const override;
	};
}

UCLASS()
class UOutlinerMobilityWidgetFactory : public UEditorDataStorageFactory
{
	GENERATED_BODY()

public:
	~UOutlinerMobilityWidgetFactory() override = default;

	void RegisterWidgetConstructors(UE::Editor::DataStorage::ICoreProvider& DataStorage,
		UE::Editor::DataStorage::IUiProvider& DataStorageUi) const override;
};

USTRUCT()
struct FOutlinerMobilityWidgetConstructor : public FSimpleWidgetConstructor
{
	GENERATED_BODY()

public:
	UE_API FOutlinerMobilityWidgetConstructor();
	~FOutlinerMobilityWidgetConstructor() override = default;

	UE_API virtual TSharedPtr<SWidget> CreateWidget(
		UE::Editor::DataStorage::ICoreProvider* DataStorage,
		UE::Editor::DataStorage::IUiProvider* DataStorageUi,
		UE::Editor::DataStorage::RowHandle TargetRow,
		UE::Editor::DataStorage::RowHandle WidgetRow,
		const UE::Editor::DataStorage::FMetaDataView& Arguments) override;

	UE_API virtual TArray<TSharedPtr<const UE::Editor::DataStorage::FColumnSearcherInterface>> ConstructColumnSearchers(
		UE::Editor::DataStorage::ICoreProvider* DataStorage, UE::Editor::DataStorage::IUiProvider* DataStorageUi,
		const UE::Editor::DataStorage::FMetaDataView& Arguments) override;

};

#undef UE_API
