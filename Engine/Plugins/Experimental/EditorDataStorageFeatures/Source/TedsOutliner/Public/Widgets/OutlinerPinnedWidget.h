// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/Interfaces/TypedElementDataStorageFactory.h"
#include "Elements/Interfaces/TypedElementDataStorageUiInterface.h"
#include "Internationalization/Text.h"
#include "UObject/ObjectMacros.h"
#include "Widgets/Images/SImage.h"
#include "ScopedTransaction.h"
#include "SceneOutlinerFwd.h"

#include "OutlinerPinnedWidget.generated.h"

#define UE_API TEDSOUTLINER_API

namespace UE::Editor::DataStorage
{
	class ICoreProvider;
}

UCLASS()
class UOutlinerPinnedFactory : public UEditorDataStorageFactory
{
	GENERATED_BODY()

public:
	~UOutlinerPinnedFactory() override = default;

	TEDSOUTLINER_API void RegisterWidgetConstructors(UE::Editor::DataStorage::ICoreProvider& DataStorage, UE::Editor::DataStorage::IUiProvider& DataStorageUi) const override;
};

USTRUCT()
struct FOutlinerPinnedWidgetConstructor : public FSimpleWidgetConstructor
{
	GENERATED_BODY()

public:
	TEDSOUTLINER_API FOutlinerPinnedWidgetConstructor();
	~FOutlinerPinnedWidgetConstructor() override = default;

	TEDSOUTLINER_API virtual FText CreateWidgetDisplayNameText(
		UE::Editor::DataStorage::ICoreProvider* DataStorage,
		UE::Editor::DataStorage::RowHandle Row = UE::Editor::DataStorage::InvalidRowHandle) const override;

	TEDSOUTLINER_API virtual TSharedPtr<SWidget> CreateWidget(
		UE::Editor::DataStorage::ICoreProvider* DataStorage,
		UE::Editor::DataStorage::IUiProvider* DataStorageUi,
		UE::Editor::DataStorage::RowHandle TargetRow,
		UE::Editor::DataStorage::RowHandle WidgetRow,
		const UE::Editor::DataStorage::FMetaDataView& Arguments) override;
};

USTRUCT()
struct FOutlinerPinnedHeaderConstructor : public FSimpleWidgetConstructor
{
	GENERATED_BODY()

public:
	TEDSOUTLINER_API FOutlinerPinnedHeaderConstructor();
	~FOutlinerPinnedHeaderConstructor() override = default;

	TEDSOUTLINER_API virtual TSharedPtr<SWidget> CreateWidget(
		UE::Editor::DataStorage::ICoreProvider* DataStorage,
		UE::Editor::DataStorage::IUiProvider* DataStorageUi,
		UE::Editor::DataStorage::RowHandle TargetRow,
		UE::Editor::DataStorage::RowHandle WidgetRow,
		const UE::Editor::DataStorage::FMetaDataView& Arguments) override;
};

template<>
struct TWidgetTypeTraits<class STedsPinnedWidget>
{
	static constexpr bool SupportsInvalidation() { return true; }
};

/** Widget responsible for displaying and toggling the World Partition pinned state for a single item */
class STedsPinnedWidget : public SImage
{
	using RowHandle = UE::Editor::DataStorage::RowHandle;

public:
	SLATE_BEGIN_ARGS(STedsPinnedWidget) {}
	SLATE_END_ARGS()

	UE_API void Construct(const FArguments& InArgs, const RowHandle& InTargetRow);

protected:
	UE_API virtual FReply OnMouseButtonDoubleClick(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent) override;
	UE_API virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;

	UE_API FReply HandleClick();
	UE_API virtual FSlateColor GetForegroundColor() const override;

	/** Toggle the pinned state of InRow in the data storage */
	UE_API void SetIsPinned(RowHandle InRow, bool bPinned);

	RowHandle TargetRow = UE::Editor::DataStorage::InvalidRowHandle;

	TAttribute<bool> bIsPinnedAttr;
	TAttribute<bool> bIsSelectedAttr;

private:
	static UE_API void CommitPinnedState(UE::Editor::DataStorage::ICoreProvider& DataStorage, RowHandle Row, bool bPinned);

	UE_API void GetSelectedRows(TArray<RowHandle>& OutSelectedRows) const;

	static UE_API UE::Editor::DataStorage::ICoreProvider* GetDataStorage();
};

#undef UE_API
