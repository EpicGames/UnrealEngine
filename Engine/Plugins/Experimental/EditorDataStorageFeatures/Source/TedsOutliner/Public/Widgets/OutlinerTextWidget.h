// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/Interfaces/TypedElementDataStorageFactory.h"
#include "Elements/Interfaces/TypedElementDataStorageUiInterface.h"
#include "Types/SlateEnums.h"
#include "UObject/ObjectMacros.h"

#include "OutlinerTextWidget.generated.h"

struct FSlateBrush;
struct FTypedElementClassTypeInfoColumn;
class  SInlineEditableTextBlock;
class  STextBlock;

UCLASS()
class UOutlinerTextWidgetFactory : public UEditorDataStorageFactory
{
	GENERATED_BODY()

public:
	virtual ~UOutlinerTextWidgetFactory() override = default;

	void RegisterWidgetConstructors(UE::Editor::DataStorage::ICoreProvider& DataStorage,
		UE::Editor::DataStorage::IUiProvider& DataStorageUi) const override;
};

USTRUCT()
struct FOutlinerTextWidgetConstructor : public FSimpleWidgetConstructor
{
	GENERATED_BODY()

public:
	TEDSOUTLINER_API FOutlinerTextWidgetConstructor();
	TEDSOUTLINER_API FOutlinerTextWidgetConstructor(const UScriptStruct* InTypeInfo);
	virtual ~FOutlinerTextWidgetConstructor() override = default;

	TEDSOUTLINER_API virtual TSharedPtr<SWidget> CreateWidget(
		UE::Editor::DataStorage::ICoreProvider* DataStorage,
		UE::Editor::DataStorage::IUiProvider* DataStorageUi,
		UE::Editor::DataStorage::RowHandle TargetRow,
		UE::Editor::DataStorage::RowHandle WidgetRow,
		const UE::Editor::DataStorage::FMetaDataView& Arguments) override;

	//! Updates the label columns with the given text.
	static TEDSOUTLINER_API void OnCommitText(const FText& NewText, ETextCommit::Type Type, UE::Editor::DataStorage::ICoreProvider* DataStorage, UE::Editor::DataStorage::RowHandle TargetRow);
	static TEDSOUTLINER_API bool OnVerifyText(const FText& Label, FText& ErrorMessage);

protected:

	//! Virtual relay function so that deriving Constructors may edit the editable
	//! widget without having to override the whole CreateWidget procedure.
	TEDSOUTLINER_API virtual TSharedPtr<SWidget> CreateEditableWidget(
		UE::Editor::DataStorage::ICoreProvider* DataStorage,
		UE::Editor::DataStorage::IUiProvider* DataStorageUi,
		UE::Editor::DataStorage::RowHandle TargetRow,
		UE::Editor::DataStorage::RowHandle WidgetRow,
		const UE::Editor::DataStorage::FMetaDataView& Arguments);

	//! Virtual relay function so that deriving Constructors may edit the non-editable
	//! widget without having to override the whole CreateWidget procedure.
	TEDSOUTLINER_API virtual TSharedPtr<SWidget> CreateNonEditableWidget(
		UE::Editor::DataStorage::ICoreProvider* DataStorage,
		UE::Editor::DataStorage::IUiProvider* DataStorageUi,
		UE::Editor::DataStorage::RowHandle TargetRow,
		UE::Editor::DataStorage::RowHandle WidgetRow,
		const UE::Editor::DataStorage::FMetaDataView& Arguments);

	//! The actual function that creates the editable widget.
	TEDSOUTLINER_API TSharedPtr<SInlineEditableTextBlock> CreateEditableTextBlock(
		UE::Editor::DataStorage::ICoreProvider* DataStorage,
		UE::Editor::DataStorage::RowHandle TargetRow,
		UE::Editor::DataStorage::RowHandle WidgetRow) const;

	//! The actual function that creates the non-editable widget.
	TEDSOUTLINER_API TSharedPtr<STextBlock> CreateNonEditableTextBlock(
		UE::Editor::DataStorage::ICoreProvider* DataStorage,
		UE::Editor::DataStorage::RowHandle TargetRow,
		UE::Editor::DataStorage::RowHandle WidgetRow) const;
};
