// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/Columns/TypedElementCompatibilityColumns.h"
#include "Elements/Interfaces/TypedElementDataStorageFactory.h"
#include "Elements/Interfaces/TypedElementDataStorageInterface.h"
#include "Elements/Interfaces/TypedElementDataStorageUiInterface.h"

#include "OutlinerUncachedLightsWidget.generated.h"

#define UE_API TEDSOUTLINER_API

namespace UE::Editor::DataStorage
{
	class FUncachedLightsWidgetSearcher final : public FColumnSearcherInterface
	{
	protected:
		virtual void GetSearchableString(FString& SearchableString, const ICoreProvider& Storage, RowHandle Row) const override;
	};

	class FUncachedLightsWidgetSorter final : public FColumnSorterInterface
	{
	public:
		virtual ESortType GetSortType() const override;
		
		virtual FText GetShortName() const override;

		virtual int32 Compare(const ICoreProvider& Storage, RowHandle Left, RowHandle Right) const override;
		
		virtual FPrefixInfo CalculatePrefix(const ICoreProvider& Storage, RowHandle Row, uint32 ByteIndex) const override;
	};
}

UCLASS()
class UOutlinerUncachedLightsWidgetFactory : public UEditorDataStorageFactory
{
	GENERATED_BODY()

public:
	~UOutlinerUncachedLightsWidgetFactory() override = default;

	void RegisterWidgetConstructors(UE::Editor::DataStorage::ICoreProvider& DataStorage,
		UE::Editor::DataStorage::IUiProvider& DataStorageUi) const override;
};

USTRUCT()
struct FOutlinerUncachedLightsWidgetConstructor : public FSimpleWidgetConstructor
{
	GENERATED_BODY()

public:
	UE_API FOutlinerUncachedLightsWidgetConstructor();
	~FOutlinerUncachedLightsWidgetConstructor() override = default;

	UE_API virtual TSharedPtr<SWidget> CreateWidget(
		UE::Editor::DataStorage::ICoreProvider* DataStorage,
		UE::Editor::DataStorage::IUiProvider* DataStorageUi,
		UE::Editor::DataStorage::RowHandle TargetRow,
		UE::Editor::DataStorage::RowHandle WidgetRow,
		const UE::Editor::DataStorage::FMetaDataView& Arguments) override;

	UE_API virtual FText CreateWidgetDisplayNameText(
		UE::Editor::DataStorage::ICoreProvider* DataStorage,
		UE::Editor::DataStorage::RowHandle Row = UE::Editor::DataStorage::InvalidRowHandle) const override;

	UE_API virtual TArray<TSharedPtr<const UE::Editor::DataStorage::FColumnSearcherInterface>> ConstructColumnSearchers(
		UE::Editor::DataStorage::ICoreProvider* DataStorage, UE::Editor::DataStorage::IUiProvider* DataStorageUi,
		const UE::Editor::DataStorage::FMetaDataView& Arguments) override;

	UE_API virtual TArray<TSharedPtr<const UE::Editor::DataStorage::FColumnSorterInterface>> ConstructColumnSorters(
		UE::Editor::DataStorage::ICoreProvider* DataStorage, UE::Editor::DataStorage::IUiProvider* DataStorageUi,
		const UE::Editor::DataStorage::FMetaDataView& Arguments) override;
};

#undef UE_API
