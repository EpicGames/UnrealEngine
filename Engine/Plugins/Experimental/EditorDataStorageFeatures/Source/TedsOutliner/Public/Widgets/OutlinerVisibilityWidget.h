// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/Interfaces/TypedElementDataStorageFactory.h"
#include "Elements/Interfaces/TypedElementDataStorageUiInterface.h"
#include "Internationalization/Text.h"
#include "UObject/ObjectMacros.h"
#include "Widgets/Images/SImage.h"
#include "ScopedTransaction.h"
#include "SceneOutlinerFwd.h"

#include "OutlinerVisibilityWidget.generated.h"

#define UE_API TEDSOUTLINER_API

namespace UE::Editor::DataStorage
{
	class IHierarchyViewerDataInterface;
	class ICoreProvider;
}

class UScriptStruct;

UCLASS()
class UOutlinerVisibilityWidgetFactory : public UEditorDataStorageFactory
{
	GENERATED_BODY()

public:
	~UOutlinerVisibilityWidgetFactory() override = default;

	TEDSOUTLINER_API void RegisterWidgetConstructors(UE::Editor::DataStorage::ICoreProvider& DataStorage, UE::Editor::DataStorage::IUiProvider& DataStorageUi) const override;
};

UCLASS()
class UOutlinerVisibilityHeaderFactory : public UEditorDataStorageFactory
{
	GENERATED_BODY()

public:
	~UOutlinerVisibilityHeaderFactory() override = default;

	TEDSOUTLINER_API void RegisterWidgetConstructors(UE::Editor::DataStorage::ICoreProvider& DataStorage, UE::Editor::DataStorage::IUiProvider& DataStorageUi) const override;
};

USTRUCT()
struct FOutlinerVisibilityWidgetConstructor : public FSimpleWidgetConstructor
{
	GENERATED_BODY()

public:
	TEDSOUTLINER_API FOutlinerVisibilityWidgetConstructor();
	~FOutlinerVisibilityWidgetConstructor() override = default;

	TEDSOUTLINER_API virtual TSharedPtr<SWidget> CreateWidget(
		UE::Editor::DataStorage::ICoreProvider* DataStorage,
		UE::Editor::DataStorage::IUiProvider* DataStorageUi,
		UE::Editor::DataStorage::RowHandle TargetRow,
		UE::Editor::DataStorage::RowHandle WidgetRow, 
		const UE::Editor::DataStorage::FMetaDataView& Arguments) override;
};

USTRUCT()
struct FOutlinerVisibilityHeaderConstructor : public FSimpleWidgetConstructor
{
	GENERATED_BODY()

public:
	TEDSOUTLINER_API FOutlinerVisibilityHeaderConstructor();
	~FOutlinerVisibilityHeaderConstructor() override = default;

	TEDSOUTLINER_API virtual TSharedPtr<SWidget> CreateWidget(
		UE::Editor::DataStorage::ICoreProvider* DataStorage,
		UE::Editor::DataStorage::IUiProvider* DataStorageUi,
		UE::Editor::DataStorage::RowHandle TargetRow,
		UE::Editor::DataStorage::RowHandle WidgetRow, 
		const UE::Editor::DataStorage::FMetaDataView& Arguments) override;
};

template<>
struct TWidgetTypeTraits<class STedsOutlinerVisibilityWidget>
{
	static constexpr bool SupportsInvalidation() { return true; }
};

/** Widget responsible for managing the visibility for a single item */
class STedsOutlinerVisibilityWidget : public SImage
{
	using RowHandle = UE::Editor::DataStorage::RowHandle;

public:
	SLATE_BEGIN_ARGS(STedsOutlinerVisibilityWidget) {}
	SLATE_END_ARGS()

	/** Construct this widget */
	UE_API void Construct(const FArguments& InArgs, const RowHandle& InTargetRow, const RowHandle& InWidgetRow);

protected:

	/** Returns whether the widget is enabled or not */
	virtual bool IsEnabled() const;

	/** Returns whether the item on the given row can commit visibility */
	virtual bool CanCommitVisibility(UE::Editor::DataStorage::ICoreProvider& DataStorage, RowHandle Row) const;

	/** Get the brush for this widget */
	UE_API virtual const FSlateBrush* GetBrush() const;

	/** Start a new drag/drop operation for this widget */
	UE_API virtual FReply OnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;

	/** If a visibility drag drop operation has entered this widget, set its item to the new visibility state */
	UE_API virtual void OnDragEnter(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;

	UE_API FReply HandleClick();

	UE_API virtual FReply OnMouseButtonDoubleClick(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent) override;

	/** Called when the mouse button is pressed down on this widget */
	UE_API virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;

	/** Process a mouse up message */
	UE_API virtual FReply OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;

	/** Called when this widget had captured the mouse, but that capture has been revoked for some reason. */
	UE_API virtual void OnMouseCaptureLost(const FCaptureLostEvent& CaptureLostEvent) override;

	/** Whether visibility change should propagate down to children. */
	virtual bool ShouldPropagateVisibilityChangeOnChildren() const { return true; }

	UE_API virtual FSlateColor GetForegroundColor() const;

	/** Check if the TargetRow Object is Visible */
	UE_API bool IsVisible() const;

	/** Check if the TargetRow Object is Selected */
	UE_API bool IsSelected() const;

	/** Set the item this widget is responsible for to be hidden or shown */
	UE_API void SetIsVisible(RowHandle InRow, const bool bVisible, 
		UE::Editor::DataStorage::FRowHandleArray* ChangedRowsToAppendTo = nullptr);

	RowHandle TargetRow = UE::Editor::DataStorage::InvalidRowHandle;
	RowHandle WidgetRow = UE::Editor::DataStorage::InvalidRowHandle;

	/** Scoped undo transaction */
	TUniquePtr<FScopedTransaction> UndoTransaction;

	/** Visibility brushes for the various states */
	const FSlateBrush* VisibleHoveredBrush = nullptr;
	const FSlateBrush* VisibleNotHoveredBrush = nullptr;
	const FSlateBrush* NotVisibleHoveredBrush = nullptr;
	const FSlateBrush* NotVisibleNotHoveredBrush = nullptr;

private:

	// Commit the visibility state into the Rows DataStorage. Will trigger a SyncBackToWorld manually but the object is never dirtied.
	UE_API void CommitVisibility(UE::Editor::DataStorage::ICoreProvider& DataStorage, RowHandle Row, bool bVisible, 
		UE::Editor::DataStorage::FRowHandleArray* ChangedRowsToAppendTo);

	// Recursive internal method to set visibility on a row and all its children.
	UE_API void SetVisibility_Recursive(UE::Editor::DataStorage::ICoreProvider& DataStorage, RowHandle Row,
		TSharedPtr<UE::Editor::DataStorage::IHierarchyViewerDataInterface> HierarchyData, bool bVisible,
		UE::Editor::DataStorage::FRowHandleArray* ChangedRowsToAppendTo);

	// Legacy method to work on Outliner tree items since they don't use the new hierarchy interface yet.
	UE_API void SetVisibility_Recursive(UE::Editor::DataStorage::ICoreProvider& DataStorage, FSceneOutlinerTreeItemPtr TreeItem, bool bVisible,
		UE::Editor::DataStorage::FRowHandleArray* ChangedRowsToAppendTo);

	// Returns the tree item associated with this widget.
	UE_API FSceneOutlinerTreeItemPtr GetTreeItem(RowHandle InRow) const;

	static UE_API UE::Editor::DataStorage::ICoreProvider* GetDataStorage();
	static UE_API UE::Editor::DataStorage::ICoreProvider* GetDataStorageUI();
	static UE_API UE::Editor::DataStorage::ICompatibilityProvider* GetDataStorageCompatibility();

	UE_API void GetSelectedRows(TArray<RowHandle>& OutSelectedRows) const;

	TWeakObjectPtr<const UScriptStruct> EditorObjectHierarchyParentTag;
};

#undef UE_API
