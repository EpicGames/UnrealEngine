// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/OutlinerTextWidget.h"

#include "OutlinerWorldLevelLabelWidget.generated.h"

UCLASS()
class UOutlinerWorldLevelLabelWidgetFactory : public UEditorDataStorageFactory
{
	GENERATED_BODY()

public:
	virtual ~UOutlinerWorldLevelLabelWidgetFactory() override = default;

	virtual void RegisterWidgetConstructors(UE::Editor::DataStorage::ICoreProvider& DataStorage,
		UE::Editor::DataStorage::IUiProvider& DataStorageUi) const override;
};

USTRUCT()
struct FOutlinerWorldLevelLabelWidgetConstructor : public FOutlinerTextWidgetConstructor
{
	GENERATED_BODY()

public:
	TEDSOUTLINER_API FOutlinerWorldLevelLabelWidgetConstructor();
	TEDSOUTLINER_API FOutlinerWorldLevelLabelWidgetConstructor(const UScriptStruct* InTypeInfo);
	virtual ~FOutlinerWorldLevelLabelWidgetConstructor() override = default;

protected:
	TEDSOUTLINER_API virtual TSharedPtr<SWidget> CreateEditableWidget(
		UE::Editor::DataStorage::ICoreProvider* DataStorage,
		UE::Editor::DataStorage::IUiProvider* DataStorageUi,
		UE::Editor::DataStorage::RowHandle TargetRow,
		UE::Editor::DataStorage::RowHandle WidgetRow,
		const UE::Editor::DataStorage::FMetaDataView& Arguments) override;
};
