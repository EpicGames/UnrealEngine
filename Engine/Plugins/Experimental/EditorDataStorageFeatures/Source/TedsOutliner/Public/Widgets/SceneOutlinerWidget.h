// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/Interfaces/TypedElementDataStorageUiInterface.h"
#include "Elements/Interfaces/TypedElementDataStorageFactory.h"
#include "Elements/Interfaces/TypedElementDataStorageInterface.h"
#include "Elements/Framework/TypedElementQueryContext.h"
#include "Delegates/Delegate.h"
#include "ISceneOutliner.h"
#include "TedsOutlinerFilter.h"

#include "SceneOutlinerWidget.generated.h"

#define UE_API TEDSOUTLINER_API

class IWorldPartitionEditorModule;
class FExtender;
class FMenuBuilder;
class SHorizontalBox;
struct FSceneOutlinerInitializationOptions;

namespace UE::Editor::Outliner
{
	class FTedsSceneOutlinerWorldFilter;
	class ITedsOutlinerHierarchyDataInterface;
	class FTedsOutlinerFilter;
	struct FTedsOutlinerColumnDescription;
	struct FTedsOutlinerParams;
}

namespace UE::Editor::DataStorage
{
	class ICoreProvider;
}

class UTypedElementSelectionSet;

UCLASS()
class UTedsOutlinerWidgetFactory : public UEditorDataStorageFactory
{
	GENERATED_BODY()

public:
	~UTedsOutlinerWidgetFactory() override = default;

	UE_API void RegisterWidgetPurposes(UE::Editor::DataStorage::IUiProvider& DataStorageUi) const override;

	UE_API void RegisterWidgetConstructors(UE::Editor::DataStorage::ICoreProvider& DataStorage,
		UE::Editor::DataStorage::IUiProvider& DataStorageUi) const override;

	/** Allows external filters to be provided to the Outliner */
	using FExternalFilterProvider = TFunction<void(TArray<TSharedPtr<UE::Editor::Outliner::FTedsOutlinerFilter>>&, UE::Editor::DataStorage::ICoreProvider*)>;
	UE_API void RegisterExternalFilterProvider(FName ProviderName, FExternalFilterProvider Provider);
	UE_API void UnregisterExternalFilterProvider(FName ProviderName);
	UE_API void CallExternalFilterProviders(TArray<TSharedPtr<UE::Editor::Outliner::FTedsOutlinerFilter>>& Filters, UE::Editor::DataStorage::ICoreProvider* DataStorage) const;

private:
	TMap<FName, FExternalFilterProvider> ExternalFilterProviders;
};

USTRUCT()
struct FTedsSceneOutlinerWidgetConstructor : public FSimpleWidgetConstructor
{
	GENERATED_BODY()

public:
	TEDSOUTLINER_API FTedsSceneOutlinerWidgetConstructor();
	~FTedsSceneOutlinerWidgetConstructor() override = default;

	/**
	 * Constructs the Outliner Widget. If overridden, ensure that the resulting SSceneOutliner is
	 * referenced in an added FSceneOutlinerColumn on the WidgetRow and the TabIdentifier is gotten via metadata
	 */
	TEDSOUTLINER_API virtual TSharedPtr<SWidget> CreateWidget(
		UE::Editor::DataStorage::ICoreProvider* DataStorage,
		UE::Editor::DataStorage::IUiProvider* DataStorageUi,
		UE::Editor::DataStorage::RowHandle TargetRow,
		UE::Editor::DataStorage::RowHandle WidgetRow,
		const UE::Editor::DataStorage::FMetaDataView& Arguments) override;

	/** Called after the Outliner and its row are created. Override to perform post-creation setup such as adding the settings cache column */
	TEDSOUTLINER_API virtual void PostOutlinerCreated(UE::Editor::DataStorage::ICoreProvider* DataStorage, UE::Editor::DataStorage::RowHandle OutlinerRow);
	/** Adds parameters to the default (non-TEDS) SceneOutlinerInitializationOptions used to create the SSceneOutliner */
	TEDSOUTLINER_API virtual void GetSceneOutlinerInitializationOptions(FSceneOutlinerInitializationOptions& InitOptions);
	/** Gets the query description that will be used to populate rows in this Teds Outliner */
	TEDSOUTLINER_API virtual UE::Editor::DataStorage::FQueryDescription GetQueryDescription() const;
	/** Gets the columns to display in this Teds Outliner and any additional metadata */
	TEDSOUTLINER_API virtual UE::Editor::Outliner::FTedsOutlinerColumnDescription GetColumnDescription() const;
	/** Adds FTedsOutlinerFilters to an array used to populate the Outliner FilterBar */
	TEDSOUTLINER_API virtual void GetFilters(UE::Editor::DataStorage::ICoreProvider* DataStorage,
		TArray<TSharedPtr<UE::Editor::Outliner::FTedsOutlinerFilter>>& Filters) const;
	/** Adds FTedsOutlinerFilters to an array used to populate the Outliner Show Options */
	TEDSOUTLINER_API virtual void GetShowOptionsFilters(UE::Editor::DataStorage::ICoreProvider* DataStorage,
		TMap<FName, TSharedPtr<UE::Editor::Outliner::FTedsOutlinerFilter>>& Filters) const;
	TEDSOUTLINER_API virtual void GetInitializeViewMenuExtender(UE::Editor::Outliner::FTedsOutlinerParams& OutlinerParams);
	/** Binds the delegate called before default double-click behavior */
	TEDSOUTLINER_API virtual void GetOnItemDoubleClick(UE::Editor::Outliner::FTedsOutlinerParams& OutlinerParams);
	/** Binds the delegate called during Tick for whether the Outliner should Populate */
	TEDSOUTLINER_API virtual void GetOnCanPopulate(UE::Editor::Outliner::FTedsOutlinerParams& OutlinerParams);
	/** Binds the delegate that injects entries into the "Options" section of the View Menu */
	TEDSOUTLINER_API virtual void GetInitializeOptionsMenuExtender(UE::Editor::Outliner::FTedsOutlinerParams& OutlinerParams, UE::Editor::DataStorage::ICoreProvider* DataStorage);
	/** Binds the delegate called on item expansion state change */
	TEDSOUTLINER_API virtual void GetExpansionStateBridge(UE::Editor::Outliner::FTedsOutlinerParams& OutlinerParams);
	/** Gets any additional observer queries */
	TEDSOUTLINER_API virtual void GetAdditionalObserverQueries(UE::Editor::Outliner::FTedsOutlinerParams& OutlinerParams);
	/** Binds the delegate fired by the mode's CustomAddToToolbar so the widget can extend the outliner toolbar */
	TEDSOUTLINER_API virtual void GetOnCustomAddToToolbar(UE::Editor::Outliner::FTedsOutlinerParams& OutlinerParams, UE::Editor::DataStorage::ICoreProvider* DataStorage);
	/** Registers interactive filters */
	TEDSOUTLINER_API virtual void GetOnRegisterInteractiveFilters(UE::Editor::Outliner::FTedsOutlinerParams& OutlinerParams, UE::Editor::DataStorage::ICoreProvider* DataStorage);
	/** Whether we should add the Row Handle column to the Outliner, by default this is controlled by a CVar */
	TEDSOUTLINER_API virtual bool ShowRowHandleColumn() const;
	/** Whether Parent rows should be force added back when filtered or searched out */
	TEDSOUTLINER_API virtual bool ForceShowParents() const;
	/** Whether the Teds Outliner will create observers to track addition/removal of rows and update */
	TEDSOUTLINER_API virtual bool UseDefaultObservers() const;
	/** Whether the Teds Outliner will create a view button (settings cog) pre-populated with hierarchy actions */
	TEDSOUTLINER_API virtual bool ShowViewButton() const;
	/** Gets the defined hierarchy to show, if nullptr, a flat list will be presented */
	TEDSOUTLINER_API virtual TSharedPtr<UE::Editor::Outliner::ITedsOutlinerHierarchyDataInterface> GetHierarchyData(UE::Editor::DataStorage::ICoreProvider* DataStorage) const;
	/** Gets the selection set to use for this Outliner, if nullptr it doesn't propagate tree selection through TypedElementSelectionInterface */
	TEDSOUTLINER_API virtual UTypedElementSelectionSet* GetSelectionSet() const;
	/** Gets a conversion map to map legacy Outliner filters to TEDS Filters, if a filter is not mapped, it will not appear in the add filter menu */
	TEDSOUTLINER_API virtual TMap<FName, TVariant<UE::Editor::DataStorage::QueryHandle, UE::Editor::DataStorage::Queries::TConstQueryFunction<bool>>>
		GetLegacyFilterConversionMap(UE::Editor::DataStorage::ICoreProvider* DataStorage) const;

protected:
	/** Gets the FSceneOutlinerInitializationOptions defined in the Level Editor for the Scene Outliner to be used as the default */
	TEDSOUTLINER_API void GetLevelEditorSceneOutlinerInitOptions(FSceneOutlinerInitializationOptions& OutInitOptions) const;
	// Tab Id that can be used by the parameter creation function
	FName TabIdentifier;
	
private:
	void AddFavoriteFilters(UE::Editor::DataStorage::ICoreProvider* DataStorage, TArray<TSharedPtr<UE::Editor::Outliner::FTedsOutlinerFilter>>& Filters) const;
	void AddAlertFilters(UE::Editor::DataStorage::ICoreProvider* DataStorage, TArray<TSharedPtr<UE::Editor::Outliner::FTedsOutlinerFilter>>& Filters) const;
	void AddSCCFilters(UE::Editor::DataStorage::ICoreProvider* DataStorage, TArray<TSharedPtr<UE::Editor::Outliner::FTedsOutlinerFilter>>& Filters) const;

	// Store the World filter so it can be referenced by the World Picker post-creation
	TSharedPtr<UE::Editor::Outliner::FTedsSceneOutlinerWorldFilter> WorldFilter;
};

class FTedsSceneOutlinerActorEditorContextSubsystemFilter : public UE::Editor::Outliner::FTedsOutlinerFilter
{
public:
	TEDSOUTLINER_API FTedsSceneOutlinerActorEditorContextSubsystemFilter(const FName& InFilterName, const FText InFilterDisplayName, const FText InFilterTooltip,
		const UE::Editor::DataStorage::Queries::TConstQueryFunction<bool>& InFilterQuery);

	TEDSOUTLINER_API ~FTedsSceneOutlinerActorEditorContextSubsystemFilter();

	TEDSOUTLINER_API virtual void ActiveStateChanged(bool bActive) override;
	TEDSOUTLINER_API void OnActorEditorContextSubsystemChanged() const;

protected:
	IWorldPartitionEditorModule* WorldPartitionModule = nullptr;
	UE::Editor::DataStorage::ICoreProvider* DataStorage = nullptr;
	
private:
	FDelegateHandle OnActorEditorContextSubsystemChangedDelegate;
};

class FTedsSceneOutlinerCurrentDataLayerFilter : public FTedsSceneOutlinerActorEditorContextSubsystemFilter
{
public:
	TEDSOUTLINER_API FTedsSceneOutlinerCurrentDataLayerFilter(const FName& InFilterName);
};

class FTedsSceneOutlinerCurrentContentBundleFilter : public FTedsSceneOutlinerActorEditorContextSubsystemFilter
{
public:
	TEDSOUTLINER_API FTedsSceneOutlinerCurrentContentBundleFilter(const FName& InFilterName);
};

#undef UE_API
