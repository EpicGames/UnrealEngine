// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/Interfaces/TypedElementDataStorageFactory.h"
#include "Elements/Interfaces/TypedElementDataStorageInterface.h"
#include "Elements/Interfaces/TypedElementDataStorageUiInterface.h"
#include "Elements/Columns/TypedElementTypeInfoColumns.h"

#include "TypeInfoWidget.generated.h"

#define UE_API TEDSOUTLINER_API

namespace UE::Editor::DataStorage
{
	class FTypeInfoWidgetSorter final : public FColumnSorterInterface
	{
	public:
		virtual FText GetShortName() const override;
		virtual ESortType GetSortType() const override;
		virtual int32 Compare(const ICoreProvider& Storage, RowHandle Left, RowHandle Right) const override;
		virtual FPrefixInfo CalculatePrefix(const ICoreProvider& Storage, RowHandle Row, uint32 ByteIndex) const override;
	};

	class FTypeInfoWidgetSearcher final : public FColumnSearcherInterface
	{
	public:
		virtual void GetSearchableString(FString& SearchableString, const ICoreProvider& Storage, RowHandle Row) const override;
	};
}

UCLASS()
class UTypeInfoWidgetFactory : public UEditorDataStorageFactory
{
	GENERATED_BODY()

public:
	~UTypeInfoWidgetFactory() override = default;

	void RegisterWidgetConstructors(UE::Editor::DataStorage::ICoreProvider& DataStorage,
		UE::Editor::DataStorage::IUiProvider& DataStorageUi) const override;
};

USTRUCT()
struct FTypeInfoWidgetConstructor : public FSimpleWidgetConstructor
{
	GENERATED_BODY()

public:
	UE_API FTypeInfoWidgetConstructor();
	~FTypeInfoWidgetConstructor() override = default;

protected:
	UE_API explicit FTypeInfoWidgetConstructor(const UScriptStruct* InTypeInfo);
	
	UE_API virtual TSharedPtr<SWidget> CreateWidget(
		UE::Editor::DataStorage::ICoreProvider* DataStorage,
		UE::Editor::DataStorage::IUiProvider* DataStorageUi,
		UE::Editor::DataStorage::RowHandle TargetRow,
		UE::Editor::DataStorage::RowHandle WidgetRow,
		const UE::Editor::DataStorage::FMetaDataView& Arguments) override;
	
	UE_API TArray<TSharedPtr<const UE::Editor::DataStorage::FColumnSorterInterface>> ConstructColumnSorters(
		UE::Editor::DataStorage::ICoreProvider* DataStorage,
		UE::Editor::DataStorage::IUiProvider* DataStorageUi,
		const UE::Editor::DataStorage::FMetaDataView& Arguments) override;

	UE_API TArray<TSharedPtr<const UE::Editor::DataStorage::FColumnSearcherInterface>> ConstructColumnSearchers(
		UE::Editor::DataStorage::ICoreProvider* DataStorage,
		UE::Editor::DataStorage::IUiProvider* DataStorageUi,
		const UE::Editor::DataStorage::FMetaDataView& Arguments) override;

protected:

	static TSharedRef<SWidget> CreateTextWidget(UE::Editor::DataStorage::ICoreProvider* DataStorage, UE::Editor::DataStorage::RowHandle TargetRow, UE::Editor::DataStorage::RowHandle WidgetRow);
};

#undef UE_API
