// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class TedsOutliner : ModuleRules
{
	public TedsOutliner(ReadOnlyTargetRules Target) : base(Target)
	{
		// Enable truncation warnings in this module
		CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Warning;

		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		if (Target.bBuildEditor)
		{
			PublicIncludePaths.AddRange(new string[] {});
			PrivateIncludePaths.AddRange(new string[] {});

			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
					"EditorConfig",
					"EditorFramework",
					"EditorWidgets",
					"Engine",
					"SceneOutliner",
					"Slate",
					"SlateCore",
					"TedsTableViewer",
					"TypedElementFramework",
					"TypedElementRuntime"
				});
			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"WorkspaceMenuStructure",
					"UnrealEd", // FEditorUndoClient used by SSceneOutliner
					"ToolMenus",
					"ApplicationCore",
					"TedsAlerts",
					"TedsTableViewer",
					"ToolWidgets",
					"InputCore",
					"LevelEditor",
					"TedsQueryStack",
					"TedsActorCompatibility",
					"TedsAssetData",
					"TedsOperations",
					"LevelInstanceEditor",
					"TedsEditorCompatibility",
					"TedsEverythingPicker"
				});

			DynamicallyLoadedModuleNames.AddRange(new string[] {});
		}
	}
}
