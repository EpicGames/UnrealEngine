// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_TESTS

#include "Elements/Framework/TypedElementTestColumns.h"
#include "Misc/AutomationTest.h"
#include "TedsOutlinerHelpers.h"
#include "TedsOutlinerImpl.h"
#include "Tests/Assertions.h"
#include "Tests/TestHarnessAdapter.h"

static uint8 GetPriorityFromMap(const TWeakObjectPtr<const UScriptStruct>& ColumnType, const TMap<TWeakObjectPtr<const UScriptStruct>, uint8>& PriorityMap)
{
	if(const uint8* Priority = PriorityMap.Find(ColumnType))
	{
		return *Priority;
	}
	return UINT8_MAX;
}

TEST_CASE_NAMED(TEDS_Outliner_Column_Tests, "Editor::DataStorage::Outliner::ColumnOrder", "[ApplicationContextMask][EngineFilter]")
{
	using namespace UE::Editor::DataStorage;
	using namespace UE::Editor::Outliner;
	using FTedsOutlinerColumnDescription = UE::Editor::Outliner::FTedsOutlinerColumnDescription;
	using FTedsOutlinerColumnParams = UE::Editor::Outliner::FTedsOutlinerColumnParams;
	using EColumnPriorityGroup = FTedsOutlinerColumnParams::EColumnPriorityGroup;
	using FColumnPriorityRelation = FTedsOutlinerColumnParams::FColumnPriorityRelation;
	using EColumnPriorityRelation = FColumnPriorityRelation::EColumnPriorityRelation;

	const TArray<TWeakObjectPtr<const UScriptStruct>> TestColumns({
		FTestColumnA::StaticStruct(),
		FTestColumnB::StaticStruct(),
		FTestColumnC::StaticStruct(),
		FTestColumnD::StaticStruct(),
		FTestColumnE::StaticStruct(),
		FTestColumnF::StaticStruct(),
		FTestColumnG::StaticStruct()});
	TMap<TWeakObjectPtr<const UScriptStruct>, uint8> OrderedColumnMap;

	SECTION("No order specified")
	{
		const FTedsOutlinerColumnDescription TestColumnDescription = FTedsOutlinerColumnDescription(TestColumns);
		OrderedColumnMap.Reset();
		Helpers::OrderColumns(TestColumns, TestColumnDescription, OrderedColumnMap);

		CHECK_EQUALS(TEXT("Priority Map Size"), OrderedColumnMap.Num(), 7);
		CHECK_EQUALS(TEXT("Priority[Column A]"), GetPriorityFromMap(FTestColumnA::StaticStruct(), OrderedColumnMap), 85);
		CHECK_EQUALS(TEXT("Priority[Column B]"), GetPriorityFromMap(FTestColumnB::StaticStruct(), OrderedColumnMap), 86);
		CHECK_EQUALS(TEXT("Priority[Column C]"), GetPriorityFromMap(FTestColumnC::StaticStruct(), OrderedColumnMap), 87);
		CHECK_EQUALS(TEXT("Priority[Column D]"), GetPriorityFromMap(FTestColumnD::StaticStruct(), OrderedColumnMap), 88);
		CHECK_EQUALS(TEXT("Priority[Column E]"), GetPriorityFromMap(FTestColumnE::StaticStruct(), OrderedColumnMap), 89);
		CHECK_EQUALS(TEXT("Priority[Column F]"), GetPriorityFromMap(FTestColumnF::StaticStruct(), OrderedColumnMap), 90);
		CHECK_EQUALS(TEXT("Priority[Column G]"), GetPriorityFromMap(FTestColumnG::StaticStruct(), OrderedColumnMap), 91);
	}

	SECTION("Order by group")
	{
		const TMap<TWeakObjectPtr<const UScriptStruct>, FTedsOutlinerColumnParams> TestColumnParams
		{
			{FTestColumnA::StaticStruct(), FTedsOutlinerColumnParams(EColumnPriorityGroup::Right)},
			{FTestColumnB::StaticStruct(), FTedsOutlinerColumnParams(EColumnPriorityGroup::Left)},
			{FTestColumnC::StaticStruct(), FTedsOutlinerColumnParams(EColumnPriorityGroup::Middle)},
			{FTestColumnD::StaticStruct(), FTedsOutlinerColumnParams(EColumnPriorityGroup::Left)},
			{FTestColumnE::StaticStruct(), FTedsOutlinerColumnParams(EColumnPriorityGroup::Right)},
			{FTestColumnF::StaticStruct(), FTedsOutlinerColumnParams(EColumnPriorityGroup::Middle)},
			{FTestColumnG::StaticStruct(), FTedsOutlinerColumnParams(EColumnPriorityGroup::Left)},
		};
		const FTedsOutlinerColumnDescription TestColumnDescription = FTedsOutlinerColumnDescription(TestColumns, TestColumnParams);
		OrderedColumnMap.Reset();
		Helpers::OrderColumns(TestColumns, TestColumnDescription, OrderedColumnMap);

		CHECK_EQUALS(TEXT("Priority Map Size"), OrderedColumnMap.Num(), 7);
		CHECK_EQUALS(TEXT("Priority[Column B]"), GetPriorityFromMap(FTestColumnB::StaticStruct(), OrderedColumnMap), 0);
		CHECK_EQUALS(TEXT("Priority[Column D]"), GetPriorityFromMap(FTestColumnD::StaticStruct(), OrderedColumnMap), 1);
		CHECK_EQUALS(TEXT("Priority[Column G]"), GetPriorityFromMap(FTestColumnG::StaticStruct(), OrderedColumnMap), 2);
		CHECK_EQUALS(TEXT("Priority[Column C]"), GetPriorityFromMap(FTestColumnC::StaticStruct(), OrderedColumnMap), 85);
		CHECK_EQUALS(TEXT("Priority[Column F]"), GetPriorityFromMap(FTestColumnF::StaticStruct(), OrderedColumnMap), 86);
		CHECK_EQUALS(TEXT("Priority[Column A]"), GetPriorityFromMap(FTestColumnA::StaticStruct(), OrderedColumnMap), 170);
		CHECK_EQUALS(TEXT("Priority[Column E]"), GetPriorityFromMap(FTestColumnE::StaticStruct(), OrderedColumnMap), 171);
	}

	SECTION("Order by group with before relations")
	{
		const TMap<TWeakObjectPtr<const UScriptStruct>, FTedsOutlinerColumnParams> TestColumnParams
		{
			{FTestColumnA::StaticStruct(), FTedsOutlinerColumnParams(EColumnPriorityGroup::Left)},
			{FTestColumnB::StaticStruct(), FTedsOutlinerColumnParams(EColumnPriorityGroup::Left,
				FColumnPriorityRelation(EColumnPriorityRelation::Before, FTestColumnA::StaticStruct()))},
			{FTestColumnC::StaticStruct(), FTedsOutlinerColumnParams(EColumnPriorityGroup::Middle)},
			{FTestColumnD::StaticStruct(), FTedsOutlinerColumnParams(EColumnPriorityGroup::Middle,
				FColumnPriorityRelation(EColumnPriorityRelation::Before, FTestColumnC::StaticStruct()))},
			{FTestColumnE::StaticStruct(), FTedsOutlinerColumnParams(EColumnPriorityGroup::Left,
				FColumnPriorityRelation(EColumnPriorityRelation::Before, FTestColumnB::StaticStruct()))},
			{FTestColumnF::StaticStruct(), FTedsOutlinerColumnParams(EColumnPriorityGroup::Middle,
				FColumnPriorityRelation(EColumnPriorityRelation::Before, FTestColumnC::StaticStruct()))},
			{FTestColumnG::StaticStruct(), FTedsOutlinerColumnParams(EColumnPriorityGroup::Left, 
				FColumnPriorityRelation(EColumnPriorityRelation::Before, FTestColumnE::StaticStruct()))},
		};
		const FTedsOutlinerColumnDescription TestColumnDescription = FTedsOutlinerColumnDescription(TestColumns, TestColumnParams);
		OrderedColumnMap.Reset();
		Helpers::OrderColumns(TestColumns, TestColumnDescription, OrderedColumnMap);

		CHECK_EQUALS(TEXT("Priority Map Size"), OrderedColumnMap.Num(), 7);
		CHECK_EQUALS(TEXT("Priority[Column G]"), GetPriorityFromMap(FTestColumnG::StaticStruct(), OrderedColumnMap), 0);
		CHECK_EQUALS(TEXT("Priority[Column E]"), GetPriorityFromMap(FTestColumnE::StaticStruct(), OrderedColumnMap), 1);
		CHECK_EQUALS(TEXT("Priority[Column B]"), GetPriorityFromMap(FTestColumnB::StaticStruct(), OrderedColumnMap), 2);
		CHECK_EQUALS(TEXT("Priority[Column A]"), GetPriorityFromMap(FTestColumnA::StaticStruct(), OrderedColumnMap), 3);
		CHECK_EQUALS(TEXT("Priority[Column D]"), GetPriorityFromMap(FTestColumnD::StaticStruct(), OrderedColumnMap), 85);
		CHECK_EQUALS(TEXT("Priority[Column F]"), GetPriorityFromMap(FTestColumnF::StaticStruct(), OrderedColumnMap), 86);
		CHECK_EQUALS(TEXT("Priority[Column C]"), GetPriorityFromMap(FTestColumnC::StaticStruct(), OrderedColumnMap), 87);	
	}

	SECTION("Order by group with after relations")
	{
		const TMap<TWeakObjectPtr<const UScriptStruct>, FTedsOutlinerColumnParams> TestColumnParams
		{
			{FTestColumnA::StaticStruct(), FTedsOutlinerColumnParams(EColumnPriorityGroup::Left,
				FColumnPriorityRelation(EColumnPriorityRelation::After, FTestColumnB::StaticStruct()))},
			{FTestColumnB::StaticStruct(), FTedsOutlinerColumnParams(EColumnPriorityGroup::Left)},
			{FTestColumnC::StaticStruct(), FTedsOutlinerColumnParams(EColumnPriorityGroup::Middle,
				FColumnPriorityRelation(EColumnPriorityRelation::After, FTestColumnE::StaticStruct()))},
			{FTestColumnD::StaticStruct(), FTedsOutlinerColumnParams(EColumnPriorityGroup::Middle,
				FColumnPriorityRelation(EColumnPriorityRelation::After, FTestColumnC::StaticStruct()))},
			{FTestColumnE::StaticStruct(), FTedsOutlinerColumnParams(EColumnPriorityGroup::Middle,
				FColumnPriorityRelation(EColumnPriorityRelation::After, FTestColumnF::StaticStruct()))},
			{FTestColumnF::StaticStruct(), FTedsOutlinerColumnParams(EColumnPriorityGroup::Middle)},
			{FTestColumnG::StaticStruct(), FTedsOutlinerColumnParams(EColumnPriorityGroup::Middle, 
				FColumnPriorityRelation(EColumnPriorityRelation::After, FTestColumnC::StaticStruct()))},
		};
		const FTedsOutlinerColumnDescription TestColumnDescription = FTedsOutlinerColumnDescription(TestColumns, TestColumnParams);
		OrderedColumnMap.Reset();
		Helpers::OrderColumns(TestColumns, TestColumnDescription, OrderedColumnMap);

		CHECK_EQUALS(TEXT("Priority Map Size"), OrderedColumnMap.Num(), 7);
		CHECK_EQUALS(TEXT("Priority[Column B]"), GetPriorityFromMap(FTestColumnB::StaticStruct(), OrderedColumnMap), 0);
		CHECK_EQUALS(TEXT("Priority[Column A]"), GetPriorityFromMap(FTestColumnA::StaticStruct(), OrderedColumnMap), 1);
		CHECK_EQUALS(TEXT("Priority[Column F]"), GetPriorityFromMap(FTestColumnF::StaticStruct(), OrderedColumnMap), 85);
		CHECK_EQUALS(TEXT("Priority[Column E]"), GetPriorityFromMap(FTestColumnE::StaticStruct(), OrderedColumnMap), 86);
		CHECK_EQUALS(TEXT("Priority[Column C]"), GetPriorityFromMap(FTestColumnC::StaticStruct(), OrderedColumnMap), 87);
		CHECK_EQUALS(TEXT("Priority[Column D]"), GetPriorityFromMap(FTestColumnD::StaticStruct(), OrderedColumnMap), 88);
		CHECK_EQUALS(TEXT("Priority[Column G]"), GetPriorityFromMap(FTestColumnG::StaticStruct(), OrderedColumnMap), 89);
	}

	SECTION("Order by group with before and after relations")
	{
		const TMap<TWeakObjectPtr<const UScriptStruct>, FTedsOutlinerColumnParams> TestColumnParams
		{
			{FTestColumnA::StaticStruct(), FTedsOutlinerColumnParams(EColumnPriorityGroup::Right,
				FColumnPriorityRelation(EColumnPriorityRelation::After, FTestColumnF::StaticStruct()))},
			{FTestColumnB::StaticStruct(), FTedsOutlinerColumnParams(EColumnPriorityGroup::Middle,
				FColumnPriorityRelation(EColumnPriorityRelation::After, FTestColumnD::StaticStruct()))},
			{FTestColumnC::StaticStruct(), FTedsOutlinerColumnParams(EColumnPriorityGroup::Left)},
			{FTestColumnD::StaticStruct(), FTedsOutlinerColumnParams(EColumnPriorityGroup::Middle)},
			{FTestColumnE::StaticStruct(), FTedsOutlinerColumnParams(EColumnPriorityGroup::Left,
				FColumnPriorityRelation(EColumnPriorityRelation::Before, FTestColumnC::StaticStruct()))},
			{FTestColumnF::StaticStruct(), FTedsOutlinerColumnParams(EColumnPriorityGroup::Right,
				FColumnPriorityRelation(EColumnPriorityRelation::Before, FTestColumnG::StaticStruct()))},
			{FTestColumnG::StaticStruct(), FTedsOutlinerColumnParams(EColumnPriorityGroup::Right)},
		};
		const FTedsOutlinerColumnDescription TestColumnDescription = FTedsOutlinerColumnDescription(TestColumns, TestColumnParams);
		OrderedColumnMap.Reset();
		Helpers::OrderColumns(TestColumns, TestColumnDescription, OrderedColumnMap);

		CHECK_EQUALS(TEXT("Priority Map Size"), OrderedColumnMap.Num(), 7);
		CHECK_EQUALS(TEXT("Priority[Column E]"), GetPriorityFromMap(FTestColumnE::StaticStruct(), OrderedColumnMap), 0);
		CHECK_EQUALS(TEXT("Priority[Column C]"), GetPriorityFromMap(FTestColumnC::StaticStruct(), OrderedColumnMap), 1);
		CHECK_EQUALS(TEXT("Priority[Column D]"), GetPriorityFromMap(FTestColumnD::StaticStruct(), OrderedColumnMap), 85);
		CHECK_EQUALS(TEXT("Priority[Column B]"), GetPriorityFromMap(FTestColumnB::StaticStruct(), OrderedColumnMap), 86);
		CHECK_EQUALS(TEXT("Priority[Column F]"), GetPriorityFromMap(FTestColumnF::StaticStruct(), OrderedColumnMap), 170);
		CHECK_EQUALS(TEXT("Priority[Column A]"), GetPriorityFromMap(FTestColumnA::StaticStruct(), OrderedColumnMap), 171);	
		CHECK_EQUALS(TEXT("Priority[Column G]"), GetPriorityFromMap(FTestColumnG::StaticStruct(), OrderedColumnMap), 172);
	}

	SECTION("Order by group with before and after relations on the same column")
	{
		const TMap<TWeakObjectPtr<const UScriptStruct>, FTedsOutlinerColumnParams> TestColumnParams
		{
			{FTestColumnA::StaticStruct(), FTedsOutlinerColumnParams(EColumnPriorityGroup::Middle)},
			{FTestColumnB::StaticStruct(), FTedsOutlinerColumnParams(EColumnPriorityGroup::Middle,
				FColumnPriorityRelation(EColumnPriorityRelation::After, FTestColumnA::StaticStruct()))},
			{FTestColumnC::StaticStruct(), FTedsOutlinerColumnParams(EColumnPriorityGroup::Middle,
				FColumnPriorityRelation(EColumnPriorityRelation::After, FTestColumnA::StaticStruct()))},
			{FTestColumnD::StaticStruct(), FTedsOutlinerColumnParams(EColumnPriorityGroup::Middle,
				FColumnPriorityRelation(EColumnPriorityRelation::Before, FTestColumnA::StaticStruct()))},
			{FTestColumnE::StaticStruct(), FTedsOutlinerColumnParams(EColumnPriorityGroup::Middle,
				FColumnPriorityRelation(EColumnPriorityRelation::Before, FTestColumnA::StaticStruct()))},
			{FTestColumnF::StaticStruct(), FTedsOutlinerColumnParams(EColumnPriorityGroup::Middle,
				FColumnPriorityRelation(EColumnPriorityRelation::Before, FTestColumnA::StaticStruct()))},
			{FTestColumnG::StaticStruct(), FTedsOutlinerColumnParams(EColumnPriorityGroup::Middle,
				FColumnPriorityRelation(EColumnPriorityRelation::After, FTestColumnA::StaticStruct()))},
		};
		const FTedsOutlinerColumnDescription TestColumnDescription = FTedsOutlinerColumnDescription(TestColumns, TestColumnParams);
		OrderedColumnMap.Reset();
		Helpers::OrderColumns(TestColumns, TestColumnDescription, OrderedColumnMap);

		CHECK_EQUALS(TEXT("Priority Map Size"), OrderedColumnMap.Num(), 7);
		CHECK_EQUALS(TEXT("Priority[Column D]"), GetPriorityFromMap(FTestColumnD::StaticStruct(), OrderedColumnMap), 85);
		CHECK_EQUALS(TEXT("Priority[Column E]"), GetPriorityFromMap(FTestColumnE::StaticStruct(), OrderedColumnMap), 86);
		CHECK_EQUALS(TEXT("Priority[Column F]"), GetPriorityFromMap(FTestColumnF::StaticStruct(), OrderedColumnMap), 87);
		CHECK_EQUALS(TEXT("Priority[Column A]"), GetPriorityFromMap(FTestColumnA::StaticStruct(), OrderedColumnMap), 88);
		CHECK_EQUALS(TEXT("Priority[Column B]"), GetPriorityFromMap(FTestColumnB::StaticStruct(), OrderedColumnMap), 89);
		CHECK_EQUALS(TEXT("Priority[Column C]"), GetPriorityFromMap(FTestColumnC::StaticStruct(), OrderedColumnMap), 90);
		CHECK_EQUALS(TEXT("Priority[Column G]"), GetPriorityFromMap(FTestColumnG::StaticStruct(), OrderedColumnMap), 91);
	}
	
	SECTION("Order by group with duplicate column defined parameters")
    {
    	const TMap<TWeakObjectPtr<const UScriptStruct>, FTedsOutlinerColumnParams> TestColumnParams
		{
			{FTestColumnA::StaticStruct(), FTedsOutlinerColumnParams(EColumnPriorityGroup::Right,
				FColumnPriorityRelation(EColumnPriorityRelation::After, FTestColumnF::StaticStruct()))},
			{FTestColumnB::StaticStruct(), FTedsOutlinerColumnParams(EColumnPriorityGroup::Middle,
				FColumnPriorityRelation(EColumnPriorityRelation::After, FTestColumnD::StaticStruct()))},
			{FTestColumnC::StaticStruct(), FTedsOutlinerColumnParams(EColumnPriorityGroup::Left)},
			{FTestColumnD::StaticStruct(), FTedsOutlinerColumnParams(EColumnPriorityGroup::Middle)},
			{FTestColumnE::StaticStruct(), FTedsOutlinerColumnParams(EColumnPriorityGroup::Left,
				FColumnPriorityRelation(EColumnPriorityRelation::Before, FTestColumnC::StaticStruct()))},
			{FTestColumnF::StaticStruct(), FTedsOutlinerColumnParams(EColumnPriorityGroup::Right,
				FColumnPriorityRelation(EColumnPriorityRelation::Before, FTestColumnG::StaticStruct()))},
			{FTestColumnG::StaticStruct(), FTedsOutlinerColumnParams(EColumnPriorityGroup::Right)},
			// Add another definition of TestColumnA's parameters, it should overwrite the previous
			{FTestColumnA::StaticStruct(), FTedsOutlinerColumnParams(EColumnPriorityGroup::Left,
				FColumnPriorityRelation(EColumnPriorityRelation::After, FTestColumnE::StaticStruct()))}
		};
		const FTedsOutlinerColumnDescription TestColumnDescription = FTedsOutlinerColumnDescription(TestColumns, TestColumnParams);
		OrderedColumnMap.Reset();
		Helpers::OrderColumns(TestColumns, TestColumnDescription, OrderedColumnMap);

		CHECK_EQUALS(TEXT("Priority Map Size"), OrderedColumnMap.Num(), 7);
		CHECK_EQUALS(TEXT("Priority[Column E]"), GetPriorityFromMap(FTestColumnE::StaticStruct(), OrderedColumnMap), 0);
		CHECK_EQUALS(TEXT("Priority[Column A]"), GetPriorityFromMap(FTestColumnA::StaticStruct(), OrderedColumnMap), 1);	
		CHECK_EQUALS(TEXT("Priority[Column C]"), GetPriorityFromMap(FTestColumnC::StaticStruct(), OrderedColumnMap), 2);
		CHECK_EQUALS(TEXT("Priority[Column D]"), GetPriorityFromMap(FTestColumnD::StaticStruct(), OrderedColumnMap), 85);
		CHECK_EQUALS(TEXT("Priority[Column B]"), GetPriorityFromMap(FTestColumnB::StaticStruct(), OrderedColumnMap), 86);
		CHECK_EQUALS(TEXT("Priority[Column F]"), GetPriorityFromMap(FTestColumnF::StaticStruct(), OrderedColumnMap), 170);
		CHECK_EQUALS(TEXT("Priority[Column G]"), GetPriorityFromMap(FTestColumnG::StaticStruct(), OrderedColumnMap), 171);
    }

	SECTION("Order by group with before relation to column in a different group")
	{
		const TMap<TWeakObjectPtr<const UScriptStruct>, FTedsOutlinerColumnParams> TestColumnParams
		{
			{FTestColumnA::StaticStruct(), FTedsOutlinerColumnParams(EColumnPriorityGroup::Right)},
			{FTestColumnB::StaticStruct(), FTedsOutlinerColumnParams(EColumnPriorityGroup::Middle)},
			{FTestColumnC::StaticStruct(), FTedsOutlinerColumnParams(EColumnPriorityGroup::Left)},
			{FTestColumnD::StaticStruct(), FTedsOutlinerColumnParams(EColumnPriorityGroup::Middle)},
			{FTestColumnE::StaticStruct(), FTedsOutlinerColumnParams(EColumnPriorityGroup::Left,
				FColumnPriorityRelation(EColumnPriorityRelation::Before, FTestColumnD::StaticStruct()))},
			{FTestColumnF::StaticStruct(), FTedsOutlinerColumnParams(EColumnPriorityGroup::Right)},
			{FTestColumnG::StaticStruct(), FTedsOutlinerColumnParams(EColumnPriorityGroup::Right)},
		};
		const FTedsOutlinerColumnDescription TestColumnDescription = FTedsOutlinerColumnDescription(TestColumns, TestColumnParams);
		OrderedColumnMap.Reset();
		Helpers::OrderColumns(TestColumns, TestColumnDescription, OrderedColumnMap);

		CHECK_EQUALS(TEXT("Priority Map Size"), OrderedColumnMap.Num(), 7);
		CHECK_EQUALS(TEXT("Priority[Column C]"), GetPriorityFromMap(FTestColumnC::StaticStruct(), OrderedColumnMap), 0);
		CHECK_EQUALS(TEXT("Priority[Column E]"), GetPriorityFromMap(FTestColumnE::StaticStruct(), OrderedColumnMap), 1);
		CHECK_EQUALS(TEXT("Priority[Column B]"), GetPriorityFromMap(FTestColumnB::StaticStruct(), OrderedColumnMap), 85);
		CHECK_EQUALS(TEXT("Priority[Column D]"), GetPriorityFromMap(FTestColumnD::StaticStruct(), OrderedColumnMap), 86);
		CHECK_EQUALS(TEXT("Priority[Column A]"), GetPriorityFromMap(FTestColumnA::StaticStruct(), OrderedColumnMap), 170);
		CHECK_EQUALS(TEXT("Priority[Column F]"), GetPriorityFromMap(FTestColumnF::StaticStruct(), OrderedColumnMap), 171);
		CHECK_EQUALS(TEXT("Priority[Column G]"), GetPriorityFromMap(FTestColumnG::StaticStruct(), OrderedColumnMap), 172);
	}

	SECTION("Order by group with relations to invalid columns")
	{
		const TArray<TWeakObjectPtr<const UScriptStruct>> TestColumnsWithInvalidColumn({
			FTestColumnA::StaticStruct(),
			FTestColumnB::StaticStruct(),
			FTestColumnC::StaticStruct(),
			FTestColumnD::StaticStruct(),
			FTestColumnE::StaticStruct(),
			FTestColumnF::StaticStruct(),
			FTestColumnG::StaticStruct(),
			"/Script/TedsStyling.SlateStyleTag"_TypeOptional});
		
		const TMap<TWeakObjectPtr<const UScriptStruct>, FTedsOutlinerColumnParams> TestColumnParams
		{
			{FTestColumnA::StaticStruct(), FTedsOutlinerColumnParams(EColumnPriorityGroup::Right,
				FColumnPriorityRelation(EColumnPriorityRelation::After, "/Script/TedsStyling.SlateStyleTag"_TypeOptional))},
			{FTestColumnB::StaticStruct(), FTedsOutlinerColumnParams(EColumnPriorityGroup::Middle,
				FColumnPriorityRelation(EColumnPriorityRelation::After, FTestColumnD::StaticStruct()))},
			{FTestColumnC::StaticStruct(), FTedsOutlinerColumnParams(EColumnPriorityGroup::Left)},
			{FTestColumnD::StaticStruct(), FTedsOutlinerColumnParams(EColumnPriorityGroup::Middle)},
			{FTestColumnE::StaticStruct(), FTedsOutlinerColumnParams(EColumnPriorityGroup::Left,
				FColumnPriorityRelation(EColumnPriorityRelation::Before, FTestColumnC::StaticStruct()))},
			{FTestColumnF::StaticStruct(), FTedsOutlinerColumnParams(EColumnPriorityGroup::Right,
				FColumnPriorityRelation(EColumnPriorityRelation::Before, "/Script/TedsStyling.SlateStyleTag"_TypeOptional))},
			{FTestColumnG::StaticStruct(), FTedsOutlinerColumnParams(EColumnPriorityGroup::Right)},
			{"/Script/TedsStyling.SlateStyleTag"_TypeOptional, FTedsOutlinerColumnParams(EColumnPriorityGroup::Right,
				FColumnPriorityRelation(EColumnPriorityRelation::After, FTestColumnG::StaticStruct()))}
		};

		const FTedsOutlinerColumnDescription TestColumnDescription = FTedsOutlinerColumnDescription(TestColumns, TestColumnParams);
		OrderedColumnMap.Reset();
		Helpers::OrderColumns(TestColumns, TestColumnDescription, OrderedColumnMap);

		// The invalid column should be ignored and any relations pointing to it will have it ignored
		CHECK_EQUALS(TEXT("Priority Map Size"), OrderedColumnMap.Num(), 7);
		CHECK_EQUALS(TEXT("Priority[Column E]"), GetPriorityFromMap(FTestColumnE::StaticStruct(), OrderedColumnMap), 0);
		CHECK_EQUALS(TEXT("Priority[Column C]"), GetPriorityFromMap(FTestColumnC::StaticStruct(), OrderedColumnMap), 1);
		CHECK_EQUALS(TEXT("Priority[Column D]"), GetPriorityFromMap(FTestColumnD::StaticStruct(), OrderedColumnMap), 85);
		CHECK_EQUALS(TEXT("Priority[Column B]"), GetPriorityFromMap(FTestColumnB::StaticStruct(), OrderedColumnMap), 86);
		CHECK_EQUALS(TEXT("Priority[Column A]"), GetPriorityFromMap(FTestColumnA::StaticStruct(), OrderedColumnMap), 170);
		CHECK_EQUALS(TEXT("Priority[Column F]"), GetPriorityFromMap(FTestColumnF::StaticStruct(), OrderedColumnMap), 171);
		CHECK_EQUALS(TEXT("Priority[Column G]"), GetPriorityFromMap(FTestColumnG::StaticStruct(), OrderedColumnMap), 172);
	}
}

#endif // #if WITH_TESTS
