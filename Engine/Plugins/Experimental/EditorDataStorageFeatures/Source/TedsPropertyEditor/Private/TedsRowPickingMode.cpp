// Copyright Epic Games, Inc. All Rights Reserved.

#include "TedsRowPickingMode.h"

#include "TedsOutlinerItem.h"

FTedsRowPickingMode::FTedsRowPickingMode(const UE::Editor::Outliner::FTedsOutlinerParams& Params, FOnSceneOutlinerItemPicked OnItemPickedDelegate)
	: FTedsOutlinerMode(Params)
	, OnItemPicked(OnItemPickedDelegate)
{

}

void FTedsRowPickingMode::OnItemSelectionChanged(FSceneOutlinerTreeItemPtr Item, ESelectInfo::Type SelectionType, const FSceneOutlinerItemSelection& Selection)
{
	// In picking mode, we fire off the notification to whoever is listening.
	// This may often cause the widget itself to be enqueued for destruction
	auto SelectedItems = SceneOutliner->GetSelectedItems();
	if (SelectedItems.Num() > 0)
	{
		auto FirstItem = SelectedItems[0];
		if (FirstItem->CanInteract())
		{
			OnItemPicked.ExecuteIfBound(FirstItem.ToSharedRef());
		}
	}
}

/** Allow the user to commit their selection by pressing enter if it is valid */
void FTedsRowPickingMode::OnFilterTextCommited(FSceneOutlinerItemSelection& Selection, ETextCommit::Type CommitType)
{
	// In picking mode, we check to see if we have any Typed Element items, and if so, fire
	// off the notification to whoever is listening. This may often cause the widget itself
	// to be enqueued for destruction
	TArray<UE::Editor::Outliner::FTedsOutlinerTreeItem*> OutlinerRows;
	Selection.Get(OutlinerRows);
	if (OutlinerRows.Num() == 1 && OutlinerRows[0])
	{
		// Signal that a Typed Element was selected. We assume it is valid as it won't have been added to Selection if not.
		SceneOutliner->SetItemSelection(OutlinerRows[0]->AsShared(), true, ESelectInfo::OnKeyPress);
	}
}