// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SPropertyMenuTedsRowPicker.h"

#include "DataStorage/Features.h"
#include "TedsRowPickingMode.h"
#include "TedsOutlinerItem.h"

#define LOCTEXT_NAMESPACE "TedsPropertyEditor"

void SPropertyMenuTedsRowPicker::Construct(const FArguments& InArgs)
{
	bAllowClear = InArgs._AllowClear;
	QueryFilter = InArgs._QueryFilter;
	ElementFilter = InArgs._ElementFilter;
	InteractiveFilter = InArgs._InteractiveFilter;
	OnSet = InArgs._OnSet;

	FMenuBuilder MenuBuilder(true, nullptr);

	MenuBuilder.BeginSection(NAME_None, LOCTEXT("CurrentTypedElementOperationsHeader", "Current Element"));
	{
		if (bAllowClear)
		{
			MenuBuilder.AddMenuEntry(
				LOCTEXT("ClearElement", "Clear"),
				LOCTEXT("ClearElement_Tooltip", "Clears the item set on this field"),
				FSlateIcon(),
				FUIAction(FExecuteAction::CreateSP(this, &SPropertyMenuTedsRowPicker::OnClear))
			);
		}
	}
	MenuBuilder.EndSection();

	MenuBuilder.BeginSection(NAME_None, LOCTEXT("BrowseHeader", "Browse"));
	{
		TSharedPtr<SWidget> MenuContent;
		{
			using namespace UE::Editor::Outliner;
			using namespace UE::Editor::DataStorage;

			// TEDS-Outliner TODO: Taken from private implementation of PropertyEditorAssetConstants.
			//                     Should be centralized when TEDS is moved to core
			static const FVector2D ContentBrowserWindowSize(300.0f, 300.0f);
			static const FVector2D SceneOutlinerWindowSize(350.0f, 300.0f);

			if (!AreEditorDataStorageFeaturesEnabled())
			{
				MenuContent = SNew(STextBlock)
					.Text(LOCTEXT("TEDSPluginNotEnabledText", "Typed Element Data Storage plugin required to use this property picker."));
			}
			else
			{
				auto OnItemPicked = FOnSceneOutlinerItemPicked::CreateLambda([&](TSharedRef<ISceneOutlinerTreeItem> Item)
					{
						if (FTedsOutlinerTreeItem* ElementItem = Item->CastTo<FTedsOutlinerTreeItem>())
						{
							if (ElementItem->IsValid())
							{
								OnSet.ExecuteIfBound(ElementItem->GetRowHandle());
							}
						}
					});

				FSceneOutlinerInitializationOptions InitOptions;
				InitOptions.bShowHeaderRow = true;
				InitOptions.bShowTransient = true;
				InitOptions.bShowSearchBox = true;

				InitOptions.ModeFactory = FCreateSceneOutlinerMode::CreateLambda([&](SSceneOutliner* Outliner)
					{
						FTedsOutlinerParams Params(Outliner);
						Params.QueryDescription = QueryFilter;
						Params.bForceShowParents = false;
						return new FTedsRowPickingMode(Params, OnItemPicked);
					});

				InitOptions.Filters->Add(
					MakeShared<TSceneOutlinerPredicateFilter<FTedsOutlinerTreeItem>>(
						FTedsOutlinerTreeItem::FFilterPredicate::CreateLambda([this](const UE::Editor::DataStorage::RowHandle RowHandle) -> bool
							{
								if (ElementFilter.IsBound())
								{
									return ElementFilter.Execute(RowHandle);
								}
								return true;
							}), FSceneOutlinerFilter::EDefaultBehaviour::Pass,
						FTedsOutlinerTreeItem::FInteractivePredicate::CreateLambda([this](const UE::Editor::DataStorage::RowHandle RowHandle) -> bool
							{
								if (InteractiveFilter.IsBound())
								{
									return InteractiveFilter.Execute(RowHandle);
								}
								return true;
							})));

				TSharedPtr<SSceneOutliner> Outliner = SNew(SSceneOutliner, InitOptions);

				MenuContent =
					SNew(SBox)
					.WidthOverride(static_cast<float>(SceneOutlinerWindowSize.X))
					.HeightOverride(static_cast<float>(SceneOutlinerWindowSize.Y))
					[
						Outliner.ToSharedRef()
					];
			}
		}

		MenuBuilder.AddWidget(MenuContent.ToSharedRef(), FText::GetEmpty(), true);
	}
	MenuBuilder.EndSection();

	ChildSlot
	[
		MenuBuilder.MakeWidget()
	];
}

void SPropertyMenuTedsRowPicker::OnClear()
{
	SetValue(UE::Editor::DataStorage::InvalidRowHandle);
	OnClose.ExecuteIfBound();
}

void SPropertyMenuTedsRowPicker::OnElementSelected(UE::Editor::DataStorage::RowHandle RowHandle)
{
	SetValue(RowHandle);
	OnClose.ExecuteIfBound();
}

void SPropertyMenuTedsRowPicker::SetValue(UE::Editor::DataStorage::RowHandle RowHandle)
{
	OnSet.ExecuteIfBound(RowHandle);
}

#undef LOCTEXT_NAMESPACE // "TedsPropertyEditor"
