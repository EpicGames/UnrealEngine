// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "TedsOutlinerMode.h"
#include "SceneOutlinerFwd.h"
#include "DataStorage/Handles.h"

#define UE_API TEDSPROPERTYEDITOR_API

DECLARE_DELEGATE_OneParam(FOnTedsRowSelected, UE::Editor::DataStorage::RowHandle RowHandle);
DECLARE_DELEGATE_RetVal_OneParam(bool, FOnShouldFilterTedsRow, const UE::Editor::DataStorage::RowHandle RowHandle);
DECLARE_DELEGATE_RetVal_OneParam(bool, FOnShouldInteractTedsRow, const UE::Editor::DataStorage::RowHandle RowHandle);

/*
* Picking mode for TEDs Scene Outliner Widgets. Based off of FActorPickingMode
*/
class FTedsRowPickingMode : public UE::Editor::Outliner::FTedsOutlinerMode
{
public:
	UE_API FTedsRowPickingMode(const UE::Editor::Outliner::FTedsOutlinerParams& Params, FOnSceneOutlinerItemPicked OnItemPickedDelegate);

	virtual ~FTedsRowPickingMode() = default;
public:
	UE_API virtual void OnItemSelectionChanged(FSceneOutlinerTreeItemPtr Item, ESelectInfo::Type SelectionType, const FSceneOutlinerItemSelection& Selection) override;

	/** Allow the user to commit their selection by pressing enter if it is valid */
	UE_API virtual void OnFilterTextCommited(FSceneOutlinerItemSelection& Selection, ETextCommit::Type CommitType) override;

	virtual bool ShowViewButton() const override { return false; }
private:
	FOnSceneOutlinerItemPicked OnItemPicked;
};

#undef UE_API
