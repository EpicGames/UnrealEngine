// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "TedsRowPickingMode.h"
#include "Widgets/SCompoundWidget.h"

#define UE_API TEDSPROPERTYEDITOR_API

class SPropertyMenuTedsRowPicker : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SPropertyMenuTedsRowPicker)
		: _AllowClear(true)
		, _ElementFilter()
		, _InteractiveFilter()
	{}
		SLATE_ARGUMENT(bool, AllowClear)
		SLATE_ARGUMENT(UE::Editor::DataStorage::FQueryDescription, QueryFilter)
		SLATE_ARGUMENT(FOnShouldFilterTedsRow, ElementFilter)
		SLATE_ARGUMENT(FOnShouldInteractTedsRow, InteractiveFilter)
		SLATE_EVENT(FOnTedsRowSelected, OnSet)
		SLATE_EVENT(FSimpleDelegate, OnClose)
	SLATE_END_ARGS()

	UE_API void Construct(const FArguments& InArgs);

private:

	UE_API void OnClear();

	UE_API void OnElementSelected(UE::Editor::DataStorage::RowHandle RowHandle);

	UE_API void SetValue(UE::Editor::DataStorage::RowHandle RowHandle);

private:
	bool bAllowClear;

	UE::Editor::DataStorage::FQueryDescription QueryFilter;

	FOnShouldFilterTedsRow ElementFilter;

	FOnShouldInteractTedsRow InteractiveFilter;

	FOnTedsRowSelected OnSet;

	FSimpleDelegate OnClose;

	FSimpleDelegate OnUseSelected;
};

#undef UE_API
