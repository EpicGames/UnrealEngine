// Copyright Epic Games, Inc. All Rights Reserved.

#include "Searching/SearchUtils.h"

#include <algorithm>
#include "Templates/FunctionFwd.h"
#include "UObject/Class.h"
#include "UObject/StrProperty.h"
#include "UObject/TextProperty.h"
#include "UObject/UnrealType.h"


namespace UE::Editor::DataStorage::QueryStack::Searching
{
	size_t FCaseInsensitiveHash::operator()(FString::ElementType Value) const
	{
		return TChar<FString::ElementType>::ToLower(Value);
	}
	bool FCaseInsensitiveCompare::operator()(FString::ElementType Lhs, FString::ElementType Rhs) const
	{
		return  Lhs == Rhs ? true : TChar<FString::ElementType>::ToLower(Lhs) == TChar<FString::ElementType>::ToLower(Rhs);
	}

	FSearchContext::FSearchContext(FString InSearchString)
		: SearchString(MoveTemp(InSearchString))
		, StringSearcher(FStringSearcher(
			*SearchString, *SearchString + SearchString.Len()))
	{
	}

	StringSearchFunction CreateSearchFunction(const FProperty* Property)
	{
		if (const FStrProperty* StringProperty = CastField<FStrProperty>(Property))
		{
			return [](const Searching::FSearchContext& Context, const FProperty* Property, const void* Column, FString& TempString)
				{
					// Assume that the property has been checked at a higher call.
					const FString* String = Property->ContainerPtrToValuePtr<FString>(Column);
					return String ? Searching::Search(*String, Context, TempString) : false;
				};
		}
		else if (const FTextProperty* TextProperty = CastField<FTextProperty>(Property))
		{
			return [](const Searching::FSearchContext& Context, const FProperty* Property, const void* Column, FString& TempString)
				{
					// Assume that the property has been checked at a higher call.
					const FText* String = Property->ContainerPtrToValuePtr<FText>(Column);
					return String ? Searching::Search(*String, Context, TempString) : false;
				};
		}
		else if (const FNameProperty* NameProperty = CastField<FNameProperty>(Property))
		{
			return [](const Searching::FSearchContext& Context, const FProperty* Property, const void* Column, FString& TempString)
				{
					// Assume that the property has been checked at a higher call.
					const FName* String = Property->ContainerPtrToValuePtr<FName>(Column);
					return String ? Searching::Search(*String, Context, TempString) : false;
				};
		}
		else
		{
			return nullptr;
		}
	}

	bool Search(const FString& String, const Searching::FSearchContext& Context, FString& TempString)
	{
		FStringView View(String);
		return !String.IsEmpty() && std::search(View.begin(), View.end(), Context.StringSearcher) != View.end();
	}

	bool Search(const FName& String, const Searching::FSearchContext& Context, FString& TempString)
	{
		TempString.Reset();
		String.ToString(TempString);
		return Search(TempString, Context, TempString);
	}

	bool Search(const FText& String, const Searching::FSearchContext& Context, FString& TempString)
	{
		return Search(String.ToString(), Context, TempString);
	}
	
	void ListSearchableColumns(TFunctionRef<void(const UScriptStruct*)> Callback)
	{
		static const FName SearchableMetaDataTag = FName("Searchable");
		
		UScriptStruct* DataColumnType = FEditorDataStorageColumn::StaticStruct();
		ForEachObjectOfClass(UScriptStruct::StaticClass(),
			[Callback, DataColumnType](UObject* Object)
			{
				UScriptStruct* Struct = static_cast<UScriptStruct*>(Object);
				if (DataColumnType != Struct && Struct->IsChildOf(DataColumnType))
				{
					// Don't use ListSearchableProperties since we can't early out and only need to find one
					for (TFieldIterator<FProperty> PropIt(Struct); PropIt; ++PropIt)
					{
						if ((*PropIt)->HasMetaData(SearchableMetaDataTag))
						{
							Callback(Struct);
							break;
						}
					}
				}
			});
	}

	void ListSearchableProperties(const UScriptStruct* ColumnType, TFunctionRef<void(const FProperty*)> Callback)
	{
		static const FName SearchableMetaDataTag = FName("Searchable");

		for (TFieldIterator<FProperty> PropertyIt(ColumnType); PropertyIt; ++PropertyIt)
		{
			const FProperty* Property = *PropertyIt;
			if (Property->HasMetaData(SearchableMetaDataTag))
			{
				Callback(Property);
			}
		}
	}
} // namespace UE::Editor::DataStorage::QueryStack::Searching
