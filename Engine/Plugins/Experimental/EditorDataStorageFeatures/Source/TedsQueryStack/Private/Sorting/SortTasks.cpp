// Copyright Epic Games, Inc. All Rights Reserved.

#include "Sorting/SortTasks.h"

#include "Algo/StableSort.h"
#include "Async/ParallelFor.h"
#include "Elements/Framework/TypedElementRowHandleArray.h"
#include "Elements/Framework/TypedElementSorter.h"
#include "ProfilingDebugging/CpuProfilerTrace.h"

namespace UE::Editor::DataStorage::QueryStack::Sorting
{
	void FSortTasks::AddBarrier()
	{
		AddBarrier(MainTaskList);
	}

	void FSortTasks::CollectPrefixes(TArrayView<FPrefix> Output, TConstArrayView<RowHandle> Input,
		const ICoreProvider& Storage, TSharedPtr<const FColumnSorterInterface> Sorter, int32 BatchSize)
	{
		CollectPrefixes(MainTaskList, Output, Input, Storage, Sorter, BatchSize);
	}

	void FSortTasks::RefreshPrefixes(TArrayView<FPrefix> Prefixes, const ICoreProvider& Storage,
		TSharedPtr<const FColumnSorterInterface> Sorter, uint32 ByteIndex, int32 BatchSize)
	{
		RefreshPrefixes(MainTaskList, Prefixes, Storage, Sorter, ByteIndex, BatchSize);
	}

	void FSortTasks::RadixSortPrefixes(TArrayView<FPrefix> Output)
	{
		RadixSortPrefixes(MainTaskList, Output);
	}

	void FSortTasks::CopyPrefixesToRowHandleArray(TArrayView<RowHandle> Output, TConstArrayView<FPrefix> Input)
	{
		CopyPrefixesToRowHandleArray(MainTaskList, Output, Input);
	}

	void FSortTasks::ComparativeSortRowHandleRanges(TArrayView<RowHandle> Rows, const ICoreProvider& Storage,
		TSharedPtr<const FColumnSorterInterface> Sorter, int32 BatchSize)
	{
		ComparativeSortRowHandleRanges(MainTaskList, Rows, Storage, Sorter, BatchSize);
	}

	void FSortTasks::MergeSortedAdjacentRanges(TArrayView<RowHandle> Array, TArrayView<RowHandle> Scratch,
		const ICoreProvider& Storage, TSharedPtr<const FColumnSorterInterface> Sorter, int32 SplitPosition)
	{
		MergeSortedAdjacentRanges(MainTaskList, Array, Scratch, Storage, Sorter, SplitPosition);
	}

	void FSortTasks::BucketizeRange(TArrayView<RowHandle> Array, TArrayView<FPrefix> Prefixes, TArrayView<FPrefix> PrefixesShadow, 
		TSharedPtr<const FColumnSorterInterface> Sorter, const ICoreProvider& Storage, int32 PrefixBatchSize, int32 SortBatchSize)
	{
		BucketizeRange(MainTaskList, Array, Prefixes, PrefixesShadow, Sorter, Storage, PrefixBatchSize, SortBatchSize, 0);
	}

	void FSortTasks::BuildTaskChain(TaskReferenceChain& Output)
	{
		// First collect all newly added chains.
		while (TaskChain* Next = PendingTasksQueue.Peek())
		{
			PendingTasks.Add(MoveTemp(*Next));
			PendingTasksQueue.Dequeue();
		}

		// Then for each chain, add their commands to the overall chain.
		auto AddTaskToChain = [](TaskReferenceChain& Output, TaskChain& Chain)
			{
				for (FPrefixCollectionBatch& Batch : Chain)
				{
					if (!Batch.bCompleted)
					{
						if (!Batch.Task.IsType<FTaskBarrier>())
						{
							Output.Add(&Batch);
						}
						else
						{
							break;
						}
					}
				}
			};
		for (TaskChain& Chain : PendingTasks)
		{
			AddTaskToChain(Output, Chain);
		}
		AddTaskToChain(Output, MainTaskList);
	}

	void FSortTasks::RemoveCompletedTasks()
	{
		auto HasCompleted = [](TaskChain& Chain)
			{
				for (FPrefixCollectionBatch& Batch : Chain)
				{
					if (!Batch.bCompleted)
					{
						if (Batch.Task.IsType<FTaskBarrier>())
						{
							Batch.bCompleted = true;
						}
						else
						{
							return false;
						}
					}
				}
				return true;
			};
		for (auto It = PendingTasks.CreateIterator(); It; ++It)
		{
			if (HasCompleted(*It))
			{
				It.RemoveCurrentSwap();
			}
		}
		if (HasCompleted(MainTaskList))
		{
			MainTaskList.Reset();
		}
	}

	void FSortTasks::Update(FTimespan AllottedTime)
	{
		if (HasRemainingTasks())
		{
			bIsUpdating = true;
			uint64 StartTime = FPlatformTime::Cycles64();
			do
			{
				BuildTaskChain(LocalTaskChain);

				// Execute the tasks in this batch.
				ParallelFor(LocalTaskChain.Num(), [this, StartTime, AllottedTime](int32 Batch)
					{
						if (FTimespan(FPlatformTime::Cycles64() - StartTime) < AllottedTime)
						{
							FPrefixCollectionBatch* BatchInfo = LocalTaskChain[Batch];
							checkf(BatchInfo->bCompleted == false, TEXT("Trying to execute an already completed sort task."));
							BatchInfo->bCompleted = true;
							Visit(FTaskVisitor
								{
									.Owner = *this
								}, BatchInfo->Task);
						}
					}, EParallelForFlags::Unbalanced);

				RemoveCompletedTasks();
				LocalTaskChain.Reset();
			} while (HasRemainingTasks() && FTimespan(FPlatformTime::Cycles64() - StartTime) < AllottedTime);
			bIsUpdating = false;
		}
	}

	bool FSortTasks::HasRemainingTasks() const
	{
		return !MainTaskList.IsEmpty() || !PendingTasks.IsEmpty() || !PendingTasksQueue.IsEmpty();
	}

	void FSortTasks::PrintRemainingTaskList(FString& Output)
	{
		checkf(!bIsUpdating, TEXT("It's not allowed to call CompileRemainingTaskList while updating."));

		FTaskPrinter Printer{ .Output = Output };

		// Collect pending tasks to make them accessible.
		while (TaskChain* Next = PendingTasksQueue.Peek())
		{
			PendingTasks.Add(MoveTemp(*Next));
			PendingTasksQueue.Dequeue();
		}

		auto ReportChain = [](FTaskPrinter& Printer, const TaskChain& Chain)
			{
				for (const FPrefixCollectionBatch& Batch : Chain)
				{
					Printer.Output.Append(TEXT("    "));
					Visit(Printer, Batch.Task);
					Printer.Output.Append(Batch.bCompleted ? TEXT(" (Completed)\n") : TEXT("\n"));
				}
			};

		Output.Append(TEXT("  Main task list\n"));
		ReportChain(Printer, MainTaskList);
		
		int32 BatchCount = 0;
		for (TaskChain& Chain : PendingTasks)
		{
			Output.Appendf(TEXT("  Batch %i\n"), BatchCount++);
			ReportChain(Printer, Chain);
		}
	}

	void FSortTasks::FTaskVisitor::operator()(const FTaskBarrier& Task)
	{
		checkf(false, TEXT("Task barriers should never be processed."));
	}

	void FSortTasks::FTaskVisitor::operator()(const FCollectPrefixesTask& Task)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE("[TEDS Sorting] Collect prefixes");

		const RowHandle* Source = Task.Input.GetData();
		FPrefix* Target = Task.Output.GetData();
		int32 Count = Task.Input.Num();

		for (int32 Index = 0; Index < Count; ++Index)
		{
			FPrefixInfo Result = Task.Sorter->CalculatePrefix(Task.Storage, *Source, 0);
			Target->Prefix = Result.Prefix;
			Target->Row = *Source;
			Target->bHasRemainingBytes = Result.bHasRemainingBytes;

			Source++;
			Target++;
		}
	}

	void FSortTasks::FTaskVisitor::operator()(const FRefreshPrefixesTask& Task)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE("[TEDS Sorting] Refresh prefixes");

		for (FPrefix& Prefix : Task.Prefixes)
		{
			if (Prefix.bHasRemainingBytes)
			{
				FPrefixInfo Result = Task.Sorter->CalculatePrefix(Task.Storage, Prefix.Row, Task.ByteIndex);
				Prefix.Prefix = Result.Prefix;
				Prefix.bHasRemainingBytes = Result.bHasRemainingBytes;
			}
			else
			{
				Prefix.Prefix = 0;
			}
		}
	}

	void FSortTasks::FTaskVisitor::operator()(const FRadixSortPrefixesTask& Task)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE("[TEDS Sorting] Radix sort prefixes");

		RadixSort64(Task.Output.GetData(), Task.Output.Num(),
			[](const FPrefix& Prefix)
			{
				return Prefix.Prefix;
			});
	}

	void FSortTasks::FTaskVisitor::operator()(const FCopyPrefixesToRowHandleArrayTask& Task)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE("[TEDS Sorting] Copy out prefixes to array");

		const FPrefix* InputIt = Task.Input.GetData();
		const FPrefix* InputEnd = InputIt + Task.Input.Num();
		RowHandle* OutputIt = Task.Output.GetData();

		while (InputIt < InputEnd)
		{
			*OutputIt = InputIt->Row;
			OutputIt++;
			InputIt++;
		}
	}

	void FSortTasks::FTaskVisitor::operator()(const FComparativeSortRowHandleRangeTask& Task)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE("[TEDS Sorting] Comparative sort");

		Algo::StableSort(Task.Rows, [&Task](RowHandle Left, RowHandle Right)
			{
				return Task.Sorter->Compare(Task.Storage, Left, Right) < 0;
			});
	}

	void FSortTasks::FTaskVisitor::operator()(const FComparativeSortRowHandlePrefixRangeTask& Task)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE("[TEDS Sorting] Comparative sort with prefix");

		Algo::StableSort(Task.Rows, [&Task](const FPrefix& Left, const FPrefix& Right)
			{
				if (Left.Prefix == Right.Prefix)
				{
					return Task.Sorter->Compare(Task.Storage, Left.Row, Right.Row) < 0;
				}
				else
				{
					return (Left.Prefix < Right.Prefix) ? true : false;
				}
			});
	}

	void FSortTasks::FTaskVisitor::operator()(const FMergeSortedAdjacentRangesTask& Task)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE("[TEDS Sorting] Merge");

		// Only merge if the first element from the right range is within the left range, otherwise the 
		// array is already sorted.
		TArrayView<RowHandle> Left(Task.Array.GetData(), Task.SplitPosition);
		
		RowHandle* LeftBegin = Task.Array.GetData();
		RowHandle* RightBegin = LeftBegin + Task.SplitPosition;
		RowHandle* LeftLast = RightBegin - 1;
		RowHandle* LeftEnd = RightBegin;
		RowHandle* RightEnd = LeftBegin + Task.Array.Num();
		RowHandle* RightIt = RightEnd - 1;
		RowHandle* InsertBack = RightIt;

		if (Task.Sorter->Compare(Task.Storage, *LeftLast, *RightBegin) > 0)
		{
			int32 RightCount = static_cast<int32>(RightEnd - RightBegin);

			// Copy the right section to a temporary location so it doesn't interfere with the sorting. Update the pointers
			// to for right as the values are now in the scratch buffer.
			FMemory::Memcpy(Task.Scratch.GetData(), RightBegin, RightCount * sizeof(RowHandle));
			RightBegin = Task.Scratch.GetData();
			RightEnd = RightBegin + RightCount;
			RightIt = RightEnd - 1;
			
			// For each entry in the right section, find the insertion point in the left section. Move everything from the
			// left up to that point and insert the right value.
			while (RightIt >= RightBegin)
			{
				int32 InsertPoint = Algo::LowerBound(Left, *RightIt, 
					[&Task](RowHandle Left, RowHandle Right)
					{
						return Task.Sorter->Compare(Task.Storage, Left, Right) < 0;
					});
				if (int32 ReadBlockSize = Left.Num() - InsertPoint; ReadBlockSize > 0)
				{
					InsertBack -= ReadBlockSize - 1;
					FMemory::Memmove(InsertBack, Left.GetData() + InsertPoint, ReadBlockSize * sizeof(RowHandle));
					Left.LeftChopInline(ReadBlockSize);
					InsertBack--;
				}
				*InsertBack = *RightIt;
				RightIt--;
				InsertBack--;
			}
			// Any values remaining on the left side will already be sorted.
		}
	}

	void FSortTasks::FTaskVisitor::operator()(const FBucketizeRangesTask& Task)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE("[TEDS Sorting] Bucketize");

		int32 Counts[256];
		FMemory::Memzero(Counts, sizeof(Counts));
		bool bHasRemainingBytes[256];
		FMemory::Memzero(bHasRemainingBytes, sizeof(bHasRemainingBytes));

		// Collect the prefix count at the request byte.
		int32 Shift = 56 - ((Task.Index & 7) << 3);
		for (const FPrefix& Prefix : Task.Prefixes)
		{
			uint8 Index = static_cast<uint8>(Prefix.Prefix >> Shift);
			Counts[Index]++;
			bHasRemainingBytes[Index] = bHasRemainingBytes[Index] || Prefix.bHasRemainingBytes;
		}

		// Convert counts into offsets.
		int32 Offsets[256];
		Offsets[0] = 0;
		for (int32 Index = 0; Index < 255; ++Index)
		{
			Offsets[Index + 1] = Offsets[Index] + Counts[Index];
		}

		// Distribute the prefixes.
		for (const FPrefix& Prefix : Task.Prefixes)
		{
			uint8 Index = static_cast<uint8>(Prefix.Prefix >> Shift);
			Task.PrefixesShadow[Offsets[Index]++] = Prefix;
		}
		
		// Create new tasks for each group that still needs work.
		uint32 NextIndex = Task.Index + 1;
		TaskChain SubTasks;
		if ((NextIndex & 7) == 0)
		{
			int32 Offset = 0;
			bool bAddedRefreshPrefixesTask = false;
			for (int32 Index = 0; Index < 256; ++Index)
			{
				int32 Count = Counts[Index];
				if (Count == 1)
				{
					// If there's only one value in the bucket, then it's already in the correct place and there's no need for further 
					// sorting.
					Task.Array[Offset] = Task.PrefixesShadow[Offset].Row;
				}
				else if (bHasRemainingBytes[Index]) // If there are no entries in this slot this is guaranteed to be false.
				{
					Owner.RefreshPrefixes(
						SubTasks, Task.PrefixesShadow.Slice(Offset, Count), Task.Storage, Task.Sorter, NextIndex, Task.PrefixBatchSize);
					bAddedRefreshPrefixesTask = true;
				}
				else
				{
					// If there are no prefixes left, there's no more sorting that can happen, which means the values all have the same value.
					for (int32 SubIndex = Offset; SubIndex < Offset + Count; ++SubIndex)
					{
						Task.Array[SubIndex] = Task.PrefixesShadow[SubIndex].Row;
					}
				}
				Offset += Count;
			}

			if (bAddedRefreshPrefixesTask)
			{
				Owner.AddBarrier(SubTasks);
				
				Offset = 0;
				for (int32 Index = 0; Index < 256; ++Index)
				{
					int32 Count = Counts[Index];
					if (bHasRemainingBytes[Index])
					{
						// Can't start a comparative sort here because the barrier wouldn't protect against work in a separate task list.
						// This would mean adding the compares in the same task list which would result in additional barriers in this list
						// and less parallelization. Instead let another loop of bucketization happen which guarantees the prefixes are
						// collected.
						Owner.BucketizeRange(
							SubTasks,
							Task.Array.Slice(Offset, Count),
							Task.PrefixesShadow.Slice(Offset, Count),
							Task.Prefixes.Slice(Offset, Count),
							Task.Sorter, Task.Storage, Task.PrefixBatchSize, Task.SortBatchSize, NextIndex);
					}
					Offset += Count;
				}
			}
		}
		else
		{
			int32 Offset = 0;
			for (int32 Index = 0; Index < 256; ++Index)
			{
				int32 Count = Counts[Index];
				if (Count == 1)
				{
					// If there's only one value in the bucket, then it's already in the correct place and there's no need for further 
					// sorting.
					Task.Array[Offset] = Task.PrefixesShadow[Offset].Row;
				}
				else if (Count > 1)
				{
					if (Count < Task.SortBatchSize)
					{
						// Do the final sort and copy in a separate list to avoid the barrier causing additional delays and maximizing
						// parallelization.
						TArrayView<FPrefix> SubPrefixView = Task.PrefixesShadow.Slice(Offset, Count);
						TaskChain CompareSortTask;
						CompareSortTask.Reserve(3);
						Owner.ComparativeSortRowHandlePrefixRanges(CompareSortTask, SubPrefixView, Task.Storage, Task.Sorter);
						Owner.AddBarrier(CompareSortTask);
						Owner.CopyPrefixesToRowHandleArray(CompareSortTask, Task.Array.Slice(Offset, Count), SubPrefixView);
						Owner.PendingTasksQueue.Enqueue(MoveTemp(CompareSortTask));
					}
					else
					{
						Owner.BucketizeRange(
							SubTasks,
							Task.Array.Slice(Offset, Count),
							Task.PrefixesShadow.Slice(Offset, Count),
							Task.Prefixes.Slice(Offset, Count),
							Task.Sorter, Task.Storage, Task.PrefixBatchSize, Task.SortBatchSize, NextIndex);
					}
				}
				Offset += Count;
			}
		}
		if (!SubTasks.IsEmpty())
		{
			Owner.PendingTasksQueue.Enqueue(MoveTemp(SubTasks));
		}
	}

	void FSortTasks::FTaskPrinter::operator()(const FTaskBarrier& Task)
	{
		Output += TEXT("- Barrier -");
	}

	void FSortTasks::FTaskPrinter::operator()(const FCollectPrefixesTask& Task)
	{
		Output.Appendf(TEXT("Collect prefixes - Input [0x%p, %i], Output [0x%p, %i]"), 
			Task.Input.GetData(), Task.Input.Num(), Task.Output.GetData(), Task.Output.Num());
	}

	void FSortTasks::FTaskPrinter::operator()(const FRefreshPrefixesTask& Task)
	{
		Output.Appendf(TEXT("Refresh prefixes - Prefixes [0x%p, %i], Byte Index %u"), 
			Task.Prefixes.GetData(), Task.Prefixes.Num(), Task.ByteIndex);
	}

	void FSortTasks::FTaskPrinter::operator()(const FRadixSortPrefixesTask& Task)
	{
		Output.Appendf(TEXT("Radix sort prefixes - Output [0x%p, %i]"), Task.Output.GetData(), Task.Output.Num());
	}

	void FSortTasks::FTaskPrinter::operator()(const FCopyPrefixesToRowHandleArrayTask& Task)
	{
		Output.Appendf(TEXT("Copy prefixes to row handle array - Input [0x%p, %i], Output [0x%p, %i]"), 
			Task.Input.GetData(), Task.Input.Num(), Task.Output.GetData(), Task.Output.Num());
	}

	void FSortTasks::FTaskPrinter::operator()(const FComparativeSortRowHandleRangeTask& Task)
	{
		Output.Appendf(TEXT("Comparative sort - Rows [0x%p, %i]"), Task.Rows.GetData(), Task.Rows.Num());
	}

	void FSortTasks::FTaskPrinter::operator()(const FComparativeSortRowHandlePrefixRangeTask& Task)
	{
		Output.Appendf(TEXT("Comparative sort (prefixed) - Rows [0x%p, %i]"), Task.Rows.GetData(), Task.Rows.Num());
	}

	void FSortTasks::FTaskPrinter::operator()(const FMergeSortedAdjacentRangesTask& Task)
	{
		Output.Appendf(TEXT("Comparative sort (prefixed) - Array [0x%p, %i], Scratch [0x%p, %i], Split pos %i"), 
			Task.Array.GetData(), Task.Array.Num(),
			Task.Scratch.GetData(), Task.Scratch.Num(),
			Task.SplitPosition);
	}

	void FSortTasks::FTaskPrinter::operator()(const FBucketizeRangesTask& Task)
	{
		Output.Appendf(TEXT("Bucketize range - Array [0x%p, %i], Prefixes [0x%p, %i], Shadow Prefixes [0x%p, %i] Prefix batch size %i, Sort batch size %i, Index %i"),
			Task.Array.GetData(), Task.Array.Num(),
			Task.Prefixes.GetData(), Task.Prefixes.Num(),
			Task.PrefixesShadow.GetData(), Task.PrefixesShadow.Num(),
			Task.PrefixBatchSize, Task.SortBatchSize, Task.Index);
	}

	void FSortTasks::AddBarrier(TaskChain& Tasks)
	{
		Tasks.Emplace(FPrefixCollectionBatch
			{
				.Task = TaskVariant(TInPlaceType<FTaskBarrier>(), FTaskBarrier{})
			});
	}

	void FSortTasks::CollectPrefixes(TaskChain& Tasks, TArrayView<FPrefix> Output, TConstArrayView<RowHandle> Input,
		const ICoreProvider& Storage, TSharedPtr<const FColumnSorterInterface> Sorter, int32 BatchSize)
	{
		checkf(Output.Num() >= Input.Num(), TEXT("A prefix array can't be generated from an input that's smaller than the prefix table."));

		int32 Count = Input.Num();
		Tasks.Reserve(Tasks.Num() + (Count / BatchSize) + 1);
		for (int32 Batch = 0; Batch < Count; Batch += BatchSize)
		{
			int32 SubBatchEnd = FMath::Min(Batch + BatchSize, Count);
			int32 SubBatchCount = SubBatchEnd - Batch;

			Tasks.Emplace(FPrefixCollectionBatch
				{
					.Task = TaskVariant(TInPlaceType<FCollectPrefixesTask>(), FCollectPrefixesTask
						{
							.Output = Output.Slice(Batch, SubBatchCount),
							.Input = Input.Slice(Batch, SubBatchCount),
							.Sorter = Sorter,
							.Storage = Storage
						})
				});
		}
	}

	void FSortTasks::RefreshPrefixes(TaskChain& Tasks, TArrayView<FPrefix> Prefixes, const ICoreProvider& Storage,
		TSharedPtr<const FColumnSorterInterface> Sorter, uint32 ByteIndex, int32 BatchSize)
	{
		int32 Count = Prefixes.Num();
		Tasks.Reserve(Tasks.Num() + (Count / BatchSize) + 1);
		for (int32 Batch = 0; Batch < Count; Batch += BatchSize)
		{
			int32 SubBatchEnd = FMath::Min(Batch + BatchSize, Count);
			int32 SubBatchCount = SubBatchEnd - Batch;

			Tasks.Emplace(FPrefixCollectionBatch
				{
					.Task = TaskVariant(TInPlaceType<FRefreshPrefixesTask>(), FRefreshPrefixesTask
					{
						.Prefixes = Prefixes.Slice(Batch, SubBatchCount),
						.Sorter = Sorter,
						.Storage = Storage,
						.ByteIndex = ByteIndex
					})
				});
		}
	}

	void FSortTasks::RadixSortPrefixes(TaskChain& Tasks, TArrayView<FPrefix> Output)
	{
		Tasks.Emplace(FPrefixCollectionBatch
			{
				.Task = TaskVariant(TInPlaceType<FRadixSortPrefixesTask>(), FRadixSortPrefixesTask
					{
						.Output = Output
					})
			});
	}

	void FSortTasks::CopyPrefixesToRowHandleArray(TaskChain& Tasks, TArrayView<RowHandle> Output, TConstArrayView<FPrefix> Input)
	{
		checkf(Output.Num() == Input.Num(),
			TEXT("The input sort prefix list doesn't have the same number of entries as the row handle output array."));
		Tasks.Emplace(FPrefixCollectionBatch
			{
				.Task = TaskVariant(TInPlaceType<FCopyPrefixesToRowHandleArrayTask>(), FCopyPrefixesToRowHandleArrayTask
					{
						.Output = Output,
						.Input = Input
					})
			});
	}

	void FSortTasks::ComparativeSortRowHandleRanges(TaskChain& Tasks, TArrayView<RowHandle> Rows, const ICoreProvider& Storage,
		TSharedPtr<const FColumnSorterInterface> Sorter, int32 BatchSize)
	{
		int32 Count = Rows.Num();
		Tasks.Reserve(Tasks.Num() + (Count / BatchSize) + 1);
		for (int32 Batch = 0; Batch < Count; Batch += BatchSize)
		{
			int32 SubBatchCount = FMath::Min(Batch + BatchSize, Count) - Batch;
			Tasks.Emplace(FPrefixCollectionBatch
				{
					.Task = TaskVariant(TInPlaceType<FComparativeSortRowHandleRangeTask>(), FComparativeSortRowHandleRangeTask
						{
							.Rows = Rows.Slice(Batch, SubBatchCount),
							.Sorter = Sorter,
							.Storage = Storage
						})
				});
		}
	}

	void FSortTasks::ComparativeSortRowHandlePrefixRanges(TaskChain& Tasks, TArrayView<FPrefix> Rows, const ICoreProvider& Storage,
		TSharedPtr<const FColumnSorterInterface> Sorter)
	{
		Tasks.Emplace(FPrefixCollectionBatch
			{
				.Task = TaskVariant(TInPlaceType<FComparativeSortRowHandlePrefixRangeTask>(), FComparativeSortRowHandlePrefixRangeTask
					{
						.Rows = Rows,
						.Sorter = Sorter,
						.Storage = Storage
					})
			});
	}

	void FSortTasks::MergeSortedAdjacentRanges(TaskChain& Tasks, TArrayView<RowHandle> Array, TArrayView<RowHandle> Scratch,
		const ICoreProvider& Storage, TSharedPtr<const FColumnSorterInterface> Sorter, int32 SplitPosition)
	{
		int32 Count = Array.Num();
		// Only sort if the number of entries is larger than the split position, otherwise there is no right side to merge in.
		while (Count > SplitPosition)
		{
			int32 ScratchBatch = 0;
			int32 BatchSize = SplitPosition * 2;
			Tasks.Reserve(Tasks.Num() + (Count / BatchSize) + 1);
			for (int32 Batch = 0; Batch < Count; Batch += BatchSize)
			{
				int32 SubBatchCount = FMath::Min(Batch + BatchSize, Count) - Batch;
				if (SubBatchCount > SplitPosition)
				{
					int32 SubBatchScratchCount = SubBatchCount - SplitPosition;

					Tasks.Emplace(FPrefixCollectionBatch
						{
							.Task = TaskVariant(TInPlaceType<FMergeSortedAdjacentRangesTask>(), FMergeSortedAdjacentRangesTask
								{
									.Array = Array.Slice(Batch, SubBatchCount),
									.Scratch = Scratch.Slice(ScratchBatch, SubBatchScratchCount),
									.Sorter = Sorter,
									.Storage = Storage,
									.SplitPosition = SplitPosition
								})
						});
					ScratchBatch += SubBatchScratchCount;
				}
			}

			Tasks.Emplace(FPrefixCollectionBatch
				{
					.Task = TaskVariant(TInPlaceType<FTaskBarrier>(), FTaskBarrier{})
				});

			SplitPosition *= 2;
		}
	}

	void FSortTasks::BucketizeRange(TaskChain& Tasks, TArrayView<RowHandle> Array, TArrayView<FPrefix> Prefixes, 
		TArrayView<FPrefix> PrefixesShadow, TSharedPtr<const FColumnSorterInterface> Sorter, const ICoreProvider& Storage, 
		int32 PrefixBatchSize, int32 SortBatchSize, uint32 ByteIndex)
	{
		Tasks.Emplace(FPrefixCollectionBatch
			{
				.Task = TaskVariant(TInPlaceType<FBucketizeRangesTask>(), FBucketizeRangesTask
					{
						.Array = Array,
						.Prefixes = Prefixes,
						.PrefixesShadow = PrefixesShadow,
						.Sorter = Sorter,
						.Storage = Storage,
						.PrefixBatchSize = PrefixBatchSize,
						.SortBatchSize = SortBatchSize,
						.Index = ByteIndex
					})
			});
	}
} // namespace UE::Editor::DataStorage::QueryStack::Sorting
