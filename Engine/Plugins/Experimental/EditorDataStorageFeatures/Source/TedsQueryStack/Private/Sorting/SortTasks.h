// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Async/Mutex.h"
#include "DataStorage/Handles.h"
#include "Containers/ArrayView.h"
#include "Containers/MpscQueue.h"
#include "Misc/Timespan.h"
#include "Misc/TVariant.h"
#include "Templates/SharedPointer.h"

namespace UE::Editor::DataStorage
{
	class FColumnSorterInterface;
	class FRowHandleArray;
	class ICoreProvider;
}

namespace UE::Editor::DataStorage::QueryStack::Sorting
{
	struct FPrefix
	{
		uint64 Prefix;
		RowHandle Row;
		bool bHasRemainingBytes;
	};

	/** Simple task queue system for the various operations needed to sort row handles using a column sorter. */
	class FSortTasks
	{
	public:
		void AddBarrier();
		void CollectPrefixes(TArrayView<FPrefix> Output, TConstArrayView<RowHandle> Input,
			const ICoreProvider& Storage, TSharedPtr<const FColumnSorterInterface> Sorter, int32 BatchSize);
		void RefreshPrefixes(TArrayView<FPrefix> Prefixes, const ICoreProvider& Storage, 
			TSharedPtr<const FColumnSorterInterface> Sorter, uint32 ByteIndex, int32 BatchSize);
		void RadixSortPrefixes(TArrayView<FPrefix> Output);
		void CopyPrefixesToRowHandleArray(TArrayView<RowHandle> Output, TConstArrayView<FPrefix> Input);
		void ComparativeSortRowHandleRanges(TArrayView<RowHandle> Rows, const ICoreProvider& Storage, 
			TSharedPtr<const FColumnSorterInterface> Sorter, int32 BatchSize);
		void MergeSortedAdjacentRanges(TArrayView<RowHandle> Array, TArrayView<RowHandle> Scratch,
			const ICoreProvider& Storage, TSharedPtr<const FColumnSorterInterface> Sorter, int32 SplitPosition);
		void BucketizeRange(TArrayView<RowHandle> Array, TArrayView<FPrefix> Prefixes, TArrayView<FPrefix> PrefixesShadow, 
			TSharedPtr<const FColumnSorterInterface> Sorter, const ICoreProvider& Storage, int32 PrefixBatchSize, int32 SortBatchSize);
		
		void Update(FTimespan AllottedTime);

		/** Whether or not there are still any tasks remaining. If this is called during update it may not provide accurate results. */
		bool HasRemainingTasks() const;
		/** 
		 * Collects all tasks pending on various threads and compile them together into a list of remaining tasks. This cannot be safely
		 * called if Update is doing work.
		 */
		void PrintRemainingTaskList(FString& Output);

	private:
		struct FTaskBarrier {};
		struct FCollectPrefixesTask
		{
			TArrayView<FPrefix> Output;
			TConstArrayView<RowHandle> Input;
			TSharedPtr<const FColumnSorterInterface> Sorter;
			const ICoreProvider& Storage;
		};
		struct FRefreshPrefixesTask
		{
			TArrayView<FPrefix> Prefixes;
			TSharedPtr<const FColumnSorterInterface> Sorter;
			const ICoreProvider& Storage;
			uint32 ByteIndex;
		};
		struct FRadixSortPrefixesTask
		{
			TArrayView<FPrefix> Output;
		};
		struct FCopyPrefixesToRowHandleArrayTask
		{
			TArrayView<RowHandle> Output;
			TConstArrayView<FPrefix> Input;
		};
		struct FComparativeSortRowHandleRangeTask
		{
			TArrayView<RowHandle> Rows;
			TSharedPtr<const FColumnSorterInterface> Sorter;
			const ICoreProvider& Storage;
		};
		struct FComparativeSortRowHandlePrefixRangeTask
		{
			TArrayView<FPrefix> Rows;
			TSharedPtr<const FColumnSorterInterface> Sorter;
			const ICoreProvider& Storage;
		};
		struct FMergeSortedAdjacentRangesTask
		{
			TArrayView<RowHandle> Array;
			TArrayView<RowHandle> Scratch;
			TSharedPtr<const FColumnSorterInterface> Sorter;
			const ICoreProvider& Storage;
			int32 SplitPosition;
		};
		struct FBucketizeRangesTask
		{
			TArrayView<RowHandle> Array;
			TArrayView<FPrefix> Prefixes;
			TArrayView<FPrefix> PrefixesShadow;
			TSharedPtr<const FColumnSorterInterface> Sorter;
			const ICoreProvider& Storage;
			int32 PrefixBatchSize;
			int32 SortBatchSize;
			uint32 Index;
		};
		using TaskVariant = TVariant<
			FTaskBarrier, 
			FCollectPrefixesTask,
			FRefreshPrefixesTask,
			FRadixSortPrefixesTask,
			FCopyPrefixesToRowHandleArrayTask,
			FComparativeSortRowHandleRangeTask,
			FComparativeSortRowHandlePrefixRangeTask,
			FMergeSortedAdjacentRangesTask,
			FBucketizeRangesTask
		>;

		struct FPrefixCollectionBatch
		{
			TaskVariant Task;
			bool bCompleted = false;
		};
		
		struct FTaskVisitor
		{
			void operator()(const FTaskBarrier& Task);
			void operator()(const FCollectPrefixesTask& Task);
			void operator()(const FRefreshPrefixesTask& Task);
			void operator()(const FRadixSortPrefixesTask& Task);
			void operator()(const FCopyPrefixesToRowHandleArrayTask& Task);
			void operator()(const FComparativeSortRowHandleRangeTask& Task);
			void operator()(const FComparativeSortRowHandlePrefixRangeTask& Task);
			void operator()(const FMergeSortedAdjacentRangesTask& Task);
			void operator()(const FBucketizeRangesTask& Task);

			FSortTasks& Owner;
		};

		struct FTaskPrinter
		{
			void operator()(const FTaskBarrier& Task);
			void operator()(const FCollectPrefixesTask& Task);
			void operator()(const FRefreshPrefixesTask& Task);
			void operator()(const FRadixSortPrefixesTask& Task);
			void operator()(const FCopyPrefixesToRowHandleArrayTask& Task);
			void operator()(const FComparativeSortRowHandleRangeTask& Task);
			void operator()(const FComparativeSortRowHandlePrefixRangeTask& Task);
			void operator()(const FMergeSortedAdjacentRangesTask& Task);
			void operator()(const FBucketizeRangesTask& Task);

			FString& Output;
		};
		
		using TaskChain = TArray<FPrefixCollectionBatch>;
		using TaskReferenceChain = TArray<FPrefixCollectionBatch*>;

		void AddBarrier(TaskChain& Tasks);
		void CollectPrefixes(TaskChain& Tasks, TArrayView<FPrefix> Output, TConstArrayView<RowHandle> Input,
			const ICoreProvider& Storage, TSharedPtr<const FColumnSorterInterface> Sorter, int32 BatchSize);
		void RefreshPrefixes(TaskChain& Tasks, TArrayView<FPrefix> Prefixes, const ICoreProvider& Storage,
			TSharedPtr<const FColumnSorterInterface> Sorter, uint32 ByteIndex, int32 BatchSize);
		void RadixSortPrefixes(TaskChain& Tasks, TArrayView<FPrefix> Output);
		void CopyPrefixesToRowHandleArray(TaskChain& Tasks, TArrayView<RowHandle> Output, TConstArrayView<FPrefix> Input);
		void ComparativeSortRowHandleRanges(TaskChain& Tasks, TArrayView<RowHandle> Rows, const ICoreProvider& Storage,
			TSharedPtr<const FColumnSorterInterface> Sorter, int32 BatchSize);
		void ComparativeSortRowHandlePrefixRanges(TaskChain& Tasks, TArrayView<FPrefix> Rows, const ICoreProvider& Storage,
			TSharedPtr<const FColumnSorterInterface> Sorter);
		void MergeSortedAdjacentRanges(TaskChain& Tasks, TArrayView<RowHandle> Array, TArrayView<RowHandle> Scratch,
			const ICoreProvider& Storage, TSharedPtr<const FColumnSorterInterface> Sorter, int32 SplitPosition);
		void BucketizeRange(TaskChain& Tasks, TArrayView<RowHandle> Array, TArrayView<FPrefix> Prefixes, TArrayView<FPrefix> PrefixesShadow,
			TSharedPtr<const FColumnSorterInterface> Sorter, const ICoreProvider& Storage, int32 PrefixBatchSize, int32 SortBatchSize, 
			uint32 ByteIndex);

		void BuildTaskChain(TaskReferenceChain& Output);
		void RemoveCompletedTasks();

		TMpscQueue<TaskChain> PendingTasksQueue;
		TArray<TaskChain> PendingTasks;
		TaskChain MainTaskList;
		TaskReferenceChain LocalTaskChain;

		bool bIsUpdating = false;
	};
} // namespace UE::Editor::DataStorage::QueryStack::Sorting