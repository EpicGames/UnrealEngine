// Copyright Epic Games, Inc. All Rights Reserved.

#include "TedsColumnsSearchNode.h"

#include "Elements/Framework/TypedElementConcurrentRowHandleCollector.h"
#include "Elements/Framework/TypedElementQueryBuilder.h"
#include "Elements/Framework/TypedElementQueryContext.h"
#include "Internationalization/Text.h"
#include "ProfilingDebugging/CpuProfilerTrace.h"
#include "Searching/SearchUtils.h"
#include "UObject/Class.h"
#include "UObject/UnrealType.h"

namespace UE::Editor::DataStorage::QueryStack
{	
	FColumnsSearchNode::FColumnsSearchNode(ICoreProvider& Storage, ESyncActions SyncAction)
		: Storage(Storage)
		, SyncAction(SyncAction)
	{
	}

	FColumnsSearchNode::~FColumnsSearchNode()
	{
		UnregisterAllColumns();
	}

	void FColumnsSearchNode::StartSearch(FString SearchString)
	{
		if (!SearchString.IsEmpty())
		{
			SearchContext = MakeUnique<Searching::FSearchContext>(MoveTemp(SearchString));
			Rows.Empty();
			Revision++;
		}
		else
		{
			ClearSearch();
		}
	}

	void FColumnsSearchNode::ClearSearch()
	{
		ResetSearch();
		SearchContext.Reset();
	}

	bool FColumnsSearchNode::IsSearching() const
	{
		return SearchContext.IsValid() && SearchContext->CurrentSearcher < Searchers.Num();
	}

	FTimespan FColumnsSearchNode::GetQuerySearchTime() const
	{
		return TotalSearchTime;
	}

	uint64 FColumnsSearchNode::CalculateTotalSearchedCellCount() const
	{
		uint64 TotalCount = 0;
		for (const FSearchQuery& Searcher : Searchers)
		{
			TotalCount += Storage.RunQuery(Searcher.Query).Count;
		}
		return TotalCount;
	}

	void FColumnsSearchNode::RegisterColumn(const UScriptStruct* ColumnType, const Queries::FConditions& Conditions)
	{
		Searching::ListSearchableProperties(ColumnType,
			[this, ColumnType, &Conditions](const FProperty* Property)
			{
				RegisterColumn(ColumnType, Property, Conditions);
			});
	}

	void FColumnsSearchNode::RegisterColumn(const UScriptStruct* ColumnType, const FProperty* Property, Queries::FConditions Conditions)
	{
		using namespace UE::Editor::DataStorage;
		using namespace UE::Editor::DataStorage::Queries;

		if (Searching::StringSearchFunction SearchFunction = Searching::CreateSearchFunction(Property);
			ensureMsgf(SearchFunction, TEXT("Property '%s' in Column '%s' is marked in metadata as searchable but doesn't have a searchable type."),
			*Property->GetName(), *ColumnType->GetFName().ToString()))
		{
			QueryHandle Query = CreateQuery(ColumnType, MoveTemp(Conditions));
			uint64 QueryHash = Storage.CalculateQueryTablesTopologyHash(Query);

			Searchers.Add(FSearchQuery{
				.Query = Query,
				.Hash = QueryHash,
				.SearchFunction = SearchFunction,
				.Property = Property,
				.ColumnType = ColumnType
				});
		}
	}

	void FColumnsSearchNode::UnregisterAllColumns()
	{
		for (FSearchQuery& Searcher : Searchers)
		{
			Storage.UnregisterQuery(Searcher.Query);
		}
		Searchers.Reset();
	}

	INode::RevisionId FColumnsSearchNode::GetRevision() const
	{
		return Revision;
	}

	void FColumnsSearchNode::VisitParents(ParentListCallback Callback)
	{
		// No parents to visit.
	}

	void FColumnsSearchNode::Update(FTimespan AllottedTime)
	{
		if (Searching::FSearchContext* LocalSearchContext = SearchContext.Get())
		{
			double StartTime = FPlatformTime::Seconds();
			
			TRACE_CPUPROFILER_EVENT_SCOPE_STR("[TEDS] FColumnsSearchNode Tick");

			if (SyncAction == ESyncActions::Refresh)
			{
				bool bIsDirty = false;
				for (FSearchQuery& Searcher : Searchers)
				{
					uint64 QueryHash = Storage.CalculateQueryTablesTopologyHash(Searcher.Query);
					bIsDirty = bIsDirty || QueryHash != Searcher.Hash;
					Searcher.Hash = QueryHash;
				}
				if (bIsDirty)
				{
					ResetSearch();
					LocalSearchContext->CurrentSearcher = 0;
				}
			}

			double TimeBudget = AllottedTime.GetTotalSeconds();
			double EndTime = FPlatformTime::Seconds();
			while(LocalSearchContext->CurrentSearcher < Searchers.Num() && EndTime - StartTime < TimeBudget)
			{
				RunSearch(*LocalSearchContext, Searchers[LocalSearchContext->CurrentSearcher]);
				LocalSearchContext->CurrentSearcher++;
				EndTime = FPlatformTime::Seconds();
				Revision++;
			}

			TotalSearchTime += FTimespan::FromSeconds(EndTime - StartTime);
		}
	}
	
	FRowHandleArrayView FColumnsSearchNode::GetRows() const
	{
		return Rows.GetRows();
	}
	
	FRowHandleArray& FColumnsSearchNode::GetMutableRows()
	{
		return Rows;
	}

	QueryHandle FColumnsSearchNode::CreateQuery(const UScriptStruct* ColumnType, Queries::FConditions&& Conditions)
	{
		using namespace UE::Editor::DataStorage::Queries;

		if (Conditions.IsEmpty())
		{
			return Storage.RegisterQuery(
				Select()
					.ReadOnly(ColumnType)
				.Compile());
		}
		else
		{
			FEditorStorageQueryConditionCompileContext CompileContext(&Storage);
			Conditions.Compile(CompileContext);

			return Storage.RegisterQuery(
				Select()
					.ReadOnly(ColumnType)
				.Where(MoveTemp(Conditions))
				.Compile());
		}
	}

	void FColumnsSearchNode::RunSearch(const Searching::FSearchContext& Context, const FSearchQuery& Searcher)
	{
		using namespace UE::Editor::DataStorage;
		using namespace UE::Editor::DataStorage::Queries;

		TRACE_CPUPROFILER_EVENT_SCOPE_STR("[TEDS] FColumnsSearchNode::RunSearch");

		FConcurrentRowHandleSetCollector Collector;
		
		Storage.RunQuery(Searcher.Query,
			ERunQueryFlags::ParallelizeChunks | ERunQueryFlags::AutoBalanceParallelChunkProcessing,
			[&Context, &Searcher, &Collector](TQueryContext<RowBatchInfo> QueryContext)
				{
					if (const UScriptStruct* ColumnType = Searcher.ColumnType.Get(); ColumnType)
					{
						TRACE_CPUPROFILER_EVENT_SCOPE_TEXT(
							*FString::Printf(TEXT("[TEDS] FColumnsSearchNode query '%s'"), *ColumnType->GetFName().ToString()));

						int32 ColumnSize = ColumnType->GetCppStructOps() ? ColumnType->GetCppStructOps()->GetSize() : ColumnType->GetStructureSize();
						
						FRowHandleArray Temp;
						// Run the compare on all columns in this batch and add the found results to the temp array.
						if (const char* ColumnData = reinterpret_cast<const char*>(QueryContext.GetColumnBatchAddress(ColumnType)))
						{
							TRACE_CPUPROFILER_EVENT_SCOPE_STR(TEXT("[TEDS] FColumnsSearchNode query - Compare"));

							FString TempString;
							bool bReservedMemory = false;
							for (RowHandle Row : QueryContext.GetBatchRowHandles())
							{
								if (Searcher.SearchFunction(Context, Searcher.Property, ColumnData, TempString))
								{
									if (!bReservedMemory)
									{
										// Avoid reserving memory if there are no results.
										Temp.Reserve(QueryContext.GetBatchRowCount());
										bReservedMemory = true;
									}
									Temp.Add(Row, FRowHandleArray::EFlags::IsUnique);
								}
								ColumnData += ColumnSize;
							}
						}
						Collector.Append(MoveTemp(Temp));
					}
				});

		Collector.Collect(Rows);
	}
	
	void FColumnsSearchNode::ResetSearch()
	{
		TotalSearchTime = 0;
		Rows.Empty();
		Revision++;
	}
} // namespace UE::Editor::DataStorage::QueryStack
