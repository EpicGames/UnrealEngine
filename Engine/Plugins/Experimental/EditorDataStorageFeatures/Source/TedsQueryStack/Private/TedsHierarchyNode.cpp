// Copyright Epic Games, Inc. All Rights Reserved.

#include "TedsHierarchyNode.h"

#include "Elements/Interfaces/TypedElementDataStorageInterface.h"


namespace UE::Editor::DataStorage::QueryStack
{
	FHierarchyRowNode::FHierarchyRowNode(
		ICoreProvider& Storage,
		FHierarchyHandle InHierarchyHandle,
		TSharedPtr<QueryStack::IRowNode>& InTopLevelRows,
		ESyncFlags InSyncFlags,
		EFilterFlags InFilterFlags)
		: TopLevelRows(InTopLevelRows)
		, Storage(Storage)
		, HierarchyHandle(InHierarchyHandle)
		, SyncFlags(InSyncFlags)
		, FilterFlags(InFilterFlags)
	{
	}

	QueryStack::INode::RevisionId FHierarchyRowNode::GetRevision() const
	{
		return Revision;
	}

	void FHierarchyRowNode::Update(FTimespan AllottedTime)
	{
		switch (SyncFlags)
		{
			case ESyncFlags::ParentRevisionDifferent:
			{
				if (TopLevelRows->GetRevision() != ParentRevision)
				{
					Rows.Empty();
					RefreshInternal(Storage, Rows);
					++Revision;
					ParentRevision = TopLevelRows->GetRevision();
				}
				break;
			}
			case ESyncFlags::IncrementWhenDifferent:
			{
				FRowHandleArray& TempArray = GetTempRowHandleArray();
				RefreshInternal(Storage, TempArray);
				TempArray.Sort();
				Rows.Sort();
				if (!TempArray.IsSame(Rows))
				{
					Rows = MoveTemp(TempArray);
					++Revision;
					ParentRevision = TopLevelRows->GetRevision();
				}
				break;
			}
			case ESyncFlags::Always:
			{
				Rows.Empty();
				RefreshInternal(Storage, Rows);
				++Revision;
				ParentRevision = TopLevelRows->GetRevision();
				break;
			}
			default:
			{
				// Do nothing
			}
		}
	}

	void FHierarchyRowNode::VisitParents(ParentListCallback Callback)
	{
		Callback(TopLevelRows);
	}

	FRowHandleArrayView FHierarchyRowNode::GetRows() const
	{
		return Rows.GetRows();
	}

	FRowHandleArray& FHierarchyRowNode::GetMutableRows()
	{
		return Rows;
	}

	void FHierarchyRowNode::RefreshInternal(ICoreProvider& InCoreProvider, FRowHandleArray& InRows) const
	{
		InRows.Empty();
		
		auto VisitFn = [&InRows, FilterFlags_ = FilterFlags](const ICoreProvider&, RowHandle ParentRow, RowHandle TargetRow)
		{
			if ((FilterFlags_ == EFilterFlags::OnlyIncludeChildRows) && (ParentRow == InvalidRowHandle))
			{
				return;
			}

			InRows.Add(TargetRow);
		};

		// Top level rows are already going to have the VisitFn run on them
		FRowHandleArrayView TopLevelRowsView = TopLevelRows->GetRows();
		for (RowHandle Row : TopLevelRowsView)
		{
			// Walk recursively down through the hierarchy and add rows 
			Storage.WalkDepthFirst(HierarchyHandle, Row, VisitFn);
		}
	}

	FRowHandleArray& FHierarchyRowNode::GetTempRowHandleArray()
	{
		NewRows.Empty();
		return NewRows;
	}
}
