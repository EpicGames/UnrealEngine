// Copyright Epic Games, Inc. All Rights Reserved.

#include "TedsQueryChangeNotifyNode.h"

namespace UE::Editor::DataStorage::QueryStack
{
	FQueryChangeNotifyNode::FQueryChangeNotifyNode(TSharedPtr<IQueryNode> Parent, FOnQueryNodeChange ChangeEvent)
		: Parent(MoveTemp(Parent))
		, OnChangeEvent(MoveTemp(ChangeEvent))
	{
		checkf(Parent, TEXT("FQueryChangeNotifyNode requires that the parent is set."));
		checkf(OnChangeEvent.IsBound(), TEXT("FRowChangFQueryChangeNotifyNodeeNotifyNode requires a delegate."))
	}

	INode::RevisionId FQueryChangeNotifyNode::GetRevision() const
	{
		return Parent->GetRevision();
	}

	void FQueryChangeNotifyNode::VisitParents(ParentListCallback Callback)
	{
		Callback(Parent);
	}

	QueryHandle FQueryChangeNotifyNode::GetQuery() const
	{
		return Parent->GetQuery();
	}

	void FQueryChangeNotifyNode::Update(FTimespan AllottedTime)
	{
		RevisionId CurrentRevision = Parent->GetRevision();
		if (CurrentRevision != CachedParentRevision)
		{
			OnChangeEvent.Execute(Parent);
			CachedParentRevision = CurrentRevision;
		}
	}
} // namespace UE::Editor::DataStorage::QueryStack
