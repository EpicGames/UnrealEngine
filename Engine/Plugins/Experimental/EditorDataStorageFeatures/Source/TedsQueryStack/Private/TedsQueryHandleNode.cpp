// Copyright Epic Games, Inc. All Rights Reserved.

#include "TedsQueryHandleNode.h"

namespace UE::Editor::DataStorage::QueryStack
{
	FQueryHandleNode::FQueryHandleNode(QueryHandle Query)
		: Query(Query)
	{
	}

	void FQueryHandleNode::SetQuery(QueryHandle InQuery)
	{
		Query = InQuery;
		Revision++;
	}

	INode::RevisionId FQueryHandleNode::GetRevision() const
	{
		return Revision;
	}

	void FQueryHandleNode::Update(FTimespan AllottedTime)
	{
		// Nothing to update.
	}

	void FQueryHandleNode::VisitParents(ParentListCallback Callback)
	{
		// No parents to visit.
	}

	QueryHandle FQueryHandleNode::GetQuery() const
	{
		return Query;
	}
} // namespace UE::Editor::DataStorage::QueryStack
