// Copyright Epic Games, Inc. All Rights Reserved.

#include "TedsQueryMergeNode.h"

#include "Elements/Framework/TypedElementQueryBuilder.h"
#include "Elements/Interfaces/TypedElementDataStorageInterface.h"
#include "TedsQueryStackLog.h"

namespace UE::Editor::DataStorage::QueryStack
{
	FQueryMergeNode::FQueryMergeNode(ICoreProvider& InStorage, TConstArrayView<TSharedPtr<IQueryNode>> InParents)
		: Storage(InStorage)
	{
		checkf(!InParents.IsEmpty(), TEXT("FQueryMergeNode requires at least one parent query."));

		Parents.Reserve(InParents.Num());
		for (const TSharedPtr<IQueryNode>& Parent : InParents)
		{
			Parents.Emplace<FParentInfo>({ .Parent = Parent, .Revision = Parent->GetRevision()});
		}
		Rebuild();
	}

	FQueryMergeNode::~FQueryMergeNode()
	{
		Storage.UnregisterQuery(QueryHandle);
	}

	INode::RevisionId FQueryMergeNode::GetRevision() const
	{
		return Revision;
	}

	void FQueryMergeNode::Update(FTimespan AllottedTime)
	{
		bool bRebuild = false;
		for (FParentInfo& ParentInfo : Parents)
		{
			RevisionId ParentRevision = ParentInfo.Parent->GetRevision();
			if (ParentInfo.Revision != ParentRevision)
			{
				bRebuild = true;
				ParentInfo.Revision = ParentRevision;
			}
		}

		if (bRebuild)
		{
			Rebuild();
			Revision++;
		}
	}

	void FQueryMergeNode::VisitParents(ParentListCallback Callback)
	{
		for (const FParentInfo& ParentInfo : Parents)
		{
			Callback(ParentInfo.Parent);
		}
	}

	QueryHandle FQueryMergeNode::GetQuery() const
	{
		return QueryHandle;
	}

	void FQueryMergeNode::Rebuild()
	{
		Storage.UnregisterQuery(QueryHandle);
		QueryHandle = InvalidQueryHandle;

		FQueryDescription FinalQueryDescription;

		FText ErrorText;
		
		for (const FParentInfo& Parent : Parents)
		{
			FQueryDescription ParentQueryDescription = Storage.GetQueryDescription(Parent.Parent->GetQuery());

			if (!Queries::MergeQueries(FinalQueryDescription, ParentQueryDescription, &ErrorText))
			{
				UE_LOGF(LogTedsQueryStack, Error, "Skipping Parent Query %ls due to error: %ls",
					*ParentQueryDescription.Callback.Name.ToString(), *ErrorText.ToString());
			}
		}

		QueryHandle = Storage.RegisterQuery(MoveTemp(FinalQueryDescription));
	}
}
