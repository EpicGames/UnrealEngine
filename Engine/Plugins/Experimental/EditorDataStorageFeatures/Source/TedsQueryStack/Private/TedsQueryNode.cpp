// Copyright Epic Games, Inc. All Rights Reserved.

#include "TedsQueryNode.h"

#include "Elements/Interfaces/TypedElementDataStorageInterface.h"

namespace UE::Editor::DataStorage::QueryStack
{
	FQueryNode::FQueryNode(ICoreProvider& Storage)
		: Storage(Storage)
	{
	}

	FQueryNode::FQueryNode(ICoreProvider& Storage, UE::Editor::DataStorage::FQueryDescription Query)
		: QueryHandle(Storage.RegisterQuery(MoveTemp(Query)))
		, Storage(Storage)
	{
	}

	FQueryNode::~FQueryNode()
	{
		// Hack to work around cases where systems that persist after UObject destruction hold onto a query node, which causes a crash since
		// Storage is a UObject
		if (UObjectInitialized() && !IsEngineExitRequested())
		{
			Storage.UnregisterQuery(QueryHandle);
		}
	}

	void FQueryNode::SetQuery(UE::Editor::DataStorage::FQueryDescription Query)
	{
		Storage.UnregisterQuery(QueryHandle);
		QueryHandle = Storage.RegisterQuery(MoveTemp(Query));
		Revision++;
	}

	void FQueryNode::ClearQuery()
	{
		if (QueryHandle != InvalidQueryHandle)
		{
			Storage.UnregisterQuery(QueryHandle);
			QueryHandle = InvalidQueryHandle;
			Revision++;
		}
	}

	INode::RevisionId FQueryNode::GetRevision() const
	{
		return Revision;
	}

	void FQueryNode::Update(FTimespan AllottedTime)
	{
		// Nothing to update.
	}

	void FQueryNode::VisitParents(ParentListCallback Callback)
	{
		// No parents to report.
	}

	QueryHandle FQueryNode::GetQuery() const
	{
		return QueryHandle;
	}
} // namespace UE::Editor::DataStorage::QueryStack
