// Copyright Epic Games, Inc. All Rights Reserved.

#include "TedsQuerySearchNode.h"

#include "Elements/Framework/TypedElementConcurrentRowHandleCollector.h"
#include "Elements/Framework/TypedElementQueryBuilder.h"
#include "Elements/Framework/TypedElementQueryContext.h"
#include "Internationalization/Text.h"
#include "ProfilingDebugging/CpuProfilerTrace.h"
#include "Searching/SearchUtils.h"
#include "UObject/Class.h"
#include "UObject/UnrealType.h"

namespace UE::Editor::DataStorage::QueryStack
{	
	FQuerySearchNode::FQuerySearchNode(ICoreProvider& Storage, TSharedRef<IQueryNode> Query)
		: FRowQueryResultsNode(Storage, Query)
	{
		UpdateSearchersList();
	}

	FQuerySearchNode::FQuerySearchNode(ICoreProvider& Storage, TSharedRef<IQueryNode> Query, ESyncActions SyncAction)
		: FRowQueryResultsNode(Storage, Query, SyncAction)
	{
		UpdateSearchersList();
	}

	void FQuerySearchNode::StartSearch(FString SearchString)
	{
		if (!SearchString.IsEmpty())
		{
			bHasPendingSearch = true;
			SearchContext = MakeUnique<Searching::FSearchContext>(MoveTemp(SearchString));
			ResetSearch();
		}
		else
		{
			ClearSearch();
		}
	}

	void FQuerySearchNode::ClearSearch()
	{
		ResetSearch();
		SearchContext.Reset();
	}

	bool FQuerySearchNode::IsSearching() const
	{
		return SearchContext.IsValid() && bHasPendingSearch;
	}

	FTimespan FQuerySearchNode::GetQuerySearchTime() const
	{
		return TotalSearchTime;
	}

	void FQuerySearchNode::QueryUpdated()
	{
		if (SearchContext.IsValid())
		{
			bHasPendingSearch = true;
		}
		UpdateSearchersList();
	}

	void FQuerySearchNode::RefreshInternal(FTimespan AllottedTime, FRowHandleArray& TargetRows)
	{
		if (Searching::FSearchContext* LocalSearchContext = SearchContext.Get(); LocalSearchContext)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE_STR("[TEDS] FQuerySearchNode Tick");

			double StartTime = FPlatformTime::Seconds();
			RunSearch(*LocalSearchContext, TargetRows);
			double EndTime = FPlatformTime::Seconds();
			TotalSearchTime = FTimespan::FromSeconds(EndTime - StartTime);
		}
		else
		{
			FRowQueryResultsNode::RefreshInternal(AllottedTime, TargetRows);
		}
	}

	void FQuerySearchNode::UpdateSearchersList()
	{
		// Reset the searching if there's one active.
		ResetSearch();

		// Collect the list of property searchers.
		const FQueryDescription& Description = Storage.GetQueryDescription(QueryNode->GetQuery());
		Searchers.Reset();

		auto AddSearcher = [](TArray<FSearchQuery>& InSearchers, const UScriptStruct* Column)
			{
				Searching::ListSearchableProperties(Column,
					[&InSearchers, Column](const FProperty* Property)
					{
						if (Searching::StringSearchFunction SearchFunction = Searching::CreateSearchFunction(Property))
						{
							InSearchers.Add(FSearchQuery{
									.SearchFunction = SearchFunction,
									.Property = Property,
									.ColumnType = Column,
									.ColumnSize = Column->GetCppStructOps() ? Column->GetCppStructOps()->GetSize() : Column->GetStructureSize()
								});
						}
					});
			};

		for (TWeakObjectPtr<const UScriptStruct> Column : Description.SelectionTypes)
		{
			if (const UScriptStruct* ColumnPtr = Column.Get())
			{
				AddSearcher(Searchers, ColumnPtr);
			}
		}

		for (const FDynamicColumnDescription& DynamicColumn : Description.DynamicSelectionTypes)
		{
			if (DynamicColumn.TemplateType)
			{
				if (const UScriptStruct* ColumnPtr = Storage.FindDynamicColumnType(*DynamicColumn.TemplateType, DynamicColumn.Identifier))
				{
					AddSearcher(Searchers, ColumnPtr);
				}
			}
		}
	}

	void FQuerySearchNode::RunSearch(const Searching::FSearchContext& Context, FRowHandleArray& TargetRows)
	{
		using namespace UE::Editor::DataStorage;
		using namespace UE::Editor::DataStorage::Queries;

		FConcurrentRowHandleSetCollector Collector;
		
		Storage.RunQuery(QueryNode->GetQuery(),
			ERunQueryFlags::ParallelizeChunks | ERunQueryFlags::AutoBalanceParallelChunkProcessing |
			ERunQueryFlags::AllowBoundQueries | ERunQueryFlags::IgnoreActiveState,
				[this, &Context, &Collector](TQueryContext<RowBatchInfo> QueryContext)
				{
					TRACE_CPUPROFILER_EVENT_SCOPE_TEXT(TEXT("[TEDS] FQuerySearchNode query"));

					// Collect the column addresses.
					struct FInfo { const char* Address; const FSearchQuery& Searcher; };
					TArray<FInfo, TInlineAllocator<InlineColumnCount>> ColumnInfo;
					ColumnInfo.Reserve(Searchers.Num());
					for (const FSearchQuery& Searcher : Searchers)
					{
						if (const UScriptStruct* ColumnType = Searcher.ColumnType.Get(); ColumnType)
						{
							ColumnInfo.Add(FInfo
								{
									.Address = static_cast<const char*>(QueryContext.GetColumnBatchAddress(ColumnType)),
									.Searcher = Searcher
								});
						}
					}

					FRowHandleArray Temp;
					FString TempString;
					bool bReservedMemory = false;

					for (RowHandle Row : QueryContext.GetBatchRowHandles())
					{
						// Compare data on current row.
						for (const FInfo& Info : ColumnInfo)
						{
							if (Info.Searcher.SearchFunction(Context, Info.Searcher.Property, Info.Address, TempString))
							{
								if (!bReservedMemory)
								{
									// Avoid reserving memory if there are no results.
									Temp.Reserve(QueryContext.GetBatchRowCount());
									bReservedMemory = true;
								}
								Temp.Add(Row, FRowHandleArray::EFlags::IsUnique);
								break; // Make sure each row is only added once.
							}
						}

						// Move to next row
						for (FInfo& Info : ColumnInfo)
						{
							Info.Address += Info.Searcher.ColumnSize;
						}
					}

					Collector.Append(MoveTemp(Temp));
				});

		Collector.Collect(TargetRows);
		bHasPendingSearch = false;
	}

	void FQuerySearchNode::ResetSearch()
	{
		TotalSearchTime = 0;
		TopologyHash = 0; // Force a refresh.
		Rows.Empty();
		Revision++;
	}
} // namespace UE::Editor::DataStorage::QueryStack
