// Copyright Epic Games, Inc. All Rights Reserved.

#include "TedsQueryStackExecutor.h"

#include <utility>
#include "TedsQueryStackInterfaces.h"

namespace UE::Editor::DataStorage::QueryStack
{
	//
	// FExecutorBase
	//

	FExecutorBase::~FExecutorBase() = default;

	const FName& FExecutorBase::GetStackName() const
	{
		return StackName;
	}

	TSharedPtr<const INode> FExecutorBase::GetRootNode() const
	{
		return RootNode;
	}

	TSharedPtr<INode> FExecutorBase::GetRootNode()
	{
		return RootNode;
	}

	FExecutorBase::FExecutorBase(const FName& InStackName, TSharedPtr<INode> InRootNode)
		: RootNode(MoveTemp(InRootNode))
		, StackName(InStackName)
	{
		checkf(RootNode, TEXT("Root node for Query Stack executor '%s' can't be a nullptr. "), *StackName.ToString());
	}

	void FExecutorBase::FullUpdate()
	{
		checkf(RootNode, TEXT("Root node for Query Stack executor '%s' cannot be a nullptr for a full update."), *StackName.ToString());
		FullNodeUpdate(StackName, RootNode);
	}

	void FExecutorBase::FullNodeUpdate(const FName& StackName, const TSharedPtr<INode>& Node)
	{
		Node->VisitParents(
			[&Node, &StackName](TSharedPtr<INode> Parent)
			{
				checkf(Parent, TEXT("Parent node for Query Stack executor '%s' cannot be a nullptr for a full update."), *StackName.ToString());
				checkf(Parent != Node, TEXT("Parent node for Query Stack executor '%s' cannot be itself for a full update."), *StackName.ToString());
				FullNodeUpdate(StackName, Parent);
			});
		Node->Update(INode::UnlimitedTime);
	}

	void FExecutorBase::TimedUpdate(FTimespan AllottedTime)
	{
		checkf(RootNode, TEXT("Root node for Query Stack executor '%s' cannot be a nullptr for a timed update."), *StackName.ToString()); 
		TimedNodeUpdate(StackName, RootNode, FPlatformTime::Cycles64(), AllottedTime);
	}

	void FExecutorBase::TimedNodeUpdate(const FName& StackName, const TSharedPtr<INode>& Node, uint64 StartTime, FTimespan AllottedTime)
	{
		Node->VisitParents(
			[&Node, &StackName, StartTime, AllottedTime](TSharedPtr<INode> Parent)
			{
				checkf(Parent, TEXT("Parent node for Query Stack executor '%s' cannot be a nullptr for a timed update."), *StackName.ToString());
				checkf(Parent != Node, TEXT("Parent node for Query Stack executor '%s' cannot be itself for a timed update."), *StackName.ToString());
				TimedNodeUpdate(StackName, Parent, StartTime, AllottedTime);
			});
		FTimespan Duration(FTimespan::FromSeconds(FPlatformTime::ToSeconds64(FPlatformTime::Cycles64() - StartTime)));
		if (Duration < AllottedTime)
		{
			Node->Update(AllottedTime - Duration);
		}
	}



	//
	// FCooperativeExecutor
	//

	FCooperativeExecutor::FCooperativeExecutor(const FName& InStackName, ICoreProvider& InDataStorage, TSharedPtr<INode> InRootNode,
		ICoreProvider::ECooperativeTaskPriority InPriority)
		: FExecutorBase(InStackName, MoveTemp(InRootNode))
		, DataStorage(&InDataStorage)
		, Priority(InPriority)
	{
		if (RootNode)
		{
			DataStorage->RegisterCooperativeUpdate(StackName, InPriority,
				[this](FTimespan AllottedTime)
				{
					TimedUpdate(AllottedTime);
				});
		}
	}

	FCooperativeExecutor::~FCooperativeExecutor()
	{
		if (RootNode && DataStorage)
		{
			DataStorage->UnregisterCooperativeUpdate(StackName);
		}
	}

	FCooperativeExecutor::FCooperativeExecutor(FCooperativeExecutor&& Rhs)
		: FExecutorBase(Rhs.StackName, MoveTemp(Rhs.RootNode))
		, DataStorage(std::exchange(Rhs.DataStorage, nullptr))
		, Priority(Rhs.Priority)
	{
		if (RootNode && DataStorage)
		{
			// Remove the call registered by the moved object.
			DataStorage->UnregisterCooperativeUpdate(StackName);
			// Register the new callback
			DataStorage->RegisterCooperativeUpdate(StackName, Priority,
				[this](FTimespan AllottedTime)
				{
					TimedUpdate(AllottedTime);
				});
		}
	}

	FCooperativeExecutor& FCooperativeExecutor::operator=(FCooperativeExecutor&& Rhs)
	{
		if (this != &Rhs)
		{
			this->~FCooperativeExecutor();
			new(this) FCooperativeExecutor(MoveTemp(Rhs));
		}
		return *this;
	}



	//
	// FOnUpdateExecutor
	//
	
	FOnUpdateExecutor::FOnUpdateExecutor(const FName& InStackName, ICoreProvider& InDataStorage, TSharedPtr<INode> InRootNode)
		: FExecutorBase(InStackName, MoveTemp(InRootNode))
		, DataStorage(&InDataStorage)
	{
		if (RootNode)
		{
			Delegate = DataStorage->OnUpdate().AddRaw(this, &FOnUpdateExecutor::OnUpdate);
		}
	}

	FOnUpdateExecutor::~FOnUpdateExecutor()
	{
		if (DataStorage)
		{
			DataStorage->OnUpdate().Remove(Delegate);
		}
		Delegate.Reset();
	}

	FOnUpdateExecutor::FOnUpdateExecutor(FOnUpdateExecutor&& Rhs)
		: FExecutorBase(Rhs.StackName, MoveTemp(Rhs.RootNode))
		, DataStorage(std::exchange(Rhs.DataStorage, nullptr))
	{
		if (RootNode && DataStorage)
		{
			DataStorage->OnUpdate().Remove(Rhs.Delegate);
			Rhs.Delegate.Reset();

			Delegate = DataStorage->OnUpdate().AddRaw(this, &FOnUpdateExecutor::OnUpdate);
		}
	}

	FOnUpdateExecutor& FOnUpdateExecutor::operator=(FOnUpdateExecutor&& Rhs)
	{
		if (this != &Rhs)
		{
			this->~FOnUpdateExecutor();
			new(this) FOnUpdateExecutor(MoveTemp(Rhs));
		}
		return *this;
	}

	void FOnUpdateExecutor::OnUpdate()
	{
		FullUpdate();
	}



	//
	// FOnUpdateCompletedExecutor
	//

	FOnUpdateCompletedExecutor::FOnUpdateCompletedExecutor(const FName& InStackName, ICoreProvider& InDataStorage, TSharedPtr<INode> InRootNode)
		: FExecutorBase(InStackName, MoveTemp(InRootNode))
		, DataStorage(&InDataStorage)
	{
		if (RootNode)
		{
			Delegate = DataStorage->OnUpdateCompleted().AddRaw(this, &FOnUpdateCompletedExecutor::OnUpdateCompleted);
		}
	}

	FOnUpdateCompletedExecutor::~FOnUpdateCompletedExecutor()
	{
		if (DataStorage)
		{
			DataStorage->OnUpdateCompleted().Remove(Delegate);
		}
		Delegate.Reset();
	}

	FOnUpdateCompletedExecutor::FOnUpdateCompletedExecutor(FOnUpdateCompletedExecutor&& Rhs)
		: FExecutorBase(Rhs.StackName, MoveTemp(Rhs.RootNode))
		, DataStorage(std::exchange(Rhs.DataStorage, nullptr))
	{
		if (RootNode && DataStorage)
		{
			DataStorage->OnUpdateCompleted().Remove(Rhs.Delegate);
			Rhs.Delegate.Reset();

			Delegate = DataStorage->OnUpdateCompleted().AddRaw(this, &FOnUpdateCompletedExecutor::OnUpdateCompleted);
		}
	}

	FOnUpdateCompletedExecutor& FOnUpdateCompletedExecutor::operator=(FOnUpdateCompletedExecutor&& Rhs)
	{
		if (this != &Rhs)
		{
			this->~FOnUpdateCompletedExecutor();
			new(this) FOnUpdateCompletedExecutor(MoveTemp(Rhs));
		}
		return *this;
	}

	void FOnUpdateCompletedExecutor::OnUpdateCompleted()
	{
		FullUpdate();
	}



	//
	// FExplicitCooperativeExecutor
	//

	FExplicitCooperativeExecutor::FExplicitCooperativeExecutor(const FName& StackName, TSharedPtr<INode> RootNode)
		: FExecutorBase(StackName, MoveTemp(RootNode))
	{
	}

	void FExplicitCooperativeExecutor::Update(FTimespan AllottedTime)
	{
		if (RootNode)
		{
			TimedUpdate(AllottedTime);
		}
	}



	//
	// FExplicitUpdateExecutor
	//

	FExplicitUpdateExecutor::FExplicitUpdateExecutor(const FName& StackName, TSharedPtr<INode> RootNode)
		: FExecutorBase(StackName, MoveTemp(RootNode))
	{
	}

	void FExplicitUpdateExecutor::Update()
	{
		if (RootNode)
		{
			FullUpdate();
		}
	}
} // namespace UE::Editor::DataStorage::QueryStack
