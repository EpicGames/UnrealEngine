// Copyright Epic Games, Inc. All Rights Reserved.

#include "TedsRowArrayNode.h"

namespace UE::Editor::DataStorage::QueryStack
{
	FRowArrayNode::FRowArrayNode(FRowHandleArray Rows)
		: Rows(MoveTemp(Rows))
	{
	}

	void FRowArrayNode::MarkDirty()
	{
		Revision++;
	}

	INode::RevisionId FRowArrayNode::GetRevision() const
	{
		return Revision;
	}

	void FRowArrayNode::Update(FTimespan AllottedTime)
	{
	}

	void FRowArrayNode::VisitParents(ParentListCallback Callback)
	{
	}

	FRowHandleArrayView FRowArrayNode::GetRows() const
	{
		return Rows.GetRows();
	}

	FRowHandleArray& FRowArrayNode::GetMutableRows()
	{
		return Rows;
	}
} // namespace UE::Editor::DataStorage::QueryStack
