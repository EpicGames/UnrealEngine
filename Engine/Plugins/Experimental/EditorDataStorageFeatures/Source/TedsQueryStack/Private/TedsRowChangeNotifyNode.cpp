// Copyright Epic Games, Inc. All Rights Reserved.

#include "TedsRowChangeNotifyNode.h"

namespace UE::Editor::DataStorage::QueryStack
{
	FRowChangeNotifyNode::FRowChangeNotifyNode(TSharedPtr<IRowNode> ParentNode, FOnRowNodeChange ChangeEvent)
		: Parent(MoveTemp(ParentNode))
		, OnChangeEvent(MoveTemp(ChangeEvent))
	{
		checkf(Parent, TEXT("FRowChangeNotifyNode requires that the parent is set."));
		checkf(OnChangeEvent.IsBound(), TEXT("FRowChangeNotifyNode requires a delegate."))
	}

	INode::RevisionId FRowChangeNotifyNode::GetRevision() const
	{
		return Parent->GetRevision();
	}

	void FRowChangeNotifyNode::VisitParents(ParentListCallback Callback)
	{
		Callback(Parent);
	}

	FRowHandleArrayView FRowChangeNotifyNode::GetRows() const
	{
		return Parent->GetRows();
	}

	FRowHandleArray& FRowChangeNotifyNode::GetMutableRows()
	{
		return Parent->GetMutableRows();
	}

	void FRowChangeNotifyNode::Update(FTimespan AllottedTime)
	{
		RevisionId CurrentRevision = Parent->GetRevision();
		if (CurrentRevision != CachedParentRevision)
		{
			OnChangeEvent.Execute(Parent);
			CachedParentRevision = CurrentRevision;
		}
	}
} // namespace UE::Editor::DataStorage::QueryStack
