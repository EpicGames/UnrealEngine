// Copyright Epic Games, Inc. All Rights Reserved.

#include "TedsRowCopyNode.h"

namespace UE::Editor::DataStorage::QueryStack
{
	FRowCopyNode::FRowCopyNode(TSharedPtr<IRowNode> InParent)
		: Parent(MoveTemp(InParent))
	{
		Rows.Append(Parent->GetRows());
	}

	void FRowCopyNode::Reset()
	{
		Rows.Empty();
		Rows.Append(Parent->GetRows());
		Revision++;
	}

	INode::RevisionId FRowCopyNode::GetRevision() const
	{
		return Revision;
	}

	void FRowCopyNode::Update(FTimespan AllottedTime)
	{
		if (Parent->GetRevision() != ParentRevision)
		{
			Reset();
			ParentRevision = Parent->GetRevision();
		}
	}

	void FRowCopyNode::VisitParents(ParentListCallback Callback)
	{
		Callback(Parent);
	}

	FRowHandleArrayView FRowCopyNode::GetRows() const
	{
		return Rows.GetRows();
	}

	FRowHandleArray& FRowCopyNode::GetMutableRows()
	{
		return Rows;
	}
} // namespace UE::Editor::DataStorage::QueryStack
