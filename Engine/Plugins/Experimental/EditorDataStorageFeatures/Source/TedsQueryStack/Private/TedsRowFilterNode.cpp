// Copyright Epic Games, Inc. All Rights Reserved

#include "TedsRowFilterNode.h"

namespace UE::Editor::DataStorage::QueryStack
{
	FRowHandleArrayView FRowFilterNode::GetRows() const
	{
		return Rows.GetRows();
	}

	FRowHandleArray& FRowFilterNode::GetMutableRows()
	{
		return Rows;
	}

	IRowNode::RevisionId FRowFilterNode::GetRevision() const
	{
		return Revision;
	}

	void FRowFilterNode::Update(FTimespan AllottedTime)
	{
		if (CachedParentRevisionID != ParentRowNode->GetRevision())
		{
			Rows.Reset();
			Storage->FilterRowsBy(Rows, ParentRowNode->GetRows(), Options, Filter);

			++Revision;
			CachedParentRevisionID = ParentRowNode->GetRevision();
		}
	}

	void FRowFilterNode::VisitParents(ParentListCallback Callback)
	{
		Callback(ParentRowNode);
	}

	void FRowFilterNode::ForceRefresh()
	{
		CachedParentRevisionID = UninitializedRevisionId;
	}
} // namespace UE::Editor::DataStorage::QueryStack
