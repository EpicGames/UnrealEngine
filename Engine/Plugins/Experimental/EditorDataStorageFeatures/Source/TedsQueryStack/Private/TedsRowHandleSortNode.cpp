// Copyright Epic Games, Inc. All Rights Reserved.

#include "TedsRowHandleSortNode.h"

namespace UE::Editor::DataStorage::QueryStack
{
	FRowHandleSortNode::FRowHandleSortNode(TSharedPtr<IRowNode> InParent)
		: Parent(MoveTemp(InParent))
	{
		FRowHandleArray& ParentRows = Parent->GetMutableRows();
		ParentRows.Sort();
		ParentRevision = Parent->GetRevision();
	}

	INode::RevisionId FRowHandleSortNode::GetRevision() const
	{
		return ParentRevision;
	}

	void FRowHandleSortNode::Update(FTimespan AllottedTime)
	{
		if (Parent->GetRevision() != ParentRevision)
		{
			Parent->GetMutableRows().Sort();
			ParentRevision = Parent->GetRevision();
		}
	}

	void FRowHandleSortNode::VisitParents(ParentListCallback Callback)
	{
		Callback(Parent);
	}

	FRowHandleArrayView FRowHandleSortNode::GetRows() const
	{
		return Parent->GetRows();
	}

	FRowHandleArray& FRowHandleSortNode::GetMutableRows()
	{
		return Parent->GetMutableRows();
	}
} // namespace UE::Editor::DataStorage::QueryStack
