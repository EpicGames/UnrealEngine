// Copyright Epic Games, Inc. All Rights Reserved.

#include "TedsRowMergeNode.h"
#include "Elements/Framework/TypedElementRowHandleArray.h"

namespace UE::Editor::DataStorage::QueryStack
{
	FRowMergeNode::FRowMergeNode(TConstArrayView<TSharedPtr<IRowNode>> InParents, EMergeApproach InMergeApproach)
		: MergeApproach(InMergeApproach)
	{
		for (const TSharedPtr<IRowNode>& Parent : InParents)
		{
			Parents.Emplace<FParentInfo>({ .Parent = Parent, .Revision = Parent->GetRevision()});
		}
		Merge();
	}

	IRowNode::RevisionId FRowMergeNode::GetRevision() const
	{
		return Revision;
	}

	void FRowMergeNode::Update(FTimespan AllottedTime)
	{
		bool bRebuild = false;
		for (FParentInfo& ParentInfo : Parents)
		{
			RevisionId ParentRevision = ParentInfo.Parent->GetRevision();
			if (ParentInfo.Revision != ParentRevision)
			{
				bRebuild = true;
				ParentInfo.Revision = ParentRevision;
			}
		}

		if (bRebuild)
		{
			Merge();
			Revision++;
		}
	}

	void FRowMergeNode::VisitParents(ParentListCallback Callback)
	{
		for (const FParentInfo& ParentInfo : Parents)
		{
			Callback(ParentInfo.Parent);
		}
	}

	FRowHandleArrayView FRowMergeNode::GetRows() const
	{
		return Rows.GetRows();
	}

	FRowHandleArray& FRowMergeNode::GetMutableRows()
	{
		return Rows;
	}

	void FRowMergeNode::Merge()
	{
		Rows.Reset();

		int32 TotalCount = 0;
		for (FParentInfo& Parent : Parents)
		{
			TotalCount += Parent.Parent->GetRows().Num();
		}
		Rows.Reserve(TotalCount);

		for (FParentInfo& Parent : Parents)
		{
			Rows.Append(Parent.Parent->GetRows());
		}

		switch (MergeApproach)
		{
		case EMergeApproach::Append:
			break;
		case EMergeApproach::Sorted:
			Rows.Sort();
			break;
		case EMergeApproach::Unique:
			Rows.MakeUnique();
			break;
		case EMergeApproach::Repeating:
			Rows.ReduceToDuplicates();
			break;
		default:
			checkf(false, TEXT("Unexpected merge type in FQueryStack_RowMerge."));
			break;
		}
	}
} // namespace UE::Editor::DataStorage::QueryStack
