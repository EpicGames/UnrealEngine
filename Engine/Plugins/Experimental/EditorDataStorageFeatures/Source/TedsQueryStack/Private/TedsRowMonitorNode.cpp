// Copyright Epic Games, Inc. All Rights Reserved.

#include "TedsRowMonitorNode.h"

#include "Elements/Framework/TypedElementQueryBuilder.h"
#include "Elements/Interfaces/TypedElementDataStorageInterface.h"

namespace UE::Editor::DataStorage::QueryStack
{
	FRowMonitorNode::FRowMonitorNode(
		ICoreProvider& InStorage, TSharedPtr<IRowNode> InParentNode, TSharedPtr<IQueryNode> InQueryNode)
		: QueryNode(MoveTemp(InQueryNode))
		, ParentNode(MoveTemp(InParentNode))
		, Storage(InStorage)
	{
		checkf(ParentNode, TEXT("FRowMonitorNode requires a valid ParentNode."));
		UpdateColumnsFromQuery();
		UpdateMonitoredColumns();
	}

	FRowMonitorNode::FRowMonitorNode(
		ICoreProvider& InStorage, TSharedPtr<IRowNode> InParentNode, TArray<TObjectPtr<const UScriptStruct>> InColumns)
		: MonitoredColumns(MoveTemp(InColumns))
		, ParentNode(MoveTemp(InParentNode))
		, Storage(InStorage)
	{
		checkf(ParentNode, TEXT("FRowMonitorNode requires a valid ParentNode."));
		UpdateMonitoredColumns();
	}

	FRowMonitorNode::FRowMonitorNode(
		ICoreProvider& InStorage, TSharedPtr<IQueryNode> InQueryNode, TSharedPtr<IRowNode> InParentNode,
		TArray<TObjectPtr<const UScriptStruct>> InMonitoredColumns)
		: MonitoredColumns(MoveTemp(InMonitoredColumns))
		, QueryNode(MoveTemp(InQueryNode))
		, ParentNode(MoveTemp(InParentNode))
		, Storage(InStorage)
		, bFixedColumns(true)
	{
		checkf(ParentNode, TEXT("FRowMonitorNode requires a valid ParentNode."));
		UpdateColumnsFromQuery();
		UpdateMonitoredColumns();
	}

	FRowMonitorNode::~FRowMonitorNode()
	{
		for (QueryHandle Observer : Observers)
		{
			Storage.UnregisterQuery(Observer);
		}
	}

	INode::RevisionId FRowMonitorNode::GetRevision() const
	{
		return Revision;
	}

	void FRowMonitorNode::Update(FTimespan AllottedTime)
	{
		if (QueryNode && QueryNode->GetRevision() != QueryRevision)
		{
			UpdateColumnsFromQuery();
			UpdateMonitoredColumns();
			Revision++;
		}
		
		bool MergeChanges = true;
		if (ParentNode && ParentNode->GetRevision() != ParentRevision)
		{
			ParentRevision = ParentNode->GetRevision();
			Revision++;
			MergeChanges = false;
		}
		
		if (MergeChanges && (!AddedRows.IsEmpty() || !RemovedRows.IsEmpty()))
		{
			AddedRows.Sort();
			AddedRows.MakeUnique();

			RemovedRows.Sort();
			RemovedRows.MakeUnique();

			ResolveRemovedRows();

			if (ParentNode)
			{
				FRowHandleArray& TargetRows = ParentNode->GetMutableRows();
				TargetRows.SortedMerge(AddedRows);
				TargetRows.Remove(RemovedRows.GetRows());
			}

			if (!AddedRows.IsEmpty())
			{
				RowsAddedEvent.Broadcast(AddedRows.GetRows());
			}
			
			if (!RemovedRows.IsEmpty())
			{
				RowsRemovedEvent.Broadcast(RemovedRows.GetRows());
			}

			AddedRows.Empty();
			RemovedRows.Empty();
			
			Revision++;
		}
	}

	void FRowMonitorNode::VisitParents(ParentListCallback Callback)
	{
		if (QueryNode)
		{
			Callback(QueryNode);
		}
		Callback(ParentNode);
	}

	FRowHandleArrayView FRowMonitorNode::GetRows() const
	{
		return ParentNode->GetRows();
	}

	FRowHandleArray& FRowMonitorNode::GetMutableRows()
	{
		return ParentNode->GetMutableRows();
	}

	FRowMonitorNode::FOnRowsChanged& FRowMonitorNode::OnMonitoredRowsAdded()
	{
		return RowsAddedEvent;
	}

	FRowMonitorNode::FOnRowsChanged& FRowMonitorNode::OnMonitoredRowsRemoved()
	{
		return RowsRemovedEvent;
	}

	void FRowMonitorNode::UpdateColumnsFromQuery()
	{
		using namespace UE::Editor::DataStorage::Queries;

		if (!bFixedColumns)
		{
			if (QueryNode)
			{
				TSet<TObjectPtr<const UScriptStruct>> LocalColumns;
				
				const FQueryDescription& Query = Storage.GetQueryDescription(QueryNode->GetQuery());
				
				const FConditions* Conditions = Query.Conditions.GetPtrOrNull();
				TConstArrayView<TWeakObjectPtr<const UScriptStruct>> ComplexConditionColumns;
				if (Conditions && !Conditions->IsEmpty())
				{
					ComplexConditionColumns = Conditions->GetColumns();
				}
				
				LocalColumns.Reserve(Query.SelectionTypes.Num() + Query.ConditionOperators.Num() + ComplexConditionColumns.Num());

				// Collect all columns that are selected for access.
				for (const TWeakObjectPtr<const UScriptStruct>& SelectionColumn : Query.SelectionTypes)
				{
					if (const UScriptStruct* SelectionColumnPtr = SelectionColumn.Get())
					{
						LocalColumns.Add(SelectionColumnPtr);
					}
				}

				// Collect all columns that are used in simple conditions.
				int32 NumConditions = Query.ConditionOperators.Num();
				for (int32 Index = 0; Index < NumConditions; ++Index)
				{
					FQueryDescription::EOperatorType Type = Query.ConditionTypes[Index];
					if (Type == FQueryDescription::EOperatorType::SimpleAll ||
						Type == FQueryDescription::EOperatorType::SimpleAny)
					{
						if (const UScriptStruct* ConditionColumn = Query.ConditionOperators[Index].Type.Get())
						{
							LocalColumns.Add(ConditionColumn);
						}
					}
				}

				// Collect all columns that are used for complex conditions.
				for (const TWeakObjectPtr<const UScriptStruct>& Column : ComplexConditionColumns)
				{
					if (const UScriptStruct* ColumnType = Column.Get())
					{
						LocalColumns.Add(ColumnType);
					}
				}

				MonitoredColumns.Empty();
				MonitoredColumns.Reserve(LocalColumns.Num());
				for (TObjectPtr<const UScriptStruct>& Column : LocalColumns)
				{
					MonitoredColumns.Add(Column);
				}
				
				QueryRevision = QueryNode->GetRevision();
			}
			else
			{
				MonitoredColumns.Empty();
				QueryRevision = UninitializedRevisionId;
			}
		}
	}

	void FRowMonitorNode::UpdateMonitoredColumns()
	{
		using namespace UE::Editor::DataStorage::Queries;

		static FName Name = FName("QueryStack Row Monitor node");

		for (QueryHandle Observer : Observers)
		{
			Storage.UnregisterQuery(Observer);
		}

		Observers.Reset(MonitoredColumns.Num() * 2); // Add double, one for the OnAdd and one for the OnRemove.

		if (QueryNode)
		{
			auto OnAdd = [this](const FQueryDescription&, IQueryContext& Context)
				{
					AddedRows.Append(FRowHandleArrayView(Context.GetRowHandles(), FRowHandleArrayView::EFlags::IsUnique));
				};
			auto OnRemove = [this](const FQueryDescription&, IQueryContext& Context)
				{
					RemovedRows.Append(FRowHandleArrayView(Context.GetRowHandles(), FRowHandleArrayView::EFlags::IsUnique));
				};
			
			const FQueryDescription& QueryBase = Storage.GetQueryDescription(QueryNode->GetQuery());

			for (const TObjectPtr<const UScriptStruct>& Column : MonitoredColumns)
			{
				FString OnAddName = TEXT("QueryStack Row Monitor node: OnAdd - ");
				Column->GetFName().AppendString(OnAddName);
				FString OnRemoveName = TEXT("QueryStack Row Monitor node: OnRemove - ");
				Column->GetFName().AppendString(OnRemoveName);

				FQueryDescription OnAddObserver = QueryBase;
				OnAddObserver.Callback.Name = FName(OnAddName);
				OnAddObserver.Callback.Type = EQueryCallbackType::ObserveAdd;
				OnAddObserver.Callback.ExecutionMode = EExecutionMode::GameThread;
				OnAddObserver.Callback.Function = OnAdd;
				OnAddObserver.Callback.MonitoredType = Column;
				Observers.Add(Storage.RegisterQuery(MoveTemp(OnAddObserver)));

				FQueryDescription OnRemoveObserver = QueryBase;
				OnRemoveObserver.Callback.Name = FName(OnRemoveName);
				OnRemoveObserver.Callback.Type = EQueryCallbackType::ObserveRemove;
				OnRemoveObserver.Callback.ExecutionMode = EExecutionMode::GameThread;
				OnRemoveObserver.Callback.Function = OnRemove;
				OnRemoveObserver.Callback.MonitoredType = Column;
				Observers.Add(Storage.RegisterQuery(MoveTemp(OnRemoveObserver)));
			}
		}
		else
		{
			auto OnAdd = [this](IQueryContext& Context)
				{
					AddedRows.Append(FRowHandleArrayView(Context.GetRowHandles(), FRowHandleArrayView::EFlags::IsUnique));
				};
			auto OnRemove = [this](IQueryContext& Context)
				{
					RemovedRows.Append(FRowHandleArrayView(Context.GetRowHandles(), FRowHandleArrayView::EFlags::IsUnique));
				};

			for (const TObjectPtr<const UScriptStruct>& Column : MonitoredColumns)
			{
				FString OnAddName = TEXT("QueryStack Row Monitor node: OnAdd - ");
				Column->GetFName().AppendString(OnAddName);
				FString OnRemoveName = TEXT("QueryStack Row Monitor node: OnRemove - ");
				Column->GetFName().AppendString(OnRemoveName);

				Observers.Add(Storage.RegisterQuery(Select(FName(OnAddName),
					FObserver(FObserver::EEvent::Add, Column.Get())
						.SetExecutionMode(EExecutionMode::GameThread), OnAdd)
					.Compile()));
				Observers.Add(Storage.RegisterQuery(Select(FName(OnRemoveName),
					FObserver(FObserver::EEvent::Remove, Column.Get())
						.SetExecutionMode(EExecutionMode::GameThread), OnRemove)
					.Compile()));
			}
		}
	}

	void FRowMonitorNode::ResolveRemovedRows()
	{
		if (QueryNode)
		{
			// OnRemove observers are fired when any column from our conditions are removed from the row, however this doesn't necessarily mean
			// that the row should be removed as it could still be matching the query. For instance, if the Conditions are (ColumnA || ColumnB), the row
			// contains both columns and only ColumnB is removed the row still matches the original query and shouldn't be removed from the parent node.
			const FQueryDescription& Query = Storage.GetQueryDescription(QueryNode->GetQuery());

			// RemovedRows is guaranteed to be sorted when this function is called, but we'll add a sanity check to sort it anyways in case that changes
			if (!RemovedRows.IsSorted())
			{
				RemovedRows.Sort();
			}

			// We don't currently have a way to detect if a row matches a query using the old syntax, so this logic only applies to the FConditions
			// based syntax. Which means that you could get an incorrect output if you use this node with a complex .Where() syntax.
			if (const Queries::FConditions* Conditions = Query.Conditions.GetPtrOrNull())
			{
				FRowHandleArray RowsToNotRemove;
				RowsToNotRemove.Reserve(RemovedRows.Num());

				for (RowHandle Row : RemovedRows.GetRows())
				{
					// If the row still matches the conditions, don't remove it.
					if (Storage.MatchesColumns(Row, *Conditions))
					{
						// Since RemovedRows is sorted, iterating through it and adding rows keep that property
						static constexpr FRowHandleArrayView::EFlags Flags(FRowHandleArrayView::EFlags::IsSorted | FRowHandleArrayView::EFlags::IsUnique);
						RowsToNotRemove.Add(Row, Flags);
					}
				}

				RemovedRows.Remove(RowsToNotRemove.GetRows());
			}
		}
	}
} // namespace UE::Editor::DataStorage::QueryStack
