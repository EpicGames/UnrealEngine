// Copyright Epic Games, Inc. All Rights Reserved.

#include "TedsRowOrderInversionNode.h"

namespace UE::Editor::DataStorage::QueryStack
{
	FRowOrderInversionNode::FRowOrderInversionNode(TSharedPtr<IRowNode> InParent, bool bInIsEnabled)
		: Parent(MoveTemp(InParent))
		, bIsEnabled(bInIsEnabled)
	{
		if (bInIsEnabled)
		{
			ApplyOrder();
		}
		else
		{
			bRequiresSync = true;
		}
	}

	void FRowOrderInversionNode::Enable(bool bInIsEnabled)
	{
		bIsEnabled = bInIsEnabled;
	}
	
	bool FRowOrderInversionNode::IsEnabled() const
    {
    	return bIsEnabled;
    }

	INode::RevisionId FRowOrderInversionNode::GetRevision() const
	{
		return Revision;
	}

	void FRowOrderInversionNode::Update(FTimespan AllottedTime)
	{
		RevisionId LocalParentRevision = Parent->GetRevision();
		if (bIsEnabled)
		{
			if (LocalParentRevision != ParentRevision || bRequiresSync)
			{
				ParentRevision = LocalParentRevision;
				Revision++;
				bRequiresSync = false;
				ApplyOrder();
			}
		}
		else
		{
			if (LocalParentRevision != ParentRevision)
			{
				ParentRevision = LocalParentRevision;
				Revision++;
				bRequiresSync = true;
			}
		}
	}

	void FRowOrderInversionNode::VisitParents(ParentListCallback Callback)
	{
		Callback(Parent);
	}

	FRowHandleArrayView FRowOrderInversionNode::GetRows() const
	{
		return Parent->GetRows();
	}

	FRowHandleArray& FRowOrderInversionNode::GetMutableRows()
	{
		return Parent->GetMutableRows();
	}

	void FRowOrderInversionNode::ApplyOrder()
	{
		Parent->GetMutableRows().InvertOrder();
		Revision++;
	}
} // namespace UE::Editor::DataStorage::QueryStack
