// Copyright Epic Games, Inc. All Rights Reserved.

#include "TedsRowQueryResultsNode.h"

#include "Elements/Framework/TypedElementQueryBuilder.h"
#include "Elements/Interfaces/TypedElementDataStorageInterface.h"
#include "Hash/xxhash.h"

namespace UE::Editor::DataStorage::QueryStack
{
	FRowQueryResultsNode::FRowQueryResultsNode(ICoreProvider& Storage, TSharedPtr<IQueryNode> InQueryNode)
		: QueryNode(MoveTemp(InQueryNode))
		, Storage(Storage)
	{
	}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	// Remaps the only valid combination of ESyncFlags to the new sync actions for backwards compatibility.
	static FRowQueryResultsNode::ESyncActions RemapSyncOptions(FRowQueryResultsNode::EDeprecatedSyncFlags SyncFlags)
	{
		bool bRequestIncrementWhenDifferent = EnumHasAllFlags(SyncFlags, FRowQueryResultsNode::EDeprecatedSyncFlags::IncrementWhenDifferent);
		EnumRemoveFlags(SyncFlags, FRowQueryResultsNode::ESyncFlags::IncrementWhenDifferent);

		switch (SyncFlags)
		{
		default:
			[[fallthrough]];
		case FRowQueryResultsNode::ESyncFlags::None:
			return FRowQueryResultsNode::ESyncActions::None;
		case FRowQueryResultsNode::ESyncFlags::RefreshOnUpdate:
			return bRequestIncrementWhenDifferent
				? FRowQueryResultsNode::ESyncActions::RefreshOnUpdate
				: FRowQueryResultsNode::ESyncActions::ForceRefreshOnUpdate;
		case FRowQueryResultsNode::ESyncFlags::RefreshOnQueryChange:
			return FRowQueryResultsNode::ESyncActions::RefreshOnQueryChange;
		}
	}

	FRowQueryResultsNode::FRowQueryResultsNode(ICoreProvider& Storage, TSharedPtr<IQueryNode> InQueryNode, EDeprecatedSyncFlags InSyncFlags)
		: FRowQueryResultsNode(Storage, MoveTemp(InQueryNode), RemapSyncOptions(InSyncFlags))
	{
	}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	FRowQueryResultsNode::FRowQueryResultsNode(ICoreProvider& Storage, TSharedPtr<IQueryNode> InQueryNode, ESyncActions InSyncAction)
		: QueryNode(MoveTemp(InQueryNode))
		, Storage(Storage)
		, SyncAction(InSyncAction)
	{
	}
	
	INode::RevisionId FRowQueryResultsNode::GetRevision() const
	{
		return Revision;
	}

	void FRowQueryResultsNode::Refresh()
	{
		Refresh(FTimespan::MaxValue());
	}

	void FRowQueryResultsNode::Refresh(FTimespan AllottedTime)
	{
		Rows.Empty();
		RefreshInternal(AllottedTime, Rows);
		Revision++;
	}

	void FRowQueryResultsNode::Update(FTimespan AllottedTime)
	{
		bool bUpdateQuery = QueryRevision != QueryNode->GetRevision();
		if (bUpdateQuery)
		{
			QueryRevision = QueryNode->GetRevision();
			TopologyHash = 0;
			QueryUpdated();
		}

		switch (SyncAction)
		{
		case ESyncActions::None:
			break;
		case ESyncActions::RefreshOnUpdate:
		{
			uint64 NewTopologyHash = Storage.CalculateQueryTablesTopologyHash(QueryNode->GetQuery());
			if (TopologyHash != NewTopologyHash)
			{
				Refresh(AllottedTime);
				TopologyHash = NewTopologyHash;
			}
			break;
		}
		case ESyncActions::RefreshOnQueryChange:
			if (bUpdateQuery)
			{
				Refresh(AllottedTime);
			}
			break;
		case ESyncActions::ForceRefreshOnUpdate:
			Refresh(AllottedTime);
			break;
		}
	}

	void FRowQueryResultsNode::VisitParents(ParentListCallback Callback)
	{
		Callback(QueryNode);
	}

	FRowHandleArrayView FRowQueryResultsNode::GetRows() const
	{
		return Rows.GetRows();
	}

	FRowHandleArray& FRowQueryResultsNode::GetMutableRows()
	{
		return Rows;
	}

	void FRowQueryResultsNode::RefreshInternal(FTimespan AllottedTime, FRowHandleArray& TargetRows)
	{
		RefreshInternal(TargetRows);
	}
	
	void FRowQueryResultsNode::RefreshInternal(FRowHandleArray& TargetRows)
	{
		using namespace UE::Editor::DataStorage::Queries;

		FQueryResult Result = Storage.RunQuery(QueryNode->GetQuery()); // This is optimized to only collect the number of rows.
		TargetRows.Reserve(Result.Count);

		Storage.RunQuery(QueryNode->GetQuery(), 
			EDirectQueryExecutionFlags::AllowBoundQueries | EDirectQueryExecutionFlags::IgnoreActiveState,
			CreateDirectQueryCallbackBinding(
				[&TargetRows](IDirectQueryContext& Context)
				{
					TargetRows.Append(FRowHandleArrayView(Context.GetRowHandles(), FRowHandleArrayView::EFlags::IsUnique));
				}));
	}
} // namespace UE::Editor::DataStorage::QueryStack
