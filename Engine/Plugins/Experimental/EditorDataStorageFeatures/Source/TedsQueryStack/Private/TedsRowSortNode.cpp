// Copyright Epic Games, Inc. All Rights Reserved.

#include "TedsRowSortNode.h"

#include "Algo/BinarySearch.h"
#include "Algo/Sort.h"
#include "Async/ParallelFor.h"
#include "Misc/ScopeLock.h"
#include "ProfilingDebugging/CpuProfilerTrace.h"
#include "Sorting/SortTasks.h"

namespace UE::Editor::DataStorage::QueryStack
{
	namespace Private
	{
		struct FSortContext
		{
			static constexpr int32 PrefixBatchSize = 2500;
			static constexpr int32 SortBatchSize = 2500;

			TArray<Sorting::FPrefix> PrefixArray;
			TArray<Sorting::FPrefix> PrefixShadowArray;
			TArray<RowHandle> RowHandleArray;
			Sorting::FSortTasks Tasks;
		};
	} // namespace Private

	FRowSortNode::FRowSortNode(ICoreProvider& InStorage, TSharedPtr<IRowNode> InParent,
		FTimespan InMaxFrameDuration)
		: Parent(MoveTemp(InParent))
		, MaxFrameDuration(InMaxFrameDuration)
		, Storage(InStorage)
	{
	}

	FRowSortNode::FRowSortNode(ICoreProvider& InStorage, TSharedPtr<IRowNode> InParent,
		TSharedPtr<const FColumnSorterInterface> InColumnSorter, FTimespan InMaxFrameDuration)
		: ColumnSorter(MoveTemp(InColumnSorter))
		, Parent(MoveTemp(InParent))
		, MaxFrameDuration(InMaxFrameDuration)
		, Storage(InStorage)
	{
	}

	void FRowSortNode::SetColumnSorter(TSharedPtr<const FColumnSorterInterface> InColumnSorter)
	{
		// Always set this even if it can cause some redundant extra work. It can be that child nodes, such as the node that reverses order
		// or a secondary sort, was called and a new pass is needed to sort everything back in the correct order.
		ColumnSorter = MoveTemp(InColumnSorter);

		// Restart sort on next update.
		LastUpdatedRevision = TNumericLimits<RevisionId>::Max();
	}
	
	TSharedPtr<const FColumnSorterInterface> FRowSortNode::GetColumnSorter() const
    {
    	return ColumnSorter;
    }

	bool FRowSortNode::IsSorting() const
	{
		return SortContext.IsValid();
	}

	INode::RevisionId FRowSortNode::GetRevision() const
	{
		return Revision;
	}

	void FRowSortNode::Update(FTimespan AllottedTime)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE("[TEDS Sorting] Sort node update");

		uint64 StartTime = FPlatformTime::Cycles64();

		if (Parent->GetRevision() != LastUpdatedRevision)
		{
			LastUpdatedRevision = Parent->GetRevision();

			if (ColumnSorter)
			{
				// If there's already a sort in progress, this will effectively cancel it, clean it up and start a new one.
				SortContext = MakePimpl<Private::FSortContext>();

				switch (ColumnSorter->GetSortType())
				{
				case FColumnSorterInterface::ESortType::FixedSize64:
					SetupFixedSize64();
					break;
				case FColumnSorterInterface::ESortType::FixedSizeOnly:
					SetupFixedSizeOnly();
					break;
				case FColumnSorterInterface::ESortType::ComparativeSort:
					SetupComparativeSort();
					break;
				case FColumnSorterInterface::ESortType::HybridSort:
					SetupHybridSort();
					break;
				default:
					checkf(false, TEXT("Unsupported sort type: %i"), static_cast<int32>(ColumnSorter->GetSortType()));
					break;
				}
			}
			else
			{
				Revision++;
			}
		}

		if (SortContext)
		{
			if (SortContext->Tasks.HasRemainingTasks())
			{
				// If the update is uncapped then limit to the requested upper limit as otherwise the editor can lock up
				// for a long time. If there's explicit time assigned, use as much of it as needed, even if it exceeds
				// the maximum duration that's specified because TEDS is telling this function that's the amount of
				// time that's available.
				SortContext->Tasks.Update(AllottedTime == INode::UnlimitedTime ? MaxFrameDuration : AllottedTime);
			}

			// Check remaining tasks again as the update above could have added new tasks.
			if (!SortContext->Tasks.HasRemainingTasks())
			{
				// No work left delete the sort context and stop further processing.
				Revision++;
				SortContext.Reset();
			}
		}
	}

	void FRowSortNode::VisitParents(ParentListCallback Callback)
	{
		Callback(Parent);
	}

	void FRowSortNode::SetupFixedSize64()
	{
		FRowHandleArray& MutableRows = Parent->GetMutableRows();
		FRowHandleArrayView Rows = Parent->GetRows();
		TArrayView<RowHandle> RowView = MutableRows.GetMutableRows(MutableRows.IsUnique()
			? FRowHandleArray::EFlags::IsUnique
			: FRowHandleArray::EFlags::None);
		SortContext->PrefixArray.AddUninitialized(Rows.Num());
		SortContext->Tasks.CollectPrefixes(SortContext->PrefixArray, Rows, Storage, ColumnSorter, Private::FSortContext::PrefixBatchSize);
		SortContext->Tasks.AddBarrier();

		SortContext->Tasks.RadixSortPrefixes(SortContext->PrefixArray);
		SortContext->Tasks.AddBarrier();

		SortContext->Tasks.CopyPrefixesToRowHandleArray(RowView, SortContext->PrefixArray);
	}

	void FRowSortNode::SetupFixedSizeOnly()
	{
		FRowHandleArray& MutableRows = Parent->GetMutableRows();
		FRowHandleArrayView Rows = Parent->GetRows();
		SortContext->PrefixArray.AddUninitialized(Rows.Num());
		SortContext->Tasks.CollectPrefixes(
			SortContext->PrefixArray, Rows, Storage, ColumnSorter, Private::FSortContext::PrefixBatchSize);
		SortContext->Tasks.AddBarrier();

		SortContext->PrefixShadowArray.AddUninitialized(Rows.Num());
		FRowHandleArray::EFlags Flags = MutableRows.IsUnique() ? FRowHandleArray::EFlags::IsUnique : FRowHandleArray::EFlags::None;
		SortContext->Tasks.BucketizeRange(MutableRows.GetMutableRows(Flags), SortContext->PrefixArray,
			SortContext->PrefixShadowArray, ColumnSorter, Storage, Private::FSortContext::PrefixBatchSize,
			0 /* Never fall down to a comparative sort. The assumption is that this is called when all prefixes are the same limited size, though this is not mandatory. */);
	}

	void FRowSortNode::SetupComparativeSort()
	{
		FRowHandleArray& MutableRows = Parent->GetMutableRows();
		FRowHandleArray::EFlags Flags = MutableRows.IsUnique() ? FRowHandleArray::EFlags::IsUnique : FRowHandleArray::EFlags::None;
		SortContext->Tasks.ComparativeSortRowHandleRanges(MutableRows.GetMutableRows(Flags), Storage, ColumnSorter, Private::FSortContext::SortBatchSize);
		SortContext->Tasks.AddBarrier();

		SortContext->RowHandleArray.AddUninitialized(MutableRows.Num() / 2); // Used only for the right portion of the merge for temporary storage.
		SortContext->Tasks.MergeSortedAdjacentRanges(
			MutableRows.GetMutableRows(Flags), SortContext->RowHandleArray, Storage, ColumnSorter, Private::FSortContext::SortBatchSize);
	}

	void FRowSortNode::SetupHybridSort()
	{
		FRowHandleArray& MutableRows = Parent->GetMutableRows();
		FRowHandleArrayView Rows = Parent->GetRows();
			
		if (Rows.Num() >= Private::FSortContext::SortBatchSize)
		{
			SortContext->PrefixArray.AddUninitialized(Rows.Num());
			SortContext->Tasks.CollectPrefixes(
				SortContext->PrefixArray, Rows, Storage, ColumnSorter, Private::FSortContext::PrefixBatchSize);
			SortContext->Tasks.AddBarrier();

			SortContext->PrefixShadowArray.AddUninitialized(Rows.Num());
			FRowHandleArray::EFlags Flags = MutableRows.IsUnique() ? FRowHandleArray::EFlags::IsUnique : FRowHandleArray::EFlags::None;
			SortContext->Tasks.BucketizeRange(MutableRows.GetMutableRows(Flags), SortContext->PrefixArray,
				SortContext->PrefixShadowArray, ColumnSorter, Storage, Private::FSortContext::PrefixBatchSize,
				Private::FSortContext::SortBatchSize);
		}
		else
		{
			// If there are less than the sort batch size of entries then there's no need for a prefix pass as it'll add extra work to sort since each
			// bucket is going to be smaller than the batch size and instantly go to comparative sort.
			FRowHandleArray::EFlags Flags = MutableRows.IsUnique() ? FRowHandleArray::EFlags::IsUnique : FRowHandleArray::EFlags::None;
			SortContext->Tasks.ComparativeSortRowHandleRanges(MutableRows.GetMutableRows(Flags), Storage, ColumnSorter, Private::FSortContext::SortBatchSize);
		}
	}

	FRowHandleArrayView FRowSortNode::GetRows() const
	{
		return Parent->GetRows();
	}

	FRowHandleArray& FRowSortNode::GetMutableRows()
	{
		return Parent->GetMutableRows();
	}
} // namespace UE::Editor::DataStorage::QueryStack
