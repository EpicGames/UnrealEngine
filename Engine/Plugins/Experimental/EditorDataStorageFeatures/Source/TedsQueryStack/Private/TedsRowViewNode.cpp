// Copyright Epic Games, Inc. All Rights Reserved.

#include "TedsRowViewNode.h"

namespace UE::Editor::DataStorage::QueryStack
{
	FRowViewNode::FRowViewNode(FRowHandleArray& Rows)
		: Rows(&Rows)
	{}

	void FRowViewNode::MarkDirty()
	{
		Revision++;
	}

	void FRowViewNode::ResetView(FRowHandleArray& InRows)
	{
		Rows = &InRows;
		MarkDirty();
	}

	INode::RevisionId FRowViewNode::GetRevision() const
	{
		return Revision;
	}

	void FRowViewNode::Update(FTimespan AllottedTime)
	{
	}

	void FRowViewNode::VisitParents(ParentListCallback Callback)
	{
		// No parents to visit.
	}

	FRowHandleArrayView FRowViewNode::GetRows() const
	{
		return Rows->GetRows();
	}

	FRowHandleArray& FRowViewNode::GetMutableRows()
	{
		// No need to check for validity as there's no function that allows a null pointer. The reason it's a pointer
		// is so it can be reassigned.
		return *Rows;
	}
} // namespace UE::Editor::DataStorage::QueryStack
