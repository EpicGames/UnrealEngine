// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <functional>
#include <type_traits>
#include "Containers/UnrealString.h"
#include "DataStorage/CommonTypes.h"
#include "Internationalization/Text.h"
#include "UObject/NameTypes.h"

class FProperty;
class UScriptStruct;

namespace UE::Editor::DataStorage::QueryStack::Searching
{
	template <typename T> struct MemberPointerTraits;
	template <typename Type, typename Class>
	struct MemberPointerTraits<Type Class::*>
	{
		using MemberType = Type;
		using ClassType = Class;
	};

	template <typename T>
	concept SupportedSearchPropertyType = std::is_same_v<T, FString> || std::is_same_v<T, FText> || std::is_same_v<T, FName>;

	template <typename T>
	concept MemberObjectPointer = std::is_member_object_pointer_v<T>;

	// Concept to allow binding to a searchable column variable.
	template <auto Ptr>
	concept SearchableMemberColumn =
		MemberObjectPointer<decltype(Ptr)> &&
		TDataColumnType<typename MemberPointerTraits<decltype(Ptr)>::ClassType> &&
		SupportedSearchPropertyType<typename MemberPointerTraits<decltype(Ptr)>::MemberType>;

	struct FCaseInsensitiveHash
	{
		size_t operator()(FString::ElementType Value) const;
	};
	struct FCaseInsensitiveCompare
	{
		bool operator()(FString::ElementType Lhs, FString::ElementType Rhs) const;
	};
	using FStringSearcher = std::boyer_moore_horspool_searcher<const FString::ElementType*, FCaseInsensitiveHash, FCaseInsensitiveCompare>;

	struct FSearchContext
	{
		FSearchContext(FString InSearchString);

		FString SearchString;
		FStringSearcher StringSearcher;
		int32 CurrentSearcher = 0;
	};

	using StringSearchFunction = bool(*)(const Searching::FSearchContext& Context, const FProperty* Property, const void* Column, FString& TempString);

	template<auto MemberVariable> requires Searching::SearchableMemberColumn<MemberVariable>
	StringSearchFunction CreateSearchFunction();
	TEDSQUERYSTACK_API StringSearchFunction CreateSearchFunction(const FProperty* Property);

	template<auto MemberVariable> requires SearchableMemberColumn<MemberVariable>
	bool Search(const void* Column, const FSearchContext& Context, FString& TempString);
	TEDSQUERYSTACK_API bool Search(const FString& String, const FSearchContext& Context, FString& TempString);
	TEDSQUERYSTACK_API bool Search(const FName& String, const FSearchContext& Context, FString& TempString);
	TEDSQUERYSTACK_API bool Search(const FText& String, const FSearchContext& Context, FString& TempString);

	TEDSQUERYSTACK_API void ListSearchableColumns(TFunctionRef<void(const UScriptStruct*)> Callback);
	TEDSQUERYSTACK_API void ListSearchableProperties(const UScriptStruct* ColumnType, TFunctionRef<void(const FProperty*)> Callback);
} // namespace UE::Editor::DataStorage::QueryStack::Searching

#include "Searching/SearchUtils.inl"