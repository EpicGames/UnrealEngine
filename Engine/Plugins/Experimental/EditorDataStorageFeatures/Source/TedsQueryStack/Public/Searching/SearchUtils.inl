// Copyright Epic Games, Inc. All Rights Reserved.

namespace UE::Editor::DataStorage::QueryStack::Searching
{
	template<auto MemberVariable> requires Searching::SearchableMemberColumn<MemberVariable>
	StringSearchFunction CreateSearchFunction()
	{
		return [](const Searching::FSearchContext& Context, const FProperty* Searcher, const void* Column, FString& TempString)
			{
				return Searching::Search<MemberVariable>(Column, Context, TempString);
			};
	}

	template<auto MemberVariable> requires SearchableMemberColumn<MemberVariable>
	bool Search(const void* Column, const FSearchContext& Context, FString& TempString)
	{
		using Class = typename Searching::MemberPointerTraits<decltype(MemberVariable)>::ClassType;
		
		const Class& ColumnCast = *static_cast<const Class*>(Column);
		return Search(ColumnCast.*MemberVariable, Context, TempString);
	}
} // namespace UE::Editor::DataStorage::QueryStack::Searching
