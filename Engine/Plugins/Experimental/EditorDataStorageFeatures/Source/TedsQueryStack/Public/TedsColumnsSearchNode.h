// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <type_traits>
#include "Containers/StringFwd.h"
#include "DataStorage/CommonTypes.h"
#include "DataStorage/Queries/Conditions.h"
#include "Elements/Framework/TypedElementRowHandleArray.h"
#include "Misc/Timespan.h"
#include "TedsQueryStackInterfaces.h"
#include "Templates/UniquePtr.h"

#include "Searching/SearchUtils.h"

class FProperty;
class UScriptStruct;
namespace UE::Editor::DataStorage
{
	class ICoreProvider;
} // namespace UE::Editor::DataStorage

namespace UE::Editor::DataStorage::QueryStack
{
	namespace Searching
	{
		struct FSearchContext;
	}

	/**
	 * Searches through the provided list of columns and generates a list of rows for all columns with a property that match the search 
	 * string.
	 */
	class FColumnsSearchNode final : public IRowNode
	{
	public:
		enum ESyncActions
		{
			None, /** Do nothing when there are changes to the tables that have columns that are being searched. */
			Refresh, /** If any of the tables with columns that are being searched change, rerun the search. */
		};

		TEDSQUERYSTACK_API FColumnsSearchNode(ICoreProvider& Storage, ESyncActions SyncAction);
		TEDSQUERYSTACK_API virtual ~FColumnsSearchNode() override;

		/**
		 * Starts searching for the provided string in all searchable columns that were selected in the query.
		 * Results may not be collected and can take several frames to arrive.
		 */
		TEDSQUERYSTACK_API void StartSearch(FString SearchString);
		/** Resets a search. */
		TEDSQUERYSTACK_API void ClearSearch();
		/** Returns true if a search is still being processed. */
		TEDSQUERYSTACK_API bool IsSearching() const;

		/**
		 * The total time spend on performing the search.
		 * The returned time span represents the time spend performing search operations. This differs from the wall clock time in that
		 * it exclusively tracks the time spend on performing one or more steps in the search.
		 */
		TEDSQUERYSTACK_API FTimespan GetQuerySearchTime() const;

		/** The total number of cells that will be searched through. */
		TEDSQUERYSTACK_API uint64 CalculateTotalSearchedCellCount() const;

		/** 
		 * Register a column to search.
		 * All properties in the column that are marked with "Searchable" will be included.
		 */
		template<TDataColumnType Column>
		void RegisterColumn(Queries::FConditions Conditions = {});
		/**
		 * Register a column to search.
		 * All properties in the column that are marked with "Searchable" will be included.
		 */
		TEDSQUERYSTACK_API void RegisterColumn(const UScriptStruct* ColumnType, const Queries::FConditions& Conditions = {});
		/**
		 * Register a specific property in a column to search.
		 * Only the provided property will be searched if it contains a type that can be searched. It does not require to be marked with
		 * the "Searchable" metadata.
		 */
		TEDSQUERYSTACK_API void RegisterColumn(const UScriptStruct* ColumnType, const FProperty* Property, Queries::FConditions Conditions = {});
		/**
		 * Register a specific variable in a column to search.
		 * Only the provided variable will be searched if it contains a type that can be searched.
		 */
		template<auto MemberVariable>  requires Searching::SearchableMemberColumn<MemberVariable>
		void RegisterColumn(Queries::FConditions Conditions = {});
		/** Clear all registered columns, properties and variables. */
		TEDSQUERYSTACK_API void UnregisterAllColumns();

		TEDSQUERYSTACK_API virtual RevisionId GetRevision() const override;
		TEDSQUERYSTACK_API virtual void VisitParents(ParentListCallback Callback) override;
		TEDSQUERYSTACK_API virtual FRowHandleArrayView GetRows() const override;
		TEDSQUERYSTACK_API virtual FRowHandleArray& GetMutableRows() override;

	protected:
		TEDSQUERYSTACK_API virtual void Update(FTimespan AllottedTime) override;
	
	private:
		struct FSearchQuery
		{
			QueryHandle Query = InvalidQueryHandle;
			uint64 Hash = 0;
			Searching::StringSearchFunction SearchFunction = nullptr;
			const FProperty* Property = nullptr;
			TWeakObjectPtr<const UScriptStruct> ColumnType;
		};

		// Exported because it's used in the template to bind to a member variable.
		TEDSQUERYSTACK_API QueryHandle CreateQuery(const UScriptStruct* ColumnType, Queries::FConditions&& Conditions);
		void RunSearch(const Searching::FSearchContext& Context, const FSearchQuery& Searcher);
		void ResetSearch();
		
		TArray<FSearchQuery> Searchers;

		TUniquePtr<Searching::FSearchContext> SearchContext;
		FRowHandleArray Rows;
		ICoreProvider& Storage;
		FTimespan TotalSearchTime; // actual time spend processing.
		RevisionId Revision = UninitializedRevisionId;
		ESyncActions SyncAction;
	};

	// Implementations
	template<TDataColumnType Column>
	void FColumnsSearchNode::RegisterColumn(Queries::FConditions Conditions)
	{
		RegisterColumn(Column::StaticStruct(), MoveTemp(Conditions));
	}

	template<auto MemberVariable> requires Searching::SearchableMemberColumn<MemberVariable>
	void FColumnsSearchNode::RegisterColumn(Queries::FConditions Conditions)
	{
		using Class = typename Searching::MemberPointerTraits<decltype(MemberVariable)>::ClassType;
		
		Searchers.Add(FSearchQuery{
			.Query = CreateQuery(Class::StaticStruct(), MoveTemp(Conditions)),
			.SearchFunction = Searching::CreateSearchFunction<MemberVariable>(),
			.ColumnType = Class::StaticStruct() });
	}
} // namespace UE::Editor::DataStorage::QueryStack
