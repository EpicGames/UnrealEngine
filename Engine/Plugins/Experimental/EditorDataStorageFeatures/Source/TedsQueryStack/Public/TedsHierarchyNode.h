// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "TedsRowQueryResultsNode.h"

namespace UE::Editor::DataStorage::QueryStack
{
	// Row which enumerates and stores all children of the given TopLevelRows (including the TopLevelRows)
	//
	// Warning: The resulting row array is not guaranteed to be sorted in depth order
	class FHierarchyRowNode final : public IRowNode
	{
	public:
		enum class ESyncFlags
		{
			/** 
			 * Increment the revision only when the parent's revision changed
			 */
			ParentRevisionDifferent = 0,
			/** 
			 * Compares the previous row list with the current and only updates the revision if there are differences.
			 */
			IncrementWhenDifferent = 1,
			/**
			 * Update the row list whenever "Update" is called. 
			 */
			Always
		};

		enum class EFilterFlags
		{
			/**
			 * No filtering will be applied
			 */
			None = 0,
			/**
			 * Root parent rows will not be included in the row list
			 */
			OnlyIncludeChildRows = 1,
		};

		using EmitRowFn = TFunctionRef<void(TArrayView<const RowHandle>)>;
		
		TEDSQUERYSTACK_API FHierarchyRowNode(
			ICoreProvider& Storage,
			FHierarchyHandle InHierarchyHandle,
			TSharedPtr<QueryStack::IRowNode>& InTopLevelRows,
			ESyncFlags SyncFlags,
			EFilterFlags FilterFlags = EFilterFlags::None);
		
		TEDSQUERYSTACK_API virtual RevisionId GetRevision() const override;
		TEDSQUERYSTACK_API virtual void VisitParents(ParentListCallback Callback) override;
		TEDSQUERYSTACK_API virtual FRowHandleArrayView GetRows() const override;
		TEDSQUERYSTACK_API virtual FRowHandleArray& GetMutableRows() override;

	protected:
		TEDSQUERYSTACK_API virtual void Update(FTimespan AllottedTime) override;

	private:
		void RefreshInternal(ICoreProvider& InCoreProvider, FRowHandleArray& NewRows) const;
		FRowHandleArray& GetTempRowHandleArray();
		
		FRowHandleArray Rows;
		FRowHandleArray NewRows;
		TSharedPtr<QueryStack::IRowNode> TopLevelRows;
		RevisionId ParentRevision = UninitializedRevisionId;
		
		ICoreProvider& Storage;
		FHierarchyHandle HierarchyHandle;
		
		RevisionId Revision = UninitializedRevisionId;
		ESyncFlags SyncFlags;
		EFilterFlags FilterFlags;
	};

	ENUM_CLASS_FLAGS(FHierarchyRowNode::ESyncFlags);
}
