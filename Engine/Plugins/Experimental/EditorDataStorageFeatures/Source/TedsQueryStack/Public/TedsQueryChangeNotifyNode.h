// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"
#include "TedsQueryStackInterfaces.h"

namespace UE::Editor::DataStorage::QueryStack
{
	/**
	 * Query node that will broadcast a message when one or more of its parent nodes has been updated.
	 */
	class FQueryChangeNotifyNode final : public IQueryNode
	{
	public:
		DECLARE_DELEGATE_OneParam(FOnQueryNodeChange, const TSharedPtr<IQueryNode>&);

		TEDSQUERYSTACK_API FQueryChangeNotifyNode(TSharedPtr<IQueryNode> Parent, FOnQueryNodeChange ChangeEvent);
		virtual ~FQueryChangeNotifyNode() override = default;

		TEDSQUERYSTACK_API virtual RevisionId GetRevision() const override;
		TEDSQUERYSTACK_API virtual void VisitParents(ParentListCallback Callback) override;
		TEDSQUERYSTACK_API virtual QueryHandle GetQuery() const override;

	protected:
		virtual void Update(FTimespan AllottedTime) override;

	private:
		TSharedPtr<IQueryNode> Parent;
		RevisionId CachedParentRevision = UninitializedRevisionId;
		FOnQueryNodeChange OnChangeEvent;
	};
} // namespace UE::Editor::DataStorage::QueryStack
