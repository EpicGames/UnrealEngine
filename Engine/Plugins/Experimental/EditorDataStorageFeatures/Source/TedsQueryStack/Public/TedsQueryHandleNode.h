// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "TedsQueryStackInterfaces.h"

namespace UE::Editor::DataStorage::QueryStack
{
	/**
	 * Stores a handle to a query. This should be used to pass an externally created query into the Query Stack.
	 */
	class FQueryHandleNode : public IQueryNode
	{
	public:
		FQueryHandleNode() = default;
		TEDSQUERYSTACK_API explicit FQueryHandleNode(QueryHandle Query);
		virtual ~FQueryHandleNode() override = default;

		TEDSQUERYSTACK_API void SetQuery(QueryHandle InQuery);

		TEDSQUERYSTACK_API virtual RevisionId GetRevision() const override;
		TEDSQUERYSTACK_API virtual void VisitParents(ParentListCallback Callback) override;
		TEDSQUERYSTACK_API virtual QueryHandle GetQuery() const override;

	protected:
		TEDSQUERYSTACK_API virtual void Update(FTimespan AllottedTime) override;

	private:
		QueryHandle Query = InvalidQueryHandle;
		RevisionId Revision = UninitializedRevisionId;
	};
} // namespace UE::Editor::DataStorage::QueryStack
