// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataStorage/Queries/Description.h"
#include "TedsQueryNode.h"

namespace UE::Editor::DataStorage
{
	class ICoreProvider;
} // namespace UE::Editor::DataStorage

namespace UE::Editor::DataStorage::QueryStack
{
	/**
	 * Merges one or more parent queries into a composite query
	 * A merge of two query nodes has the same result of the intersection of the rows when you run the queries individually
	 */
	class FQueryMergeNode : public IQueryNode
	{
	public:
		TEDSQUERYSTACK_API explicit FQueryMergeNode(ICoreProvider& InStorage, TConstArrayView<TSharedPtr<IQueryNode>> InParents);
		TEDSQUERYSTACK_API virtual ~FQueryMergeNode() override;

		TEDSQUERYSTACK_API virtual RevisionId GetRevision() const override;
		TEDSQUERYSTACK_API virtual void VisitParents(ParentListCallback Callback) override;
		TEDSQUERYSTACK_API virtual QueryHandle GetQuery() const override;

	protected:
		TEDSQUERYSTACK_API virtual void Update(FTimespan AllottedTime) override;
		void Rebuild();
		
	private:

		struct FParentInfo
		{
			TSharedPtr<IQueryNode> Parent;
			RevisionId Revision;
		};

		QueryHandle QueryHandle = InvalidQueryHandle;
		TArray<FParentInfo> Parents;
		ICoreProvider& Storage;
		RevisionId Revision = UninitializedRevisionId;
	};
} // namespace UE::Editor::DataStorage::QueryStack
