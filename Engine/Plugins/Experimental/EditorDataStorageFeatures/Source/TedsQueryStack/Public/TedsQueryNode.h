// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataStorage/Queries/Description.h"
#include "TedsQueryStackInterfaces.h"

namespace UE::Editor::DataStorage
{
	class ICoreProvider;
} // namespace UE::Editor::DataStorage

namespace UE::Editor::DataStorage::QueryStack
{
	/**
	 * Stores a query and manages its registration.
	 */
	class FQueryNode : public IQueryNode
	{
	public:
		TEDSQUERYSTACK_API explicit FQueryNode(ICoreProvider& Storage);
		TEDSQUERYSTACK_API FQueryNode(ICoreProvider& Storage, UE::Editor::DataStorage::FQueryDescription Query);
		TEDSQUERYSTACK_API virtual ~FQueryNode() override;

		TEDSQUERYSTACK_API void SetQuery(UE::Editor::DataStorage::FQueryDescription Query);
		TEDSQUERYSTACK_API void ClearQuery();

		TEDSQUERYSTACK_API virtual RevisionId GetRevision() const override;
		TEDSQUERYSTACK_API virtual void VisitParents(ParentListCallback Callback) override;
		TEDSQUERYSTACK_API virtual QueryHandle GetQuery() const override;

	protected:
		TEDSQUERYSTACK_API virtual void Update(FTimespan AllottedTime) override;

	private:
		QueryHandle QueryHandle = InvalidQueryHandle;
		ICoreProvider& Storage;
		RevisionId Revision = UninitializedRevisionId;
	};
} // namespace UE::Editor::DataStorage::QueryStack
