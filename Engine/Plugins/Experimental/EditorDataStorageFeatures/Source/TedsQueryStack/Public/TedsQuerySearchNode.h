// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/StringFwd.h"
#include "DataStorage/CommonTypes.h"
#include "DataStorage/Queries/Conditions.h"
#include "Elements/Framework/TypedElementRowHandleArray.h"
#include "Misc/Timespan.h"
#include "TedsQueryStackInterfaces.h"
#include "TedsRowQueryResultsNode.h"
#include "Templates/UniquePtr.h"

#include "Searching/SearchUtils.h"

class FProperty;
class UScriptStruct;
namespace UE::Editor::DataStorage
{
	class ICoreProvider;
} // namespace UE::Editor::DataStorage

namespace UE::Editor::DataStorage::QueryStack
{
	namespace Searching
	{
		struct FSearchContext;
	}

	/**
	 * Query Stack node that takes a query and searches all selected columns that have searchable properties.
	 * If the search is disabled, either by calling "ClearSearch" or providing an empty string, this node acts
	 * as a pass through and behaves the same way as FRowQueryResultsNode.
	 */
	class FQuerySearchNode final : public FRowQueryResultsNode
	{
	public:
		TEDSQUERYSTACK_API explicit FQuerySearchNode(ICoreProvider& Storage, TSharedRef<IQueryNode> Query);
		TEDSQUERYSTACK_API explicit FQuerySearchNode(ICoreProvider& Storage, TSharedRef<IQueryNode> Query, ESyncActions SyncAction);
		virtual ~FQuerySearchNode() override = default;

		/**
		 * Starts searching for the provided string in all searchable columns that were selected in the query.
		 * Results may not be collected and can take several frames to arrive.
		 */
		TEDSQUERYSTACK_API void StartSearch(FString SearchString);
		/** Resets a search and makes this node behave as a pass-through, resulting in similar behavior to FRowQueryResultsNode. */
		TEDSQUERYSTACK_API void ClearSearch();
		/** Returns true if a search is still being processed. */
		TEDSQUERYSTACK_API bool IsSearching() const;

		/**
		 * The total time spent on performing the search.
		 * This exclusively tracks the time spend on performing one or more steps in the search.
		 */
		TEDSQUERYSTACK_API FTimespan GetQuerySearchTime() const;

	protected:
		static constexpr uint32 InlineColumnCount = 32;

		TEDSQUERYSTACK_API virtual void RefreshInternal(FTimespan AllottedTime, FRowHandleArray& TargetRows) override;
		TEDSQUERYSTACK_API virtual void QueryUpdated();

	private:
		struct FSearchQuery
		{
			Searching::StringSearchFunction SearchFunction = nullptr;
			const FProperty* Property = nullptr;
			TWeakObjectPtr<const UScriptStruct> ColumnType;
			int32 ColumnSize = 0;
		};

		void UpdateSearchersList();
		void RunSearch(const Searching::FSearchContext& Context, FRowHandleArray& TargetRows);
		void ResetSearch();
		
		TArray<FSearchQuery> Searchers;

		TUniquePtr<Searching::FSearchContext> SearchContext;
		FTimespan TotalSearchTime; // actual time spent processing.
		bool bHasPendingSearch = false;
	};
} // namespace UE::Editor::DataStorage::QueryStack
