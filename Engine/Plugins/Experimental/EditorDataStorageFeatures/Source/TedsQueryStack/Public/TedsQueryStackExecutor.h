// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/Interfaces/TypedElementDataStorageInterface.h"
#include "Misc/Timespan.h"
#include "Misc/TVariant.h"
#include "Templates/SharedPointer.h"

namespace UE::Editor::DataStorage::QueryStack
{
	class INode;

	class FExecutorBase
	{
	public:
		TEDSQUERYSTACK_API virtual ~FExecutorBase() = 0;

		TEDSQUERYSTACK_API const FName& GetStackName() const;
		TEDSQUERYSTACK_API TSharedPtr<const INode> GetRootNode() const;
		TEDSQUERYSTACK_API TSharedPtr<INode> GetRootNode();

	// Not exporting these protected functions because this is not intended to be globally extended.
	protected:
		FExecutorBase() = default;
		FExecutorBase(const FName& InStackName, TSharedPtr<INode> InRootNode);

		// Delete copying to keep the guarantee that the name is unique and remove the risk
		// of the same query stack being updated multiple times.
		FExecutorBase(const FExecutorBase&) = delete;
		FExecutorBase& operator=(const FExecutorBase&) = delete;

		FExecutorBase(FExecutorBase&&) = default;
		FExecutorBase& operator=(FExecutorBase&&) = default;

		void FullUpdate();
		void TimedUpdate(FTimespan AllottedTime);
		
		TSharedPtr<INode> RootNode;
		FName StackName;

	private:
		static void FullNodeUpdate(const FName& StackName, const TSharedPtr<INode>& Node);
		static void TimedNodeUpdate(const FName& StackName, const TSharedPtr<INode>& Node, uint64 StartTime, FTimespan AllottedTime);
	};

	/**
	 * Updates the query stack cooperatively with other background tasks.
	 * Preferred default as it reduces overall cost to the editor by allowing TEDS to fairly distribute available CPU cycles.
	 * The cooperative executor may stop partially through updating the stack and continue at a later time.
	 */
	class FCooperativeExecutor final : public FExecutorBase
	{
	public:
		FCooperativeExecutor() = default;
		TEDSQUERYSTACK_API FCooperativeExecutor(const FName& InStackName, ICoreProvider& InDataStorage, TSharedPtr<INode> InRootNode,
			ICoreProvider::ECooperativeTaskPriority InPriority);

		TEDSQUERYSTACK_API FCooperativeExecutor(FCooperativeExecutor&& Rhs);
		TEDSQUERYSTACK_API FCooperativeExecutor& operator=(FCooperativeExecutor&& Rhs);

		TEDSQUERYSTACK_API virtual ~FCooperativeExecutor() override;

	private:
		ICoreProvider* DataStorage = nullptr;
		ICoreProvider::ECooperativeTaskPriority Priority;
	};

	/**
	 * Automatically updates the query stack just before TEDS starts running queries.
	 */
	class FOnUpdateExecutor final : public FExecutorBase
	{
	public:
		FOnUpdateExecutor() = default;
		TEDSQUERYSTACK_API FOnUpdateExecutor(const FName& InStackName, ICoreProvider& InDataStorage, TSharedPtr<INode> InRootNode);
		
		TEDSQUERYSTACK_API FOnUpdateExecutor(FOnUpdateExecutor&& Rhs);
		TEDSQUERYSTACK_API FOnUpdateExecutor& operator=(FOnUpdateExecutor&& Rhs);
		
		TEDSQUERYSTACK_API virtual ~FOnUpdateExecutor() override;

	private:
		void OnUpdate();

		FDelegateHandle Delegate;
		ICoreProvider* DataStorage = nullptr;
	};

	/**
	 * Automatically updates the query stack at the very end of TEDS' update cycle.
	 */
	class FOnUpdateCompletedExecutor final : public FExecutorBase
	{
	public:
		FOnUpdateCompletedExecutor() = default;
		TEDSQUERYSTACK_API FOnUpdateCompletedExecutor(const FName& InStackName, ICoreProvider& InDataStorage, TSharedPtr<INode> InRootNode);

		TEDSQUERYSTACK_API FOnUpdateCompletedExecutor(FOnUpdateCompletedExecutor&& Rhs);
		TEDSQUERYSTACK_API FOnUpdateCompletedExecutor& operator=(FOnUpdateCompletedExecutor&& Rhs);

		TEDSQUERYSTACK_API virtual ~FOnUpdateCompletedExecutor() override;

	private:
		void OnUpdateCompleted();

		FDelegateHandle Delegate;
		ICoreProvider* DataStorage = nullptr;
	};

	/**
	 * Allows explicit calls to update the query stack as part of a cooperative task.
	 * Does not update the provided query stack automatically but depends on calling Update.
	 */
	class FExplicitCooperativeExecutor final : public FExecutorBase
	{
	public:
		FExplicitCooperativeExecutor() = default;
		TEDSQUERYSTACK_API FExplicitCooperativeExecutor(const FName& StackName, TSharedPtr<INode> RootNode);

		FExplicitCooperativeExecutor(FExplicitCooperativeExecutor&&) = default;
		FExplicitCooperativeExecutor& operator=(FExplicitCooperativeExecutor&&) = default;

		virtual ~FExplicitCooperativeExecutor() override = default;

		TEDSQUERYSTACK_API void Update(FTimespan AllottedTime);
	};

	/**
	 * Allows explicit calls to update the query stack.
	 * Does not update the provided query stack automatically but depends on calling Update.
	 */
	class FExplicitUpdateExecutor final : public FExecutorBase
	{
	public:
		FExplicitUpdateExecutor() = default;
		TEDSQUERYSTACK_API FExplicitUpdateExecutor(const FName& StackName, TSharedPtr<INode> RootNode);
		
		FExplicitUpdateExecutor(FExplicitUpdateExecutor&&) = default;
		FExplicitUpdateExecutor& operator=(FExplicitUpdateExecutor&&) = default;

		virtual ~FExplicitUpdateExecutor() override = default;

		TEDSQUERYSTACK_API void Update();
	};
} // namespace UE::Editor::DataStorage::QueryStack