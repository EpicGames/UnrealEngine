// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataStorage/Handles.h"
#include "Containers/ContainersFwd.h"
#include "Elements/Framework/TypedElementRowHandleArray.h"
#include "Elements/Framework/TypedElementRowHandleArrayView.h"
#include "Misc/Timespan.h"
#include "Templates/FunctionFwd.h"

namespace UE::Editor::DataStorage::QueryStack
{
	/** Base interface for a all query stack nodes. */
	class INode
	{
	public:
		friend class FExecutorBase;

		using RevisionId = uint32;
		// The default uninitialized revision Id is zero to signify it is empty and needs to be updated/rows need to be initialized
		static constexpr RevisionId UninitializedRevisionId = 0;
		
		using ParentListCallback = TFunctionRef<void(const TSharedPtr<INode>& Parent)>;
		static inline FTimespan UnlimitedTime = FTimespan::MaxValue();

		virtual ~INode() = default;
		
		/**
		 * Get the current revision of the node. Whenever its internal state changes the revision should be incremented to signal a need 
		 * to update to downstream nodes.
		 */
		virtual RevisionId GetRevision() const = 0;
		
		/** List all direct parent nodes. */
		virtual void VisitParents(ParentListCallback Callback) = 0;

	protected:
		/**
		 * Periodically called to allow a node to perform some work or to verify its internal state.
		 * @AllottedTime The maximum amount of time for the update. The update should try to stay below this time. In case the
		 *		allotted time is equal to "UnlimitedTime" it means there is no upper bound to the amount of time that can be spend
		 *		on the update.
		 */
		virtual void Update(FTimespan AllottedTime) = 0;
	};

	/** Query stack node that works on queries handles. These nodes are typically run in some fashion to be turned into a row node. */
	class IQueryNode : public INode
	{
	public:
		virtual ~IQueryNode() = default;
		/** Returns the handle to the query this node represents. */
		virtual QueryHandle GetQuery() const = 0;
	};

	/** Query stack node that works on row handles. */
	class IRowNode : public INode
	{
	public:
		virtual ~IRowNode() = default;
		/** Retrieve access to the rows used by this node. */
		virtual FRowHandleArrayView GetRows() const = 0;
		/** Retrieve write access to the rows used by this node. Is allowed to return null if write access can't be granted. */
		virtual FRowHandleArray& GetMutableRows() = 0;
	};
} // UE::Editor::DataStorage::QueryStack
