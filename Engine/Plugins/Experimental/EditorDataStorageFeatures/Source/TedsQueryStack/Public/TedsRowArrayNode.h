// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ContainersFwd.h"
#include "TedsQueryStackInterfaces.h"
#include "Elements/Framework//TypedElementRowHandleArray.h"

namespace UE::Editor::DataStorage::QueryStack
{
	/**
	 * Stores row handle array. The array can be directly manipulated, but any changes
	 * outside the Query Stack require MarkDirty to inform the Query Stack that it needs
	 * to update dependent nodes.
	 */
	class FRowArrayNode : public IRowNode
	{
	public:
		FRowArrayNode() = default;
		TEDSQUERYSTACK_API explicit FRowArrayNode(FRowHandleArray Rows);
		virtual ~FRowArrayNode() override = default;

		TEDSQUERYSTACK_API void MarkDirty();
		
		TEDSQUERYSTACK_API virtual RevisionId GetRevision() const override;
		TEDSQUERYSTACK_API virtual void VisitParents(ParentListCallback Callback) override;
		TEDSQUERYSTACK_API virtual FRowHandleArrayView GetRows() const override;
		TEDSQUERYSTACK_API virtual FRowHandleArray& GetMutableRows() override;

	protected:
		TEDSQUERYSTACK_API virtual void Update(FTimespan AllottedTime) override;
	
	private:
		FRowHandleArray Rows;
		RevisionId Revision = UninitializedRevisionId;
	};
} // namespace UE::Editor::DataStorage::QueryStack
