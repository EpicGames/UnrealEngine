// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"
#include "TedsQueryStackInterfaces.h"

namespace UE::Editor::DataStorage::QueryStack
{
	/**
	 * Row node that will broadcast a message when one or more of its parent nodes has been updated.
	 */
	class FRowChangeNotifyNode final : public IRowNode
	{
	public:
		DECLARE_DELEGATE_OneParam(FOnRowNodeChange, const TSharedPtr<IRowNode>&);

		TEDSQUERYSTACK_API FRowChangeNotifyNode(TSharedPtr<IRowNode> ParentNode, FOnRowNodeChange ChangeEvent);
		virtual ~FRowChangeNotifyNode() override = default;

		TEDSQUERYSTACK_API virtual RevisionId GetRevision() const override;
		TEDSQUERYSTACK_API virtual void VisitParents(ParentListCallback Callback) override;
		TEDSQUERYSTACK_API virtual FRowHandleArrayView GetRows() const override;
		TEDSQUERYSTACK_API virtual FRowHandleArray& GetMutableRows() override;

	protected:
		virtual void Update(FTimespan AllottedTime) override;
	
	private:
		TSharedPtr<IRowNode> Parent;
		RevisionId CachedParentRevision = UninitializedRevisionId;
		FOnRowNodeChange OnChangeEvent;
	};
} // namespace UE::Editor::DataStorage::QueryStack
