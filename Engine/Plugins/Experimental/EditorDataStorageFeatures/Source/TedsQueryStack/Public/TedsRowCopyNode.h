// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ContainersFwd.h"
#include "TedsQueryStackInterfaces.h"
#include "Elements/Framework//TypedElementRowHandleArray.h"

namespace UE::Editor::DataStorage::QueryStack
{
	/**
	 * Copies the content of the parent node to a local array. This node can be used
	 * to avoid changes to the parent node being made. An example usage would be if
	 * the parent node is sorted by row handle for performance reasons but the final
	 * list needs to be sorted based on other criteria.
	 */
	class FRowCopyNode : public IRowNode
	{
	public:
		TEDSQUERYSTACK_API explicit FRowCopyNode(TSharedPtr<IRowNode> InParent);
		virtual ~FRowCopyNode() override = default;

		/** Clears out the current row list and gets a fresh copy from the parent. */
		TEDSQUERYSTACK_API void Reset();

		TEDSQUERYSTACK_API virtual RevisionId GetRevision() const override;
		TEDSQUERYSTACK_API virtual void VisitParents(ParentListCallback Callback) override;
		TEDSQUERYSTACK_API virtual FRowHandleArrayView GetRows() const override;
		TEDSQUERYSTACK_API virtual FRowHandleArray& GetMutableRows() override;

	protected:
		TEDSQUERYSTACK_API virtual void Update(FTimespan AllottedTime) override;
	
	private:
		FRowHandleArray Rows;
		TSharedPtr<IRowNode> Parent;
		RevisionId ParentRevision = UninitializedRevisionId;
		RevisionId Revision = UninitializedRevisionId;
	};
} // namespace UE::Editor::DataStorage::QueryStack
