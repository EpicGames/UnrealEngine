// Copyright Epic Games, Inc. All Rights Reserved
 
#pragma once

#include "DataStorage/CommonTypes.h"
#include "Elements/Interfaces/TypedElementDataStorageInterface.h"
#include "Elements/Framework/TypedElementQueryFunctions.h"
#include "TedsQueryStackInterfaces.h"

namespace UE::Editor::DataStorage
{
	class ICoreProvider;
}

namespace UE::Editor::DataStorage::QueryStack
{
	/**
	 * Query stack node that filters out rows based on the provided filter.
	 * Performance-wise this will be slower than using filters on queries, but this node can be used when using
	 * query level filters is not possible.
	 */
	class FRowFilterNode final : public IRowNode
	{
	public:
		FRowFilterNode(ICoreProvider* Storage, const TSharedPtr<IRowNode>& ParentRowNode, ICoreProvider::EFilterOptions Options, 
			const Queries::TConstQueryFunction<bool>& Filter)
			: Storage(Storage)
			, Filter(Filter)
			, ParentRowNode(ParentRowNode)
			, Options(Options)
		{
		}
    
		FRowFilterNode(ICoreProvider* Storage, const TSharedPtr<IRowNode>& ParentRowNode, const Queries::TConstQueryFunction<bool>& Filter)
			: FRowFilterNode(
				Storage, 
				ParentRowNode, 
				ICoreProvider::EFilterOptions::Default, 
				Filter)
		{}
            
		/*
		 * Require that the template type cannot already be a TConstQueryFunction so it will pass to the defined constructor instead of this one.
		 */
		template< typename FilterFunction UE_REQUIRES(
			!std::is_same_v<std::remove_cvref_t<FilterFunction>, Queries::TConstQueryFunction<bool>>) >
		FRowFilterNode(ICoreProvider* Storage, const TSharedPtr<IRowNode>& ParentRowNode, ICoreProvider::EFilterOptions Options, 
			FilterFunction&& Filter)
			: FRowFilterNode(
				Storage, 
				ParentRowNode, 
				Options, 
				Queries::BuildConstQueryFunction<bool>(Forward<FilterFunction>(Filter)))
		{}

		/*
		 * Require that the template type cannot already be a TConstQueryFunction so it will pass to the defined constructor instead of this one.
		 */
		template< typename FilterFunction UE_REQUIRES(
			!std::is_same_v<std::remove_cvref_t<FilterFunction>, Queries::TConstQueryFunction<bool>>) >
		FRowFilterNode(ICoreProvider* Storage, const TSharedPtr<IRowNode>& ParentRowNode, FilterFunction&& Filter)
			: FRowFilterNode(
				Storage, 
				ParentRowNode, 
				ICoreProvider::EFilterOptions::Default, 
				Queries::BuildConstQueryFunction<bool>(Forward<FilterFunction>(Filter)))
		{}

		virtual ~FRowFilterNode() override = default;

		TEDSQUERYSTACK_API virtual FRowHandleArrayView GetRows() const override;
		TEDSQUERYSTACK_API virtual FRowHandleArray& GetMutableRows() override;
		TEDSQUERYSTACK_API virtual RevisionId GetRevision() const override;
		TEDSQUERYSTACK_API virtual void VisitParents(ParentListCallback Callback) override;

		/** Call this when external inputs into the filter, such as search strings or filtering groups, have been updated. */
		TEDSQUERYSTACK_API void ForceRefresh();

	protected:
		TEDSQUERYSTACK_API virtual void Update(FTimespan AllottedTime) override;
	
	private:
		ICoreProvider* Storage;
		Queries::TConstQueryFunction<bool> Filter;
		FRowHandleArray Rows;
		TSharedPtr<IRowNode> ParentRowNode;
		RevisionId CachedParentRevisionID = UninitializedRevisionId;
		RevisionId Revision = UninitializedRevisionId;
		ICoreProvider::EFilterOptions Options;
	};
}
