// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "TedsQueryStackInterfaces.h"

namespace UE::Editor::DataStorage::QueryStack
{
	/**
	 * Sorts the rows in the parent by row handle. Prefer this over using the FRowSortNode when sorting
	 * by row handle as this is somewhat faster and allows the row handle array to keep its sorted status.
	 * If the row handle array is already sorted this node doesn't do any work, making it even faster than
	 * using FRowSortNode.
	 */
	class FRowHandleSortNode : public IRowNode
	{
	public:
		TEDSQUERYSTACK_API explicit FRowHandleSortNode(TSharedPtr<IRowNode> InParent);
		virtual ~FRowHandleSortNode() override = default;

		TEDSQUERYSTACK_API virtual RevisionId GetRevision() const override;
		TEDSQUERYSTACK_API virtual void VisitParents(ParentListCallback Callback) override;
		TEDSQUERYSTACK_API virtual FRowHandleArrayView GetRows() const override;
		TEDSQUERYSTACK_API virtual FRowHandleArray& GetMutableRows() override;

	protected:
		TEDSQUERYSTACK_API virtual void Update(FTimespan AllottedTime) override;
	
	private:
		TSharedPtr<IRowNode> Parent;
		RevisionId ParentRevision = UninitializedRevisionId;
	};
} // namespace UE::Editor::DataStorage::QueryStack
