// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ContainersFwd.h"
#include "Elements/Framework/TypedElementRowHandleArray.h"
#include "TedsQueryStackInterfaces.h"
#include "Templates/SharedPointer.h"

namespace UE::Editor::DataStorage::QueryStack
{
	class FRowMergeNode : public IRowNode
	{
	public:
		enum class EMergeApproach
		{
			Append, //< The rows in each parent are added at the end of the final array.
			Sorted, //< The rows in each parent are combined together in a final sorted array.
			Unique, //< The rows in each parent are combined then sorted and all duplicates are removed from the final array.
			Repeating, //< The rows in each parent are combined then sorted and only rows that appear in at least 2 parents are kept.
		};

		TEDSQUERYSTACK_API FRowMergeNode(TConstArrayView<TSharedPtr<IRowNode>> InParents, EMergeApproach InMergeApproach);
		virtual ~FRowMergeNode() override = default;

		TEDSQUERYSTACK_API virtual RevisionId GetRevision() const override;
		TEDSQUERYSTACK_API virtual void VisitParents(ParentListCallback Callback) override;
		TEDSQUERYSTACK_API virtual FRowHandleArrayView GetRows() const override;
		TEDSQUERYSTACK_API virtual FRowHandleArray& GetMutableRows() override;

	protected:
		TEDSQUERYSTACK_API virtual void Update(FTimespan AllottedTime) override;
	
	private:
		void Merge();

		struct FParentInfo
		{
			TSharedPtr<IRowNode> Parent;
			RevisionId Revision = UninitializedRevisionId;
		};

		TArray<FParentInfo> Parents;
		FRowHandleArray Rows;
		RevisionId Revision = UninitializedRevisionId;
		EMergeApproach MergeApproach;
	};
} // namespace UE::Editor::DataStorage::QueryStack
