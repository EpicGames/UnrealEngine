// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ContainersFwd.h"
#include "Elements/Framework/TypedElementRowHandleArray.h"
#include "TedsQueryStackInterfaces.h"
#include "UObject/ObjectPtr.h"

namespace UE::Editor::DataStorage
{
	class ICoreProvider;
} // namespace UE::Editor::DataStorage

namespace UE::Editor::DataStorage::QueryStack
{
	/**
	 * Monitors tables for the addition and removal of one or more column types and updates the internal status if a change is detected.
	 */
	class FRowMonitorNode : public IRowNode
	{
	public:

		DECLARE_MULTICAST_DELEGATE_OneParam(FOnRowsChanged, FRowHandleArrayView /* InRows */);
		
		TEDSQUERYSTACK_API FRowMonitorNode(
			ICoreProvider& InStorage, TSharedPtr<IRowNode> InParentNode, TSharedPtr<IQueryNode> InQueryNode);
		TEDSQUERYSTACK_API FRowMonitorNode(
			ICoreProvider& InStorage, TSharedPtr<IRowNode> InParentNode, TArray<TObjectPtr<const UScriptStruct>> InColumns);
		TEDSQUERYSTACK_API FRowMonitorNode(
			ICoreProvider& InStorage, TSharedPtr<IQueryNode> InQueryNode, TSharedPtr<IRowNode> InParentNode,
			TArray<TObjectPtr<const UScriptStruct>> InMonitoredColumns);

		TEDSQUERYSTACK_API virtual ~FRowMonitorNode() override;

		TEDSQUERYSTACK_API virtual RevisionId GetRevision() const override;
		TEDSQUERYSTACK_API virtual void VisitParents(ParentListCallback Callback) override;
		TEDSQUERYSTACK_API virtual FRowHandleArrayView GetRows() const override;
		TEDSQUERYSTACK_API virtual FRowHandleArray& GetMutableRows() override;

		// NOTE: These events are not meant to be a wholesale solution for tracking changes to this node, it only fires when monitored rows are modified
		// through observers. They are also deferred to Update() and not fired immediately.
		// For full tracking the regular pattern of checking the node's revision and calling GetRows() should be used
		
		// Event that is called during Update() to allow systems to respond to monitored rows being added
		TEDSQUERYSTACK_API FOnRowsChanged& OnMonitoredRowsAdded();

		// Event that is called during Update() to allow systems to respond to monitored rows being removed
		TEDSQUERYSTACK_API FOnRowsChanged& OnMonitoredRowsRemoved();
	
	protected:
		TEDSQUERYSTACK_API virtual void Update(FTimespan AllottedTime) override;
	
	private:
		void UpdateColumnsFromQuery();
		void UpdateMonitoredColumns();
		void ResolveRemovedRows();

		FRowHandleArray AddedRows;
		FRowHandleArray RemovedRows;

		TArray<QueryHandle> Observers;
		TArray<TObjectPtr<const UScriptStruct>> MonitoredColumns;
		
		TSharedPtr<IQueryNode> QueryNode;
		TSharedPtr<IRowNode> ParentNode;
		ICoreProvider& Storage;
		RevisionId QueryRevision = UninitializedRevisionId;
		RevisionId ParentRevision = UninitializedRevisionId;
		RevisionId Revision = UninitializedRevisionId;
		bool bFixedColumns = false;
		
		FOnRowsChanged RowsAddedEvent;
		FOnRowsChanged RowsRemovedEvent;
	};
} // namespace UE::Editor::DataStorage::QueryStack
