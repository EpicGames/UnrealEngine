// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ContainersFwd.h"
#include "TedsQueryStackInterfaces.h"
#include "Elements/Framework//TypedElementRowHandleArray.h"

namespace UE::Editor::DataStorage::QueryStack
{
	/**
	 * When enabled, orders the rows on the parent in reverse. If disabled, this row has no effect, but if the order on the parent
	 * was inverted it will remain inverted.
	 */
	class FRowOrderInversionNode : public IRowNode
	{
	public:
		TEDSQUERYSTACK_API FRowOrderInversionNode(TSharedPtr<IRowNode> InParent, bool bInIsEnabled);
		virtual ~FRowOrderInversionNode() override = default;

		/** Clears out the current row list and gets a fresh copy from the parent. */
		TEDSQUERYSTACK_API void Enable(bool bInIsEnabled);
		TEDSQUERYSTACK_API bool IsEnabled() const;

		TEDSQUERYSTACK_API virtual RevisionId GetRevision() const override;
		TEDSQUERYSTACK_API virtual void VisitParents(ParentListCallback Callback) override;
		TEDSQUERYSTACK_API virtual FRowHandleArrayView GetRows() const override;
		TEDSQUERYSTACK_API virtual FRowHandleArray& GetMutableRows() override;

	protected:
		TEDSQUERYSTACK_API virtual void Update(FTimespan AllottedTime) override;
	
	private:
		void ApplyOrder();

		TSharedPtr<IRowNode> Parent;
		RevisionId ParentRevision = UninitializedRevisionId;
		RevisionId Revision = UninitializedRevisionId;
		bool bIsEnabled = true;
		bool bRequiresSync = false;
	};
} // namespace UE::Editor::DataStorage::QueryStack
