// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ContainersFwd.h"
#include "Elements/Framework/TypedElementRowHandleArray.h"
#include "TedsQueryStackInterfaces.h"
#include "Misc/EnumClassFlags.h"

namespace UE::Editor::DataStorage
{
	class ICoreProvider;
} // namespace UE::Editor::DataStorage

namespace UE::Editor::DataStorage::QueryStack
{
	/**
	 * Used to convert an IQueryNode into an IRowNode by extracting the rows the query references optionally during update.
	 * This node is cheap to setup, but has diminishing returns when the number of rows increases and when updates happen frequently
	 * as it has to fully extract all rows from TEDS whenever Refresh is called. Use on small tables when the number of calls to 
	 * Refresh can be minimized.
	 */
	class FRowQueryResultsNode : public IRowNode
	{
	public:
		enum class EDeprecatedSyncFlags
		{
			None = 0,
			RefreshOnUpdate = 1 << 0,
			RefreshOnQueryChange = 1 << 1,
			IncrementWhenDifferent = 1 << 2
		};
		using ESyncFlags UE_DEPRECATED(5.8, "'ESyncFlags' has been deprecated, please use `ESyncActions`.") = EDeprecatedSyncFlags;

		enum class ESyncActions
		{
			/**
			 * The node is never updated and requires an explicit call to "Refresh".
			 */
			None,
			/**
			 * Each time "Update" is called the state of the query is checked and compared with the cached state. If any
			 * row additions/removals are detected to any of the tables associated with the query it is rerun.
			 */
			RefreshOnUpdate,
			/** Only update the row list whenever the parent query changes. */
			RefreshOnQueryChange,
			/**
			 * Each time "Update" is called the query is run again and replaces the current set of rows. Unlike "RefreshOnUpdate", 
			 * there is no caching done to mitigate the cost of running a query. This options is only recommended for debugging
			 * purposes as it can significantly impact performance.
			 */
			ForceRefreshOnUpdate,
		};

		TEDSQUERYSTACK_API FRowQueryResultsNode(ICoreProvider& Storage, TSharedPtr<IQueryNode> InQueryNode);
		TEDSQUERYSTACK_API FRowQueryResultsNode(ICoreProvider& Storage, TSharedPtr<IQueryNode> InQueryNode, ESyncActions InSyncAction);
		virtual ~FRowQueryResultsNode() override = default;

		TEDSQUERYSTACK_API void Refresh();
		TEDSQUERYSTACK_API void Refresh(FTimespan AllottedTime);

		TEDSQUERYSTACK_API virtual RevisionId GetRevision() const override;
		TEDSQUERYSTACK_API virtual void VisitParents(ParentListCallback Callback) override;
		TEDSQUERYSTACK_API virtual FRowHandleArrayView GetRows() const override;
		TEDSQUERYSTACK_API virtual FRowHandleArray& GetMutableRows() override;

		UE_DEPRECATED(5.8, "'ESyncFlags' has been deprecated, please use `ESyncActions`.")
		TEDSQUERYSTACK_API FRowQueryResultsNode(ICoreProvider& Storage, TSharedPtr<IQueryNode> InQueryNode, EDeprecatedSyncFlags InSyncFlags);

	protected:
		TEDSQUERYSTACK_API virtual void Update(FTimespan AllottedTime) override;
		TEDSQUERYSTACK_API virtual void RefreshInternal(FTimespan AllottedTime, FRowHandleArray& TargetRows);
		TEDSQUERYSTACK_API virtual void RefreshInternal(FRowHandleArray& TargetRows);
		virtual void QueryUpdated() {};

		FRowHandleArray Rows;
		TSharedPtr<IQueryNode> QueryNode;
		ICoreProvider& Storage;
		uint64 TopologyHash = 0;
		RevisionId QueryRevision = UninitializedRevisionId;
		RevisionId Revision = UninitializedRevisionId;
		ESyncActions SyncAction = ESyncActions::RefreshOnUpdate;
	};
} // namespace UE::Editor::DataStorage::QueryStack

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	ENUM_CLASS_FLAGS(UE::Editor::DataStorage::QueryStack::FRowQueryResultsNode::EDeprecatedSyncFlags)
PRAGMA_ENABLE_DEPRECATION_WARNINGS
