// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Async/Mutex.h"
#include "Elements/Interfaces/TypedElementDataStorageUiInterface.h"
#include "Math/NumericLimits.h"
#include "Misc/EnumClassFlags.h"
#include "Misc/Timespan.h"
#include "TedsQueryStackInterfaces.h"
#include "Templates/FunctionFwd.h"
#include "Templates/PimplPtr.h"

namespace UE::Editor::DataStorage
{
	class ICoreProvider;
}

namespace UE::Editor::DataStorage::QueryStack
{
	namespace Private
	{
		// Information used only when sorting is active.
		struct FSortContext;
	}

	/**
	 * Sorts the rows in the parent node using a provided column sorter. Column sorters can be found on on TEDS UI widget constructors,
	 * or custom sorters can be created.
	 */
	class FRowSortNode : public IRowNode
	{
	public:
		TEDSQUERYSTACK_API FRowSortNode(ICoreProvider& InStorage, TSharedPtr<IRowNode> InParent,
			FTimespan InMaxFrameDuration = FTimespan::FromMilliseconds(10));
		TEDSQUERYSTACK_API FRowSortNode(ICoreProvider& InStorge, TSharedPtr<IRowNode> InParent,
			TSharedPtr<const FColumnSorterInterface> InColumnSorter, FTimespan InMaxFrameDuration = FTimespan::FromMilliseconds(10));
		virtual ~FRowSortNode() override = default;

		TEDSQUERYSTACK_API void SetColumnSorter(TSharedPtr<const FColumnSorterInterface> InColumnSorter);
		TEDSQUERYSTACK_API TSharedPtr<const FColumnSorterInterface> GetColumnSorter() const;
		TEDSQUERYSTACK_API bool IsSorting() const;

		TEDSQUERYSTACK_API virtual RevisionId GetRevision() const override;
		TEDSQUERYSTACK_API virtual void VisitParents(ParentListCallback Callback) override;
		TEDSQUERYSTACK_API virtual FRowHandleArrayView GetRows() const override;
		TEDSQUERYSTACK_API virtual FRowHandleArray& GetMutableRows() override;

	protected:
		TEDSQUERYSTACK_API virtual void Update(FTimespan AllottedTime) override;
	
	private:
		void SetupFixedSize64();
		void SetupFixedSizeOnly();
		void SetupComparativeSort();
		void SetupHybridSort();
		
		TSharedPtr<const FColumnSorterInterface> ColumnSorter;
		TSharedPtr<IRowNode> Parent;
		
		TPimplPtr<Private::FSortContext> SortContext;

		FTimespan MaxFrameDuration;
		RevisionId LastUpdatedRevision = TNumericLimits<RevisionId>::Max();
		RevisionId Revision = UninitializedRevisionId;
		
		ICoreProvider& Storage;
		
		bool bIsParentUnique = false;
	};
} // namespace UE::Editor::DataStorage::QueryStack
