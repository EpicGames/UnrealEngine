// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ContainersFwd.h"
#include "TedsQueryStackInterfaces.h"

namespace UE::Editor::DataStorage::QueryStack
{
	/**
	 * Stores a reference to row handle array. The row handle array that this view is pointing to needs to be kept alive for
	 * as long as this Query Stack node is alive.
	 */
	class FRowViewNode : public IRowNode
	{
	public:
		TEDSQUERYSTACK_API explicit FRowViewNode(FRowHandleArray& Rows);
		virtual ~FRowViewNode() override = default;

		TEDSQUERYSTACK_API void MarkDirty();
		TEDSQUERYSTACK_API void ResetView(FRowHandleArray& InRows);

		TEDSQUERYSTACK_API virtual RevisionId GetRevision() const override;
		TEDSQUERYSTACK_API virtual void VisitParents(ParentListCallback Callback) override;
		TEDSQUERYSTACK_API virtual FRowHandleArrayView GetRows() const override;
		TEDSQUERYSTACK_API virtual FRowHandleArray& GetMutableRows() override;

	protected:
		TEDSQUERYSTACK_API virtual void Update(FTimespan AllottedTime) override;
	
	private:
		FRowHandleArray* Rows = nullptr;
		RevisionId Revision = 1;
	};
} // namespace UE::Editor::DataStorage::QueryStack
