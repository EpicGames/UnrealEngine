// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_TESTS

#include "DataStorage/Features.h"
#include "TedsRowMergeNode.h"
#include "TedsRowViewNode.h"
#include "TedsQueryStackExecutor.h"
#include "Tests/TestHarnessAdapter.h"

TEST_CASE_NAMED(TEDS_QueryStack_RowMergeNode_Tests, "Editor::DataStorage::QueryStack::FRowMergeNode", "[ApplicationContextMask][EngineFilter]")
{
	using namespace UE::Editor::DataStorage;
	using namespace UE::Editor::DataStorage::QueryStack;

	SECTION("No parents")
	{
		TSharedPtr<FRowMergeNode> Node = MakeShared<FRowMergeNode>(TConstArrayView<TSharedPtr<IRowNode>>(), FRowMergeNode::EMergeApproach::Append);
		CHECK_EQUALS(TEXT("Revision"), Node->GetRevision(), 0);
		CHECK_MESSAGE(TEXT("Default for rows is not empty."), Node->GetRows().IsEmpty());
		
		ICoreProvider& DataStorage = *GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName);
		FExplicitUpdateExecutor Executor("NoParentsTestExecutor", Node);
		Executor.Update();
	}

	SECTION("Append two nodes")
	{
		FRowHandleArray Values0;
		FRowHandleArray Values1;
		Values0.Append({ 1, 2, 3 }, FRowHandleArray::EFlags::IsSorted | FRowHandleArray::EFlags::IsUnique);
		Values1.Append({ 1, 2, 3 }, FRowHandleArray::EFlags::IsSorted | FRowHandleArray::EFlags::IsUnique);
		
		TSharedPtr<IRowNode> View0 = MakeShared<FRowViewNode>(Values0);
		TSharedPtr<IRowNode> View1 = MakeShared<FRowViewNode>(Values1);
		
		FRowMergeNode Node({View0, View1}, FRowMergeNode::EMergeApproach::Append);
		
		FRowHandleArrayView Rows = Node.GetRows();
		CHECK_MESSAGE(TEXT("Expected rows to not be sorted."), !Rows.IsSorted());
		CHECK_EQUALS(TEXT("Size"), Rows.Num(), 6);
		CHECK_EQUALS(TEXT("Rows[0]"), Rows[0], 1llu);
		CHECK_EQUALS(TEXT("Rows[1]"), Rows[1], 2llu);
		CHECK_EQUALS(TEXT("Rows[2]"), Rows[2], 3llu);
		CHECK_EQUALS(TEXT("Rows[3]"), Rows[3], 1llu);
		CHECK_EQUALS(TEXT("Rows[4]"), Rows[4], 2llu);
		CHECK_EQUALS(TEXT("Rows[5]"), Rows[5], 3llu);
	}

	SECTION("Merge two nodes sorted")
	{
		FRowHandleArray Values0;
		FRowHandleArray Values1;
		Values0.Append({ 1, 2, 3 }, FRowHandleArray::EFlags::IsSorted | FRowHandleArray::EFlags::IsUnique);
		Values1.Append({ 1, 2, 3 }, FRowHandleArray::EFlags::IsSorted | FRowHandleArray::EFlags::IsUnique);

		TSharedPtr<IRowNode> View0 = MakeShared<FRowViewNode>(Values0);
		TSharedPtr<IRowNode> View1 = MakeShared<FRowViewNode>(Values1);
		
		FRowMergeNode Node({ View0, View1 }, FRowMergeNode::EMergeApproach::Sorted);

		FRowHandleArrayView Rows = Node.GetRows();
		CHECK_MESSAGE(TEXT("Expected rows to be sorted."), Rows.IsSorted());
		CHECK_EQUALS(TEXT("Size"), Rows.Num(), 6);
		CHECK_EQUALS(TEXT("Rows[0]"), Rows[0], 1llu);
		CHECK_EQUALS(TEXT("Rows[1]"), Rows[1], 1llu);
		CHECK_EQUALS(TEXT("Rows[2]"), Rows[2], 2llu);
		CHECK_EQUALS(TEXT("Rows[3]"), Rows[3], 2llu);
		CHECK_EQUALS(TEXT("Rows[4]"), Rows[4], 3llu);
		CHECK_EQUALS(TEXT("Rows[5]"), Rows[5], 3llu);
	}

	SECTION("Uniquely merge two nodes")
	{
		FRowHandleArray Values0;
		FRowHandleArray Values1;
		Values0.Append({ 1, 2, 3 }, FRowHandleArray::EFlags::IsSorted | FRowHandleArray::EFlags::IsUnique);
		Values1.Append({ 1, 2, 3 }, FRowHandleArray::EFlags::IsSorted | FRowHandleArray::EFlags::IsUnique);

		TSharedPtr<IRowNode> View0 = MakeShared<FRowViewNode>(Values0);
		TSharedPtr<IRowNode> View1 = MakeShared<FRowViewNode>(Values1);

		FRowMergeNode Node({ View0, View1 }, FRowMergeNode::EMergeApproach::Unique);

		FRowHandleArrayView Rows = Node.GetRows();
		CHECK_MESSAGE(TEXT("Expected rows to be sorted."), Rows.IsSorted());
		CHECK_EQUALS(TEXT("Size"), Rows.Num(), 3);
		CHECK_EQUALS(TEXT("Rows[0]"), Rows[0], 1llu);
		CHECK_EQUALS(TEXT("Rows[1]"), Rows[1], 2llu);
		CHECK_EQUALS(TEXT("Rows[2]"), Rows[2], 3llu);
	}

	SECTION("Repeating of data in two nodes")
	{
		FRowHandleArray Values0;
		FRowHandleArray Values1;
		Values0.Append({ 1, 2, 3 }, FRowHandleArray::EFlags::IsSorted | FRowHandleArray::EFlags::IsUnique);
		Values1.Append({ 2, 3, 4 }, FRowHandleArray::EFlags::IsSorted | FRowHandleArray::EFlags::IsUnique);

		TSharedPtr<IRowNode> View0 = MakeShared<FRowViewNode>(Values0);
		TSharedPtr<IRowNode> View1 = MakeShared<FRowViewNode>(Values1);

		FRowMergeNode Node({ View0, View1 }, FRowMergeNode::EMergeApproach::Repeating);

		FRowHandleArrayView Rows = Node.GetRows();
		CHECK_MESSAGE(TEXT("Expected rows to be sorted."), Rows.IsSorted());
		CHECK_EQUALS(TEXT("Size"), Rows.Num(), 2);
		CHECK_EQUALS(TEXT("Rows[0]"), Rows[0], 2llu);
		CHECK_EQUALS(TEXT("Rows[1]"), Rows[1], 3llu);
	}

	SECTION("Repeating of data in three nodes")
	{
		FRowHandleArray Values0;
		FRowHandleArray Values1;
		FRowHandleArray Values2;
		Values0.Append({ 1, 2, 3 }, FRowHandleArray::EFlags::IsSorted | FRowHandleArray::EFlags::IsUnique);
		Values1.Append({ 1, 2, 3 }, FRowHandleArray::EFlags::IsSorted | FRowHandleArray::EFlags::IsUnique);
		Values2.Append({ 1, 3 }, FRowHandleArray::EFlags::IsSorted | FRowHandleArray::EFlags::IsUnique);

		TSharedPtr<IRowNode> View0 = MakeShared<FRowViewNode>(Values0);
		TSharedPtr<IRowNode> View1 = MakeShared<FRowViewNode>(Values1);
		TSharedPtr<IRowNode> View2 = MakeShared<FRowViewNode>(Values2);

		FRowMergeNode Node({ View0, View1, View2 }, FRowMergeNode::EMergeApproach::Repeating);

		FRowHandleArrayView Rows = Node.GetRows();
		CHECK_MESSAGE(TEXT("Expected rows to be sorted."), Rows.IsSorted());
		CHECK_EQUALS(TEXT("Size"), Rows.Num(), 3);
		CHECK_EQUALS(TEXT("Rows[0]"), Rows[0], 1llu);
		CHECK_EQUALS(TEXT("Rows[1]"), Rows[1], 2llu);
		CHECK_EQUALS(TEXT("Rows[2]"), Rows[2], 3llu);
	}
}

#endif // #if WITH_TESTS
