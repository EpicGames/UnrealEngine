// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_TESTS

#include "Algo/StableSort.h"
#include "DataStorage/Features.h"
#include "Elements/Framework/TypedElementSorter.h"
#include "Elements/Interfaces/TypedElementDataStorageInterface.h"
#include "Math/UnrealMathUtility.h"
#include "Sorting/SortTasks.h"
#include "TedsRowArrayNode.h"
#include "TedsRowSortNode.h"
#include "TedsQueryStackExecutor.h"
#include "Tests/TestHarnessAdapter.h"
#include "Templates/SharedPointer.h"

namespace UE::Editor::DataStorage::Testing
{
	class FSimpleColumnSorter : public FColumnSorterInterface
	{
	public:
		virtual ESortType GetSortType() const override
		{
			return ESortType::HybridSort;
		}

		virtual FText GetShortName() const override
		{
			return FText::GetEmpty();
		}

		virtual int32 Compare(const ICoreProvider& Storage, RowHandle Left, RowHandle Right) const override
		{
			return Left < Right ? -1 : (Left > Right ? 1 : 0);

		}

		virtual FPrefixInfo CalculatePrefix(const ICoreProvider& Storage, RowHandle Row, uint32 ByteIndex) const override
		{
			return FPrefixInfo
			{
				.Prefix = Row,
				.bHasRemainingBytes = false
			};
		}
	};

	class FPrefixExpandedColumnSorter : public FSimpleColumnSorter
	{
	public:
		virtual FPrefixInfo CalculatePrefix(const ICoreProvider& Storage, RowHandle Row, uint32 ByteIndex) const override
		{
			return FPrefixInfo
			{
				.Prefix = static_cast<uint8>(Row >> (56 - ByteIndex)),
				.bHasRemainingBytes = (ByteIndex < 56)
			};
		}
	};

	class FPrefixVariableLengthColumnSorter : public FSimpleColumnSorter
	{
	public:
		virtual FPrefixInfo CalculatePrefix(const ICoreProvider& Storage, RowHandle Row, uint32 ByteIndex) const override
		{
			bool bHasRemainingBytes = ByteIndex < static_cast<uint8>(Row & 255);
			return FPrefixInfo
			{
				.Prefix = bHasRemainingBytes ? (Row >> 8) : 0,
				.bHasRemainingBytes = bHasRemainingBytes
			};
		}
	};

	class FStringCompareColumnSorter : public FSimpleColumnSorter
	{
	public:
		TArray<FString> Strings;

		virtual int32 Compare(const ICoreProvider& Storage, RowHandle Left, RowHandle Right) const override
		{
			return Strings[static_cast<int32>(Left)].Compare(Strings[static_cast<int32>(Right)]);
		}

		virtual FPrefixInfo CalculatePrefix(const ICoreProvider& Storage, RowHandle Row, uint32 ByteIndex) const override
		{
			return CreateSortPrefix(ByteIndex, TSortStringView(FSortCaseInsensitive{}, Strings[static_cast<int32>(Row)]));
		}
	};

	template<typename SortByType>
	class FNameCompareColumnSorter : public FSimpleColumnSorter
	{
	public:
		TArray<FName> Names;

		virtual int32 Compare(const ICoreProvider& Storage, RowHandle Left, RowHandle Right) const override
		{
			return TSortNameView(SortByType{}, Names[static_cast<int32>(Left)]).Compare(Names[static_cast<int32>(Right)]);
		}

		virtual FPrefixInfo CalculatePrefix(const ICoreProvider& Storage, RowHandle Row, uint32 ByteIndex) const override
		{
			return CreateSortPrefix(ByteIndex, TSortNameView(SortByType{}, Names[static_cast<int32>(Row)]));
		}
	};

	void RunTasks(UE::Editor::DataStorage::QueryStack::Sorting::FSortTasks& Tasks, bool bIterative)
	{
		// Normally this function should return in milliseconds, but if there are bugs it could lead to tasks with infinite
		// regression. To prevent users and/or the build machine from being stuck forever or until some large time out, this
		// test fails if it hasn't been completed in 30 seconds.
		if (bIterative)
		{
			uint64 StartTime = FPlatformTime::Cycles64();
			while (FTimespan(FPlatformTime::Cycles64() - StartTime) < FTimespan::FromSeconds(30) && Tasks.HasRemainingTasks())
			{
				Tasks.Update(FTimespan::FromMilliseconds(10));
			}
		}
		else
		{
			Tasks.Update(FTimespan::FromSeconds(30));
		}
		
		if (Tasks.HasRemainingTasks())
		{
			FString Message = TEXT("Tasks ran, but after 30 seconds there was still remaining work.\n");
			Tasks.PrintRemainingTaskList(Message);
			CHECK_MESSAGE(*Message, false);
		}
		CHECK_MESSAGE(TEXT("Not all tasks completed in time"), !Tasks.HasRemainingTasks());
	}

	void UpdateNode(
		UE::Editor::DataStorage::ICoreProvider& DataStorage, const TSharedPtr<UE::Editor::DataStorage::QueryStack::FRowSortNode>& Node)
	{
		using namespace UE::Editor::DataStorage::QueryStack;
		FExplicitCooperativeExecutor Executor("RowSortNodeTestExecutor", Node);
		
		uint64 StartTime = FPlatformTime::Cycles64();
		do
		{
			Executor.Update(FTimespan::FromMilliseconds(10));
		} while (FTimespan(FPlatformTime::Cycles64() - StartTime) < FTimespan::FromSeconds(5) && Node->IsSorting());
		CHECK_MESSAGE(TEXT("Sorting didn't complete in time"), !Node->IsSorting());
	}

	bool CompareRows(TConstArrayView<RowHandle> Source, TConstArrayView<RowHandle> Compare)
	{
		int32 Count = Source.Num();
		if (Count == Compare.Num())
		{
			const RowHandle* SourceIt = Source.GetData();
			const RowHandle* CompareIt = Compare.GetData();
			for (int32 Index = 0; Index < Count; ++Index)
			{
				if (*SourceIt++ != *CompareIt++)
				{
					return false;
				}
			}
			return true;
		}
		return false;
	}
}

TEST_CASE_NAMED(TEDS_QueryStack_RowSortNode_SortTasks_Tests, "Editor::DataStorage::QueryStack::FRowSortNode::FSortTasks", "[ApplicationContextMask][EngineFilter]")
{
	using namespace UE::Editor::DataStorage;
	using namespace UE::Editor::DataStorage::Testing;
	using namespace UE::Editor::DataStorage::QueryStack::Sorting;
	
	TSharedPtr<FSimpleColumnSorter> SimpleSorter = MakeShared<FSimpleColumnSorter>();
	
	if (const ICoreProvider* Storage = GetDataStorageFeature<ICoreProvider>(StorageFeatureName))
	{
		SECTION("Radix sort")
		{
			TArray<RowHandle> Array = { 1, 3, 5, 2, 4, 6 };
			
			TArray<FPrefix> Prefixes;
			Prefixes.AddUninitialized(Array.Num());
			
			FSortTasks Tasks;
			Tasks.CollectPrefixes(Prefixes, Array, *Storage, SimpleSorter, 6);
			Tasks.AddBarrier();
			Tasks.RadixSortPrefixes(Prefixes);
			Tasks.AddBarrier();
			Tasks.CopyPrefixesToRowHandleArray(Array, Prefixes);
			RunTasks(Tasks, false);

			CHECK_MESSAGE(TEXT("Sorted array didn't match expectation."), CompareRows(Array, { 1, 2, 3, 4, 5, 6 }));
		}

		SECTION("Comparative sort")
		{
			TSharedPtr<FStringCompareColumnSorter> StringSorter = MakeShared<FStringCompareColumnSorter>();
			StringSorter->Strings.Emplace(TEXT("AAB"));
			StringSorter->Strings.Emplace(TEXT("ABB"));
			StringSorter->Strings.Emplace(TEXT("CBBEE"));
			StringSorter->Strings.Emplace(TEXT("D"));
			StringSorter->Strings.Emplace(TEXT("BDE"));
			StringSorter->Strings.Emplace(TEXT("ABBA"));
			TArray<RowHandle> Array = { 0, 1, 2, 3, 4, 5 };

			FSortTasks Tasks;
			Tasks.ComparativeSortRowHandleRanges(Array, *Storage, StringSorter, 6);
			RunTasks(Tasks, false);

			CHECK_MESSAGE(TEXT("Sorted array didn't match expectation."), CompareRows(Array, { 0, 1, 5, 4, 2, 3 }));
		}

		SECTION("Comparative sort with empty strings")
		{
			TSharedPtr<FStringCompareColumnSorter> StringSorter = MakeShared<FStringCompareColumnSorter>();

			/* 00 */ StringSorter->Strings.Emplace(TEXT(""));
			/* 01 */ StringSorter->Strings.Emplace(TEXT(""));
			/* 02 */ StringSorter->Strings.Emplace(TEXT("Toolbar"));
			/* 03 */ StringSorter->Strings.Emplace(TEXT(""));
			/* 04 */ StringSorter->Strings.Emplace(TEXT(""));
			/* 05 */ StringSorter->Strings.Emplace(TEXT("Text"));
			/* 06 */ StringSorter->Strings.Emplace(TEXT(""));
			/* 07 */ StringSorter->Strings.Emplace(TEXT("Icon"));
			/* 08 */ StringSorter->Strings.Emplace(TEXT(""));
			/* 09 */ StringSorter->Strings.Emplace(TEXT(""));
			/* 10 */ StringSorter->Strings.Emplace(TEXT("RowHandle"));
			/* 11 */ StringSorter->Strings.Emplace(TEXT(""));
			/* 12 */ StringSorter->Strings.Emplace(TEXT("Large"));
			/* 13 */ StringSorter->Strings.Emplace(TEXT("Default"));
			/* 14 */ StringSorter->Strings.Emplace(TEXT(""));
			/* 15 */ StringSorter->Strings.Emplace(TEXT("Large"));
			/* 16 */ StringSorter->Strings.Emplace(TEXT(""));
			/* 17 */ StringSorter->Strings.Emplace(TEXT(""));

			TArray<RowHandle> Array = { 0, 1, 2, 3, 4, 5, 6, 7, 8, 9, 10, 11, 12, 13, 14, 15, 16, 17 };

			FSortTasks Tasks;
			Tasks.ComparativeSortRowHandleRanges(Array, *Storage, StringSorter, 2500);
			RunTasks(Tasks, false);

			CHECK_MESSAGE(TEXT("Sorted array didn't match expectation."), 
				CompareRows(Array, { 0, 1, 3, 4, 6, 8, 9, 11, 14, 16, 17, 13, 7, 12, 15, 10, 5, 2 }));
		}

		SECTION("Comparative sort with empty names")
		{
			TSharedPtr<FNameCompareColumnSorter<TSortByName<>>> NameSorter = MakeShared<FNameCompareColumnSorter<TSortByName<>>>();

			/* 00 */ NameSorter->Names.Emplace(TEXT(""));
			/* 01 */ NameSorter->Names.Emplace(TEXT(""));
			/* 02 */ NameSorter->Names.Emplace(TEXT("Toolbar"));
			/* 03 */ NameSorter->Names.Emplace(TEXT(""));
			/* 04 */ NameSorter->Names.Emplace(TEXT(""));
			/* 05 */ NameSorter->Names.Emplace(TEXT("Text"));
			/* 06 */ NameSorter->Names.Emplace(TEXT(""));
			/* 07 */ NameSorter->Names.Emplace(TEXT("Icon"));
			/* 08 */ NameSorter->Names.Emplace(TEXT(""));
			/* 09 */ NameSorter->Names.Emplace(TEXT(""));
			/* 10 */ NameSorter->Names.Emplace(TEXT("RowHandle"));
			/* 11 */ NameSorter->Names.Emplace(TEXT(""));
			/* 12 */ NameSorter->Names.Emplace(TEXT("Large"));
			/* 13 */ NameSorter->Names.Emplace(TEXT("Default"));
			/* 14 */ NameSorter->Names.Emplace(TEXT(""));
			/* 15 */ NameSorter->Names.Emplace(TEXT("Large"));
			/* 16 */ NameSorter->Names.Emplace(TEXT(""));
			/* 17 */ NameSorter->Names.Emplace(TEXT(""));

			TArray<RowHandle> Array = { 0, 1, 2, 3, 4, 5, 6, 7, 8, 9, 10, 11, 12, 13, 14, 15, 16, 17 };

			FSortTasks Tasks;
			Tasks.ComparativeSortRowHandleRanges(Array, *Storage, NameSorter, 2500);
			RunTasks(Tasks, false);

			CHECK_MESSAGE(TEXT("Sorted array didn't match expectation."),
				CompareRows(Array, { 0, 1, 3, 4, 6, 8, 9, 11, 14, 16, 17, 13, 7, 12, 15, 10, 5, 2 }));
		}

		SECTION("Comparative sort with empty names with none")
		{
			TSharedPtr<FNameCompareColumnSorter<TSortByName<ESortByNameFlags::WithNone>>> NameSorter =
				MakeShared<FNameCompareColumnSorter<TSortByName<ESortByNameFlags::WithNone>>>();

			/* 00 */ NameSorter->Names.Emplace(TEXT(""));
			/* 01 */ NameSorter->Names.Emplace(TEXT(""));
			/* 02 */ NameSorter->Names.Emplace(TEXT("Toolbar"));
			/* 03 */ NameSorter->Names.Emplace(TEXT(""));
			/* 04 */ NameSorter->Names.Emplace(TEXT(""));
			/* 05 */ NameSorter->Names.Emplace(TEXT("Text"));
			/* 06 */ NameSorter->Names.Emplace(TEXT(""));
			/* 07 */ NameSorter->Names.Emplace(TEXT("Icon"));
			/* 08 */ NameSorter->Names.Emplace(TEXT(""));
			/* 09 */ NameSorter->Names.Emplace(TEXT(""));
			/* 10 */ NameSorter->Names.Emplace(TEXT("RowHandle"));
			/* 11 */ NameSorter->Names.Emplace(TEXT(""));
			/* 12 */ NameSorter->Names.Emplace(TEXT("Large"));
			/* 13 */ NameSorter->Names.Emplace(TEXT("Default"));
			/* 14 */ NameSorter->Names.Emplace(TEXT(""));
			/* 15 */ NameSorter->Names.Emplace(TEXT("Large"));
			/* 16 */ NameSorter->Names.Emplace(TEXT(""));
			/* 17 */ NameSorter->Names.Emplace(TEXT(""));

			TArray<RowHandle> Array = { 0, 1, 2, 3, 4, 5, 6, 7, 8, 9, 10, 11, 12, 13, 14, 15, 16, 17 };

			FSortTasks Tasks;
			Tasks.ComparativeSortRowHandleRanges(Array, *Storage, NameSorter, 2500);
			RunTasks(Tasks, false);

			CHECK_MESSAGE(TEXT("Sorted array didn't match expectation."),
				CompareRows(Array, { 13, 7, 12, 15, 0, 1, 3, 4, 6, 8, 9, 11, 14, 16, 17, 10, 5, 2 }));
		}

		SECTION("Merge sort - Equal range")
		{
			TArray<RowHandle> Array = { 1, 3, 5, 2, 4, 6 };
			TArray<RowHandle> Scratch;
			Scratch.AddUninitialized(6);

			FSortTasks Tasks;
			Tasks.MergeSortedAdjacentRanges(Array, Scratch, *Storage, SimpleSorter, 3);
			RunTasks(Tasks, false);

			CHECK_MESSAGE(TEXT("Sorted array didn't match expectation."), CompareRows(Array, { 1, 2, 3, 4, 5, 6 }));
		}

		SECTION("Merge sort - Right fits in single left slot")
		{
			TArray<RowHandle> Array = { 1, 7, 9, 2, 4, 6 };
			TArray<RowHandle> Scratch;
			Scratch.AddUninitialized(6);

			FSortTasks Tasks;
			Tasks.MergeSortedAdjacentRanges(Array, Scratch, *Storage, SimpleSorter, 3);
			RunTasks(Tasks, false);

			CHECK_MESSAGE(TEXT("Sorted array didn't match expectation."), CompareRows(Array, { 1, 2, 4, 6, 7, 9 }));
		}

		SECTION("Merge sort - Uneven split")
		{
			TArray<RowHandle> Array = { 1, 3, 5, 7, 2, 4};
			TArray<RowHandle> Scratch;
			Scratch.AddUninitialized(6);

			FSortTasks Tasks;
			Tasks.MergeSortedAdjacentRanges(Array, Scratch, *Storage, SimpleSorter, 4);
			RunTasks(Tasks, false);

			CHECK_MESSAGE(TEXT("Sorted array didn't match expectation."), CompareRows(Array, { 1, 2, 3, 4, 5, 7 }));
		}

		SECTION("Bucketize Range - Single prefix")
		{
			TArray<RowHandle> Array = 
			{ 
				0x0101, 
				0x0203, 
				0x0105, 
				0x0107, 
				0x0202, 
				0x0104
			};
			TArray<FPrefix> Prefixes;
			TArray<FPrefix> PrefixesShadow;
			Prefixes.AddUninitialized(Array.Num());
			PrefixesShadow.AddUninitialized(Array.Num());

			FSortTasks Tasks;
			Tasks.CollectPrefixes(Prefixes, Array, *Storage, SimpleSorter, 6);
			Tasks.AddBarrier();
			Tasks.BucketizeRange(Array, Prefixes, PrefixesShadow, SimpleSorter, *Storage, 6, 0);
			RunTasks(Tasks, true);

			CHECK_MESSAGE(TEXT("Sorted array didn't match expectation."), CompareRows(Array, 
				{ 
					0x0101,
					0x0104,
					0x0105,
					0x0107,
					0x0202,
					0x0203
				}));
		}

		SECTION("Bucketize Range - Multiple prefixes")
		{
			TArray<RowHandle> Array =
			{
				0x0101,
				0x0203,
				0x0105,
				0x0107,
				0x0202,
				0x0104
			};
			TArray<FPrefix> Prefixes;
			TArray<FPrefix> PrefixesShadow;
			Prefixes.AddUninitialized(Array.Num());
			PrefixesShadow.AddUninitialized(Array.Num());

			TSharedPtr<FPrefixExpandedColumnSorter> PrefixExpandedSorter = MakeShared<FPrefixExpandedColumnSorter>();
			FSortTasks Tasks;
			Tasks.CollectPrefixes(Prefixes, Array, *Storage, PrefixExpandedSorter, 6);
			Tasks.AddBarrier();
			Tasks.BucketizeRange(Array, Prefixes, PrefixesShadow, PrefixExpandedSorter, *Storage, 6, 0);
			RunTasks(Tasks, true);

			CHECK_MESSAGE(TEXT("Sorted array didn't match expectation."), CompareRows(Array,
				{
					0x0101,
					0x0104,
					0x0105,
					0x0107,
					0x0202,
					0x0203
				}));
		}

		SECTION("Bucketize Range - Variable prefix length")
		{
			TArray<RowHandle> Array =
			{
				0x1108,
				0x2218,
				0x1118,
				0x1128,
				0x2210,
				0x1120
			};
			TArray<FPrefix> Prefixes;
			TArray<FPrefix> PrefixesShadow;
			Prefixes.AddUninitialized(Array.Num());
			PrefixesShadow.AddUninitialized(Array.Num());

			TSharedPtr<FPrefixVariableLengthColumnSorter> PrefixExpandedSorter = MakeShared<FPrefixVariableLengthColumnSorter>();
			FSortTasks Tasks;
			Tasks.CollectPrefixes(Prefixes, Array, *Storage, PrefixExpandedSorter, 6);
			Tasks.AddBarrier();
			Tasks.BucketizeRange(Array, Prefixes, PrefixesShadow, PrefixExpandedSorter, *Storage, 6, 0);
			RunTasks(Tasks, true);

			CHECK_MESSAGE(TEXT("Sorted array didn't match expectation."), CompareRows(Array,
				{
					0x1108,
					0x1118,
					0x1120,
					0x1128,
					0x2210,
					0x2218
				}));
		}

		auto BucketizeRangeLargeNumber = [this, Storage](bool bWithComparisons)
			{
				constexpr int32 Count = 1000;

				TArray<RowHandle> Array;
				Array.Reserve(Count);

				for (int32 Random = 0; Random < Count; ++Random)
				{
					Array.Add(FMath::Rand32());
				}

				TArray<RowHandle> Compare = Array;
				Algo::StableSort(Compare);

				TArray<FPrefix> Prefixes;
				TArray<FPrefix> PrefixesShadow;
				Prefixes.AddUninitialized(Array.Num());
				PrefixesShadow.AddUninitialized(Array.Num());

				TSharedPtr<FPrefixExpandedColumnSorter> PrefixExpandedSorter = MakeShared<FPrefixExpandedColumnSorter>();
				FSortTasks Tasks;
				Tasks.CollectPrefixes(Prefixes, Array, *Storage, PrefixExpandedSorter, 50);
				Tasks.AddBarrier();
				Tasks.BucketizeRange(Array, Prefixes, PrefixesShadow, PrefixExpandedSorter, *Storage, 50, bWithComparisons ? 50 : 0);
				RunTasks(Tasks, true);

				for (int32 Index = 0; Index < Count; ++Index)
				{
					if (Array[Index] != Compare[Index])
					{
						CHECK_MESSAGE(
							*FString::Printf(TEXT("Sorted array didn't match expectation. First conflict at index %i."), Index), 
							false);
						break;
					}
				}
			};

		SECTION("Bucketize Range - Large number, no comparisons")
		{
			BucketizeRangeLargeNumber(false);
		}

		SECTION("Bucketize Range - Large number, with comparisons")
		{
			BucketizeRangeLargeNumber(true);
		}
	}
}

TEST_CASE_NAMED(TEDS_QueryStack_RowSortNode_SortNode_Integration_Tests, "Editor::DataStorage::QueryStack::FRowSortNode::Integration", "[ApplicationContextMask][EngineFilter]")
{
	using namespace UE::Editor::DataStorage;
	using namespace UE::Editor::DataStorage::Testing;
	using namespace UE::Editor::DataStorage::QueryStack;
	using namespace UE::Editor::DataStorage::QueryStack::Sorting;

	if (ICoreProvider* Storage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName))
	{
		SECTION("Full sort single strings")
		{
			TSharedPtr<FStringCompareColumnSorter> StringSorter = MakeShared<FStringCompareColumnSorter>();

			/* 00 */ StringSorter->Strings.Emplace(TEXT(""));
			/* 01 */ StringSorter->Strings.Emplace(TEXT(""));
			/* 02 */ StringSorter->Strings.Emplace(TEXT("Toolbar"));
			/* 03 */ StringSorter->Strings.Emplace(TEXT(""));
			/* 04 */ StringSorter->Strings.Emplace(TEXT(""));
			/* 05 */ StringSorter->Strings.Emplace(TEXT("Text"));
			/* 06 */ StringSorter->Strings.Emplace(TEXT(""));
			/* 07 */ StringSorter->Strings.Emplace(TEXT("Icon"));
			/* 08 */ StringSorter->Strings.Emplace(TEXT(""));
			/* 09 */ StringSorter->Strings.Emplace(TEXT(""));
			/* 10 */ StringSorter->Strings.Emplace(TEXT("RowHandle"));
			/* 11 */ StringSorter->Strings.Emplace(TEXT(""));
			/* 12 */ StringSorter->Strings.Emplace(TEXT("Large"));
			/* 13 */ StringSorter->Strings.Emplace(TEXT("Default"));
			/* 14 */ StringSorter->Strings.Emplace(TEXT(""));
			/* 15 */ StringSorter->Strings.Emplace(TEXT("Large"));
			/* 16 */ StringSorter->Strings.Emplace(TEXT(""));
			/* 17 */ StringSorter->Strings.Emplace(TEXT(""));

			FRowHandleArray Array(FRowHandleArrayView(
				{ 0, 1, 2, 3, 4, 5, 6, 7, 8, 9, 10, 11, 12, 13, 14, 15, 16, 17 },
				FRowHandleArrayView::EFlags::IsUnique));
			
			TSharedPtr<FRowArrayNode> ArrayNode = MakeShared<FRowArrayNode>(Array);
			TSharedPtr<FRowSortNode> SortNode = MakeShared<FRowSortNode>(*Storage, ArrayNode, StringSorter);

			UpdateNode(*Storage, SortNode);
			
			CHECK_MESSAGE(TEXT("Sorted array didn't match expectation."),
				CompareRows(SortNode->GetRows(), {0, 1, 3, 4, 6, 8, 9, 11, 14, 16, 17, 13, 7, 12, 15, 10, 5, 2}));
		}

		SECTION("Full sort single names")
		{
			TSharedPtr<FNameCompareColumnSorter<TSortByName<>>> NameSorter = MakeShared<FNameCompareColumnSorter<TSortByName<>>>();

			/* 00 */ NameSorter->Names.Emplace(TEXT(""));
			/* 01 */ NameSorter->Names.Emplace(TEXT(""));
			/* 02 */ NameSorter->Names.Emplace(TEXT("Toolbar"));
			/* 03 */ NameSorter->Names.Emplace(TEXT(""));
			/* 04 */ NameSorter->Names.Emplace(TEXT(""));
			/* 05 */ NameSorter->Names.Emplace(TEXT("Text"));
			/* 06 */ NameSorter->Names.Emplace(TEXT(""));
			/* 07 */ NameSorter->Names.Emplace(TEXT("Icon"));
			/* 08 */ NameSorter->Names.Emplace(TEXT(""));
			/* 09 */ NameSorter->Names.Emplace(TEXT(""));
			/* 10 */ NameSorter->Names.Emplace(TEXT("RowHandle"));
			/* 11 */ NameSorter->Names.Emplace(TEXT(""));
			/* 12 */ NameSorter->Names.Emplace(TEXT("Large"));
			/* 13 */ NameSorter->Names.Emplace(TEXT("Default"));
			/* 14 */ NameSorter->Names.Emplace(TEXT(""));
			/* 15 */ NameSorter->Names.Emplace(TEXT("Large"));
			/* 16 */ NameSorter->Names.Emplace(TEXT(""));
			/* 17 */ NameSorter->Names.Emplace(TEXT(""));

			FRowHandleArray Array(FRowHandleArrayView(
				{ 0, 1, 2, 3, 4, 5, 6, 7, 8, 9, 10, 11, 12, 13, 14, 15, 16, 17 },
				FRowHandleArrayView::EFlags::IsUnique));

			TSharedPtr<FRowArrayNode> ArrayNode = MakeShared<FRowArrayNode>(Array);
			TSharedPtr<FRowSortNode> SortNode = MakeShared<FRowSortNode>(*Storage, ArrayNode, NameSorter);

			UpdateNode(*Storage, SortNode);
			
			CHECK_MESSAGE(TEXT("Sorted array didn't match expectation."),
				CompareRows(SortNode->GetRows(), { 0, 1, 3, 4, 6, 8, 9, 11, 14, 16, 17, 13, 7, 12, 15, 10, 5, 2 }));
		}

		SECTION("Full sort single names with none")
		{
			TSharedPtr<FNameCompareColumnSorter<TSortByName<ESortByNameFlags::WithNone>>> NameSorter =
				MakeShared<FNameCompareColumnSorter<TSortByName<ESortByNameFlags::WithNone>>>();

			/* 00 */ NameSorter->Names.Emplace(TEXT(""));
			/* 01 */ NameSorter->Names.Emplace(TEXT(""));
			/* 02 */ NameSorter->Names.Emplace(TEXT("Toolbar"));
			/* 03 */ NameSorter->Names.Emplace(TEXT(""));
			/* 04 */ NameSorter->Names.Emplace(TEXT(""));
			/* 05 */ NameSorter->Names.Emplace(TEXT("Text"));
			/* 06 */ NameSorter->Names.Emplace(TEXT(""));
			/* 07 */ NameSorter->Names.Emplace(TEXT("Icon"));
			/* 08 */ NameSorter->Names.Emplace(TEXT(""));
			/* 09 */ NameSorter->Names.Emplace(TEXT(""));
			/* 10 */ NameSorter->Names.Emplace(TEXT("RowHandle"));
			/* 11 */ NameSorter->Names.Emplace(TEXT(""));
			/* 12 */ NameSorter->Names.Emplace(TEXT("Large"));
			/* 13 */ NameSorter->Names.Emplace(TEXT("Default"));
			/* 14 */ NameSorter->Names.Emplace(TEXT(""));
			/* 15 */ NameSorter->Names.Emplace(TEXT("Large"));
			/* 16 */ NameSorter->Names.Emplace(TEXT(""));
			/* 17 */ NameSorter->Names.Emplace(TEXT(""));

			FRowHandleArray Array(FRowHandleArrayView(
				{ 0, 1, 2, 3, 4, 5, 6, 7, 8, 9, 10, 11, 12, 13, 14, 15, 16, 17 },
				FRowHandleArrayView::EFlags::IsUnique));

			TSharedPtr<FRowArrayNode> ArrayNode = MakeShared<FRowArrayNode>(Array);
			TSharedPtr<FRowSortNode> SortNode = MakeShared<FRowSortNode>(*Storage, ArrayNode, NameSorter);

			UpdateNode(*Storage, SortNode);

			CHECK_MESSAGE(TEXT("Sorted array didn't match expectation."),
				CompareRows(SortNode->GetRows(), { 13, 7, 12, 15, 0, 1, 3, 4, 6, 8, 9, 11, 14, 16, 17, 10, 5, 2 }));
		}
	}
}

#endif // #if WITH_TESTS
