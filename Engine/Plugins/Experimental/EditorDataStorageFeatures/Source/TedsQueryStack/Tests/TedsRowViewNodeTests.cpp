// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_TESTS

#include "TedsRowViewNode.h"
#include "Tests/TestHarnessAdapter.h"

TEST_CASE_NAMED(TEDS_QueryStack_RowViewNode_Tests, "Editor::DataStorage::QueryStack::FRowViewNode", "[ApplicationContextMask][EngineFilter]")
{
	using namespace UE::Editor::DataStorage;
	using namespace UE::Editor::DataStorage::QueryStack;

	SECTION("View with a few row handles")
	{
		FRowHandleArray RowHandles;
		RowHandles.Append({ 1, 2, 3 }, FRowHandleArray::EFlags::IsSorted | FRowHandleArray::EFlags::IsUnique);
		FRowViewNode View(RowHandles);
		CHECK_EQUALS(TEXT("Revision"), View.GetRevision(), 1);
		
		FRowHandleArrayView Rows = View.GetRows();
		CHECK_MESSAGE(TEXT("Expected rows to be sorted."), Rows.IsSorted());
		CHECK_EQUALS(TEXT("Size"), Rows.Num(), 3);
		CHECK_EQUALS(TEXT("Rows[0]"), Rows[0], 1llu);
		CHECK_EQUALS(TEXT("Rows[1]"), Rows[1], 2llu);
		CHECK_EQUALS(TEXT("Rows[2]"), Rows[2], 3llu);
	} 

	SECTION("Mark dirty")
	{
		FRowHandleArray RowHandles;
		FRowViewNode View(RowHandles);
		CHECK_EQUALS(TEXT("Revision"), View.GetRevision(), 1);
		
		View.MarkDirty();
		CHECK_EQUALS(TEXT("Revision"), View.GetRevision(), 2);
	}
}

#endif // #if WITH_TESTS
