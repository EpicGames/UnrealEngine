// Copyright Epic Games, Inc. All Rights Reserved.

#include "Processors/RevisionControlProcessors.h"

#include "ISourceControlModule.h"
#include "RevisionControlOverlaySettings.h"
#include "SourceControlFileStatusMonitor.h"
#include "GameFramework/Actor.h"
#include "HAL/IConsoleManager.h"

#include "DataStorage/Features.h"
#include "DataStorage/Debug/Log.h"
#include "Elements/Columns/TypedElementCompatibilityColumns.h"
#include "Elements/Columns/TypedElementMiscColumns.h"
#include "Elements/Columns/TedsRevisionControlColumns.h"
#include "Elements/Columns/TypedElementSelectionColumns.h"
#include "Elements/Columns/TypedElementViewportColumns.h"
#include "Elements/Framework/TypedElementQueryBuilder.h"
#include "Elements/Interfaces/TypedElementQueryStorageInterfaces.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RevisionControlProcessors)

namespace UE::Editor::RevisionControl::Private
{
	static bool bAutoPopulateRevisionControlState = false;
	FAutoConsoleVariableRef CVarAutoPopulateState(
		TEXT("TEDS.RevisionControl.AutoPopulateState"),
		bAutoPopulateRevisionControlState,
		TEXT("Automatically query revision control provider and fill information into TEDS")
	);

	static bool gEnableOverlays = false;
	TAutoConsoleVariable<bool> CVarEnableOverlays(
		TEXT("RevisionControl.Overlays.Enable"),
		gEnableOverlays,
		TEXT("Enables overlays."),
		ECVF_Default);


	#if UE_BUILD_SHIPPING
	#define ENABLE_OVERLAY_DEBUG 0
	#else
	#define ENABLE_OVERLAY_DEBUG 1
	#endif

	#if ENABLE_OVERLAY_DEBUG
	static int32 gDefaultDebugForceColorOnAllValue = false; // [0..100]
	TAutoConsoleVariable<int32> CVarDebugForceColorOnAll(
		TEXT("RevisionControl.Overlays.Debug.ForceColorOnAll"),
		gDefaultDebugForceColorOnAllValue,
		TEXT("Debug to force overlay color on everything. 1 = Red, 2 = Green, 3 = Blue, 4 = White. 0 = off  ."),
		ECVF_Default);
	#endif

	using namespace UE::Editor::DataStorage;
	static FColor DetermineOverlayColor(const ICommonQueryContext& SCCContext, const FTypedElementUObjectColumn& Actor, bool bSelected)
	{
		check(IsInGameThread());

		const URevisionControlOverlaySettings* Settings = GetDefault<URevisionControlOverlaySettings>();
		const uint8 Alpha = FMath::Lerp<uint8>(static_cast<uint8>(0), static_cast<uint8>(255), static_cast<float>(Settings->OverlayAlpha) / 100.f);

	#if ENABLE_OVERLAY_DEBUG
		if (int32 Force = CVarDebugForceColorOnAll.GetValueOnGameThread())
		{
			switch(Force)
			{
			case 1:
				return FColor(255, 0, 0, Alpha);
			case 2:
				return FColor(0, 255, 0, Alpha);
			case 3:
				return FColor(0, 0, 255, Alpha);
			case 4:
				return FColor(255, 255, 255, Alpha);
			// Do normal determination for higher than 4
			}
		}
	#endif

		bool bExternal = Actor.Object.IsValid() ? Cast<AActor>(Actor.Object)->IsPackageExternal() : false;
		bool bIgnored = !bExternal;
		if (!bIgnored && !bSelected)
		{
			const FSccStateColumn* SccState = SCCContext.GetColumn<FSccStateColumn>();

			// Check if the package is outdated because there is a newer version available.
			if (SccState && SccState->bIsNotCurrent)
			{
				if (Settings->bShowNotAtHeadRevision)
				{
					// Yellow.
					return FColor(225, 255, 61, Alpha);
				}
			}

			// Check if the package is locked by someone else.
			if (SccState && SccState->bIsExternallyLocked)
			{
				if (Settings->bShowCheckedOutByOtherUser)
				{
					// Red.
					return FColor(239, 53, 53, Alpha);
				}
			}

			// Check if the package is added locally.
			if (SccState && SccState->Modification == ESCCModification::Added)
			{
				if (Settings->bShowOpenForAdd)
				{
					// Blue.
					return FColor(0, 112, 224, Alpha);
				}
			}

			// Check if the package is locked by self.
			if (SccState && SccState->bIsLocked)
			{
				if (Settings->bShowCheckedOut)
				{
					// Green.
					return FColor(31, 228, 75, Alpha);
				}
			}
		}

		return FColor(ForceInitToZero);
	}
} // namespace UE::Editor::RevisionControl::Private

void URevisionControlDataStorageFactory::RegisterTables(UE::Editor::DataStorage::ICoreProvider& DataStorage)
{
	using namespace UE::Editor::DataStorage;
	DataStorage.RegisterTable(
		TTypedElementColumnTypeList<
			FTypedElementPackagePathColumn, FTypedElementPackageLoadedPathColumn,
			FSccStateColumn>(),
		PackageTableName);
}

void URevisionControlDataStorageFactory::RegisterQueries(UE::Editor::DataStorage::ICoreProvider& DataStorage)
{
	using namespace UE::Editor::DataStorage;
	using namespace UE::Editor::RevisionControl::Private;
	
	CVarAutoPopulateState->AsVariable()->OnChangedDelegate().AddLambda(
		[this, &DataStorage](IConsoleVariable* AutoPopulate)
		{
			if (AutoPopulate->GetBool())
			{
				RegisterFetchUpdates(DataStorage);
			}
			else
			{
				DataStorage.UnregisterQuery(MonitorNewPackagesHandle);
				MonitorNewPackagesHandle = InvalidQueryHandle;

				DataStorage.UnregisterQuery(FetchUpdates);
				FetchUpdates = InvalidQueryHandle;
				
				DataStorage.UnregisterQuery(StopFetchUpdates);
				StopFetchUpdates = InvalidQueryHandle;

				DataStorage.UnregisterQuery(RemoveStatusUpdateEndedTagQuery);
				RemoveStatusUpdateEndedTagQuery = InvalidQueryHandle;

				DataStorage.UnregisterQuery(RemoveStatusUpdateRequiredTagQuery);
				RemoveStatusUpdateRequiredTagQuery = InvalidQueryHandle;

				DataStorage.UnregisterQuery(RemoveSccStateDirtyTagQuery);
				RemoveSccStateDirtyTagQuery = InvalidQueryHandle;
			}
		}
	);

	CVarEnableOverlays->AsVariable()->OnChangedDelegate().AddLambda(
		[this, &DataStorage](IConsoleVariable* EnableOverlays)
		{
			if (EnableOverlays->GetBool())
			{
				DataStorage.UnregisterQuery(RemoveOverlays);
				RemoveOverlays = InvalidQueryHandle;

				RegisterApplyOverlays(DataStorage);
			}
			else
			{
				DataStorage.UnregisterQuery(ApplyOverlaysObjectToSCC);
				ApplyOverlaysObjectToSCC = InvalidQueryHandle;

				DataStorage.UnregisterQuery(SelectionAdded);
				SelectionAdded = InvalidQueryHandle;

				DataStorage.UnregisterQuery(SelectionRemoved);
				SelectionRemoved = InvalidQueryHandle;

				DataStorage.UnregisterQuery(PackageReferenceAdded);
				PackageReferenceAdded = InvalidQueryHandle;

				DataStorage.UnregisterQuery(UpdateSCCForActors);
				UpdateSCCForActors = InvalidQueryHandle;
				
				DataStorage.UnregisterQuery(UpdateOverlays);
				UpdateOverlays = InvalidQueryHandle;

				RegisterRemoveOverlays(DataStorage);
			}
		}
	);
	
	if (CVarAutoPopulateState->GetBool())
	{
		RegisterFetchUpdates(DataStorage);
	}

	if (CVarEnableOverlays->GetBool())
	{
		RegisterApplyOverlays(DataStorage);
	}
	else
	{
		RegisterRemoveOverlays(DataStorage);
	}
	
	RegisterGeneralQueries(DataStorage);
}

void URevisionControlDataStorageFactory::UpdateOverlaysForSCCState(UE::Editor::DataStorage::ICoreProvider* DataStorage) const
{
	DataStorage->ActivateQueries(TEXT("UpdateSCCForActors"));
}

void URevisionControlDataStorageFactory::UpdateOverlayColors(UE::Editor::DataStorage::ICoreProvider* DataStorage) const
{
	DataStorage->ActivateQueries(TEXT("UpdateOverlayForActors"));
}

void URevisionControlDataStorageFactory::RegisterFetchUpdates(UE::Editor::DataStorage::ICoreProvider& DataStorage)
{
	using namespace UE::Editor::DataStorage::Queries;
	
	FSourceControlFileStatusMonitor& FileStatusMonitor = ISourceControlModule::Get().GetSourceControlFileStatusMonitor();


	if (MonitorNewPackagesHandle == InvalidQueryHandle)
	{
		MonitorNewPackagesHandle = DataStorage.RegisterQuery(
			Select(
				TEXT("Add FSCCStatusUpdateRequiredTag on new Package reference"),
				FObserver::OnAdd<FTypedElementPackageReference>()
				.SetExecutionMode(EExecutionMode::GameThread),
				[](IQueryContext& Context, RowHandle Row, const FTypedElementPackageReference& PackageReference)
				{
					Context.AddColumns<FSCCStatusUpdateRequiredTag>(Row);
				})
			.Compile());
	}

	if (FetchUpdates == InvalidQueryHandle)
	{
		FetchUpdates = DataStorage.RegisterQuery(
			Select(
				TEXT("Start monitoring source control statuses when status update is required"),
				FObserver::OnAdd<FSCCStatusUpdateRequiredTag>()
					.SetExecutionMode(EExecutionMode::GameThread),
				[this, &FileStatusMonitor, &DataStorage](IQueryContext& Context, const FTypedElementPackageReference& PackageReference)
				{
					static FSourceControlFileStatusMonitor::FOnSourceControlFileStatus EmptyDelegate{};

					if (const FTypedElementPackageFilePathOnDiskColumn* FilePathColumn = DataStorage.GetColumn<FTypedElementPackageFilePathOnDiskColumn>(PackageReference.Row))
					{
						// do not monitor file not on disk, they would stay in the monitoring queue forever...
						// I think there is a use case for adding a file to the FileStatusMonitor before the file
						// is actually saved on disk see UnsavedAssetsTracker, but in our case, we definitely want
						// only files that exists.
						if (FPaths::FileExists(FilePathColumn->FilePathOnDisk))
						{
							FileStatusMonitor.StartMonitoringFile(
								reinterpret_cast<uintptr_t>(this),
								FilePathColumn->FilePathOnDisk,
								EmptyDelegate);
						}
					}
				})
			.Compile());
	}
	
	// When FSCCStatusUpdateEndedTag is added to a package row, the P4 query has completed for that file.
	// Stop monitoring since we have the SCC data we need.
	if (StopFetchUpdates == InvalidQueryHandle)
	{
		StopFetchUpdates = DataStorage.RegisterQuery(
			Select(
				TEXT("Stop monitoring source control statuses when SCC sync has completed"),
				FObserver::OnAdd<FSCCStatusUpdateEndedTag>()
					.SetExecutionMode(EExecutionMode::GameThread),
				[this, &FileStatusMonitor, &DataStorage](IQueryContext& Context, RowHandle PackageRow)
				{
					if (const FTypedElementPackageFilePathOnDiskColumn* FilePathColumn = DataStorage.GetColumn<FTypedElementPackageFilePathOnDiskColumn>(PackageRow))
					{
						FileStatusMonitor.StopMonitoringFile(
							reinterpret_cast<uintptr_t>(this),
							FilePathColumn->FilePathOnDisk
						);
					}
				})
			.Compile());
	}

	if (RemoveStatusUpdateEndedTagQuery == InvalidQueryHandle)
	{
		// remove at start of tick phase.
		RemoveStatusUpdateEndedTagQuery = DataStorage.RegisterQuery(
			Select(
				TEXT("Remove FSCCStatusUpdateEndedTag after observers have processed it"),
				FProcessor(EQueryTickPhase::FrameEnd, DataStorage.GetQueryTickGroupName(EQueryTickGroups::Update))
				.SetExecutionMode(EExecutionMode::GameThread),
				[](IQueryContext& Context, RowHandle Row)
				{
					Context.RemoveColumns<FSCCStatusUpdateEndedTag>(Row);
				})
			.Where()
			.All<FSCCStatusUpdateEndedTag>()
			.Compile());
	}

	if (RemoveStatusUpdateRequiredTagQuery == InvalidQueryHandle)
	{
		RemoveStatusUpdateRequiredTagQuery = DataStorage.RegisterQuery(
			Select(
				TEXT("Remove FSCCStatusUpdateRequiredTag after observers have processed it"),
				FProcessor(EQueryTickPhase::FrameEnd, DataStorage.GetQueryTickGroupName(EQueryTickGroups::PostUpdate))
				.SetExecutionMode(EExecutionMode::GameThread),
				[](IQueryContext& Context, RowHandle Row)
				{
					Context.RemoveColumns<FSCCStatusUpdateRequiredTag>(Row);
				})
			.Where()
			.All<FSCCStatusUpdateRequiredTag>()
			.Compile());
	}

	if (RemoveSccStateDirtyTagQuery == InvalidQueryHandle)
	{
		RemoveSccStateDirtyTagQuery = DataStorage.RegisterQuery(
			Select(
				TEXT("Remove FSccStateDirtyTag after observers have processed it"),
				FProcessor(EQueryTickPhase::FrameEnd, DataStorage.GetQueryTickGroupName(EQueryTickGroups::PostUpdate))
				.SetExecutionMode(EExecutionMode::GameThread),
				[](IQueryContext& Context, RowHandle Row)
				{
					Context.RemoveColumns<FSccStateDirtyTag>(Row);
				})
			.Where()
			.All<FSccStateDirtyTag>()
			.Compile());
	}
}

void URevisionControlDataStorageFactory::RegisterApplyOverlays(UE::Editor::DataStorage::ICoreProvider& DataStorage)
{
	using namespace UE::Editor::DataStorage::Queries;
	using namespace UE::Editor::RevisionControl::Private;
	
	if (ApplyOverlaysObjectToSCC == InvalidQueryHandle)
	{
		ApplyOverlaysObjectToSCC = DataStorage.RegisterQuery(
			Select()
				.ReadOnly<FTypedElementPackagePathColumn>()
				.ReadOnly<FSccStateColumn>(EOptional::Yes)
			.Compile());
	}
	
	if (UpdateSCCForActors == InvalidQueryHandle)
	{
		check(ApplyOverlaysObjectToSCC != InvalidQueryHandle);

		// Query:
		// For all actors having a package reference:
		//	Determine if a color should be applied based on SCC status tags
		//		If so, either add the OverlayColorColumn to the actor row or update its value and mark it dirty to update it
		//		If not, remove the OverlayColorColumn from the actor row
		UpdateSCCForActors = DataStorage.RegisterQuery(
			Select(
				TEXT("Update overlay for all actors"),
				FProcessor(EQueryTickPhase::PrePhysics, DataStorage.GetQueryTickGroupName(EQueryTickGroups::Update))
					.SetExecutionMode(EExecutionMode::GameThread)
					.MakeActivatable(TEXT("UpdateSCCForActors")),
				[](IQueryContext& ActorQueryContext, RowHandle ObjectRow, const FTypedElementUObjectColumn& Actor, const FTypedElementPackageReference& PackageReference)
				{
					// Run a subquery on the SCC row to determine the overlay color
					ActorQueryContext.RunSubquery(0, PackageReference.Row, CreateSubqueryCallbackBinding(
						[&ActorQueryContext, &ObjectRow, &Actor](const ISubqueryContext& SubQueryContext)
						{
							FColor Color = DetermineOverlayColor(SubQueryContext, Actor, ActorQueryContext.HasColumn<FTypedElementSelectionColumn>());

							// If there is no overlay color, simply remove the column
							if (Color.Bits == 0)
							{
								ActorQueryContext.RemoveColumns<FTypedElementViewportOverlayColorColumn>(ObjectRow);
								return;
							}

							// If we already have the color column, only update it if the color has changed
							if(const FTypedElementViewportOverlayColorColumn* OverlayColorColumn = ActorQueryContext.GetColumn<FTypedElementViewportOverlayColorColumn>())
							{
								if (Color != OverlayColorColumn->OverlayColor)
								{
									// Update color and add dirty flag to trigger rendering update
									ActorQueryContext.AddColumn<FTypedElementViewportOverlayColorColumn>(ObjectRow, { .OverlayColor = Color });
									ActorQueryContext.AddColumns<FTypedElementViewportOverlayColorDirtyFlag>(ObjectRow);
								}
							}
							// If we don't have the color column, add it
							else
							{
								ActorQueryContext.AddColumn<FTypedElementViewportOverlayColorColumn>(ObjectRow, { .OverlayColor = Color });
							}
						})
					);
				})
			.ReadOnly<FTypedElementViewportOverlayColorColumn>(EOptional::Yes)
			.Where()
				.All<FTypedElementActorTag>()
			.DependsOn()
				.SubQuery(ApplyOverlaysObjectToSCC)
			.Compile());
	}

	if (SelectionAdded == InvalidQueryHandle)
	{
		SelectionAdded = DataStorage.RegisterQuery(
					Select(
					TEXT("Update Overlay on Selection"),
					FObserver::OnAdd<FTypedElementSelectionColumn>(),
					[this](IQueryContext& Context, RowHandle RowHandle, const FTypedElementSelectionColumn& SelectionColumn)
					{
						// We only care about the level editor's selection set for now. When the selection column is made dynamic
						// we can directly query for it
						if(SelectionColumn.SelectionSet.IsNone())
						{
							// Since we know DetermineOverlayColor() removes the overlay when the row is selected we can directly
							// remove the column here to skip the need to do all the checks in there. But if that logic ever
							// changes we will have to update it here too
							Context.RemoveColumns<FTypedElementViewportOverlayColorColumn>(RowHandle);
						}
					})
					.Where()
						.All<FTypedElementActorTag, FTypedElementViewportOverlayColorColumn, FTypedElementPackageReference>()
					.Compile());
	}

	if (SelectionRemoved == InvalidQueryHandle)
	{
		SelectionRemoved = DataStorage.RegisterQuery(
					Select(
					TEXT("Update Overlay on Deselection"),
					FObserver::OnRemove<FTypedElementSelectionColumn>(),
					[this](IQueryContext& ActorQueryContext, RowHandle ObjectRow, const FTypedElementSelectionColumn& SelectionColumn,
						const FTypedElementPackageReference& PackageReference, const FTypedElementUObjectColumn& Actor)
					{
						// We only care about the level editor's selection set for now. When the selection column is made dynamic
						// we can directly query for it
						if(SelectionColumn.SelectionSet.IsNone())
						{
							// When an item is deselected, add the viewport overlay color column to it if applicable
							ActorQueryContext.RunSubquery(0, PackageReference.Row, CreateSubqueryCallbackBinding(
								[&ActorQueryContext, &ObjectRow, &Actor](const ISubqueryContext& SubQueryContext)
								{
									// We manually set the selection as false because the item is being deslected, but the row still has the selection column
									// when the observer is fired
									bool bSelected = false;
									FColor Color = DetermineOverlayColor(SubQueryContext, Actor, bSelected);
									if (Color.Bits != 0)
									{
										ActorQueryContext.AddColumn<FTypedElementViewportOverlayColorColumn>(ObjectRow, { .OverlayColor = Color });
									}
								})
							);
						}
					})
					.Where()
						.All<FTypedElementActorTag>()
					.DependsOn()
						.SubQuery(ApplyOverlaysObjectToSCC)
					.Compile());
	}

	if (PackageReferenceAdded == InvalidQueryHandle)
	{
		/**
	 * Usually, when a revision control update is requested for an SCC row it adds a new row with FTypedElementPackageUpdateColumn and a reference to
	 * the actor row and the SCC row to update the overlays. However, if the revision control update happens before the actor row and SCC row have a
	 * chance to link to each other via the FTypedElementPackageReference column, FTypedElementPackageUpdateColumn cannot be added.
	 * So we add an observer to track for FTypedElementPackageReference addition to the actor rows and manually execute an overlay update.
	 */
		PackageReferenceAdded = DataStorage.RegisterQuery(
						Select(
						TEXT("Add overlay on package reference added"),
						FObserver::OnAdd<FTypedElementPackageReference>(),
						[this](IQueryContext& ActorQueryContext, RowHandle ObjectRow, const FTypedElementPackageReference& PackageReference,
							const FTypedElementUObjectColumn& Actor)
						{
							ActorQueryContext.RunSubquery(0, PackageReference.Row, CreateSubqueryCallbackBinding(
								[&ActorQueryContext, &ObjectRow, &Actor](const ISubqueryContext& SubQueryContext)
								{
									FColor Color = DetermineOverlayColor(SubQueryContext, Actor, ActorQueryContext.HasColumn<FTypedElementSelectionColumn>());
									if (Color.Bits != 0)
									{
										ActorQueryContext.AddColumn<FTypedElementViewportOverlayColorColumn>(ObjectRow, { .OverlayColor = Color });
									}
								})
							);
						})
						.Where()
							.All<FTypedElementActorTag>()
							.None<FTypedElementViewportOverlayColorColumn>()
						.DependsOn()
							.SubQuery(ApplyOverlaysObjectToSCC)
						.Compile());
	}

	// Query to update the color for all rows that currently have the overlay color column
	if (UpdateOverlays == InvalidQueryHandle)
	{
		UpdateOverlays = DataStorage.RegisterQuery(
			Select(
				TEXT("Update overlay color for actors with overlays"),
				FProcessor(EQueryTickPhase::PrePhysics, DataStorage.GetQueryTickGroupName(EQueryTickGroups::Update))
					.SetExecutionMode(EExecutionMode::GameThread)
					.MakeActivatable(TEXT("UpdateOverlayForActors")),
					[](IQueryContext& ActorQueryContext, RowHandle ObjectRow, const FTypedElementUObjectColumn& Actor,
						const FTypedElementPackageReference& PackageReference, FTypedElementViewportOverlayColorColumn& OverlayColorColumn)
					{
						ActorQueryContext.RunSubquery(0, PackageReference.Row, CreateSubqueryCallbackBinding(
							[&ActorQueryContext, &ObjectRow, &Actor, &OverlayColorColumn](const ISubqueryContext& SubQueryContext)
							{
								FColor Color = DetermineOverlayColor(SubQueryContext, Actor, ActorQueryContext.HasColumn<FTypedElementSelectionColumn>());
							
								if (Color != OverlayColorColumn.OverlayColor)
								{
									// Update color and add dirty flag to trigger rendering update
									OverlayColorColumn.OverlayColor = Color;
									ActorQueryContext.AddColumns<FTypedElementViewportOverlayColorDirtyFlag>(ObjectRow);
								}
							})
						);
					})
			.Where()
				.All<FTypedElementActorTag>()
			.DependsOn()
				.SubQuery(ApplyOverlaysObjectToSCC)
			.Compile());
	}
}

void URevisionControlDataStorageFactory::RegisterRemoveOverlays(UE::Editor::DataStorage::ICoreProvider& DataStorage)
{
	using namespace UE::Editor::DataStorage::Queries;
	
	if (RemoveOverlays == InvalidQueryHandle)
	{
		// Query:
		// For all actors WITH an overlay color column AND having a package reference:
		//   Remove the overlay color column
		//
		// This query is used to clean up the color columns if the overlay feature is disabled dynamically
		RemoveOverlays = DataStorage.RegisterQuery(
			Select(
				TEXT("Remove selection overlay colors"),
				// This is in PrePhysics because the overlay->actor query is in DuringPhysics and contexts don't flush changes between tick groups
				FProcessor(EQueryTickPhase::PrePhysics, DataStorage.GetQueryTickGroupName(EQueryTickGroups::SyncExternalToDataStorage))
					.SetExecutionMode(EExecutionMode::GameThread),
				[](IQueryContext& Context, RowHandle ObjectRow, FTypedElementUObjectColumn& Actor, const FTypedElementViewportOverlayColorColumn& ViewportColor)
				{
					Context.RemoveColumns<FTypedElementViewportOverlayColorColumn>(ObjectRow);
				})
			.Where()
				.All<FTypedElementActorTag>()
			.Compile());
	}
}

void URevisionControlDataStorageFactory::RegisterGeneralQueries(UE::Editor::DataStorage::ICoreProvider& DataStorage)
{
	using namespace UE::Editor::DataStorage::Queries;
	using namespace UE::Editor::RevisionControl::Private;

	
}

