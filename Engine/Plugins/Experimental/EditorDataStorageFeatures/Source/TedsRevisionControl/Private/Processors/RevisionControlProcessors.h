// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataStorage/Handles.h"
#include "Elements/Columns/TypedElementPackageColumns.h"
#include "Elements/Interfaces/TypedElementDataStorageFactory.h"

#include "RevisionControlProcessors.generated.h"

namespace UE::Editor::DataStorage
{
	class ICoreProvider;
} // namepsace UE::Editor::DataStorage

UCLASS()
class URevisionControlDataStorageFactory : public UEditorDataStorageFactory
{
	GENERATED_BODY()

public:
	~URevisionControlDataStorageFactory() override = default;

	void RegisterTables(UE::Editor::DataStorage::ICoreProvider& DataStorage) override;
	void RegisterQueries(UE::Editor::DataStorage::ICoreProvider& DataStorage) override;

	// Update the overlays for all SCC rows that contain the Column specified
	void UpdateOverlaysForSCCState(UE::Editor::DataStorage::ICoreProvider* DataStorage) const;

	// Update the color for all actors that currently have an overlay
	void UpdateOverlayColors(UE::Editor::DataStorage::ICoreProvider* DataStorage) const;


private:
	void RegisterFetchUpdates(UE::Editor::DataStorage::ICoreProvider& DataStorage);
	void RegisterApplyOverlays(UE::Editor::DataStorage::ICoreProvider& DataStorage);
	void RegisterRemoveOverlays(UE::Editor::DataStorage::ICoreProvider& DataStorage);
	void RegisterGeneralQueries(UE::Editor::DataStorage::ICoreProvider& DataStorage);

	UE::Editor::DataStorage::QueryHandle MonitorNewPackagesHandle = UE::Editor::DataStorage::InvalidQueryHandle;
	UE::Editor::DataStorage::QueryHandle FetchUpdates = UE::Editor::DataStorage::InvalidQueryHandle;
	UE::Editor::DataStorage::QueryHandle StopFetchUpdates = UE::Editor::DataStorage::InvalidQueryHandle;
	UE::Editor::DataStorage::QueryHandle ApplyOverlaysObjectToSCC = UE::Editor::DataStorage::InvalidQueryHandle;
	UE::Editor::DataStorage::QueryHandle RemoveOverlays = UE::Editor::DataStorage::InvalidQueryHandle;
	UE::Editor::DataStorage::QueryHandle UpdateSCCForActors = UE::Editor::DataStorage::InvalidQueryHandle;
	UE::Editor::DataStorage::QueryHandle SelectionAdded = UE::Editor::DataStorage::InvalidQueryHandle;
	UE::Editor::DataStorage::QueryHandle SelectionRemoved = UE::Editor::DataStorage::InvalidQueryHandle;
	UE::Editor::DataStorage::QueryHandle PackageReferenceAdded = UE::Editor::DataStorage::InvalidQueryHandle;

	UE::Editor::DataStorage::QueryHandle RemoveStatusUpdateEndedTagQuery = UE::Editor::DataStorage::InvalidQueryHandle;
	UE::Editor::DataStorage::QueryHandle RemoveStatusUpdateRequiredTagQuery = UE::Editor::DataStorage::InvalidQueryHandle;

	UE::Editor::DataStorage::QueryHandle RemoveSccStateDirtyTagQuery = UE::Editor::DataStorage::InvalidQueryHandle;

	// Query to fetch all rows with overlays
	UE::Editor::DataStorage::QueryHandle UpdateOverlays = UE::Editor::DataStorage::InvalidQueryHandle;

};
