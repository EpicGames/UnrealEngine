// Copyright Epic Games, Inc. All Rights Reserved.

#include "Queries/ObjectPackagePathToColumnQueries.h"

#include "DataStorage/Features.h"
#include "Elements/Columns/TypedElementCompatibilityColumns.h"
#include "Elements/Columns/TypedElementPackageColumns.h"
#include "Elements/Columns/TedsRevisionControlColumns.h"
#include "Elements/Framework/TypedElementQueryBuilder.h"
#include "Engine/Blueprint.h"
#include "HAL/IConsoleManager.h"
#include "Misc/PackageName.h"
#include "Misc/Paths.h"
#include "UObject/Package.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ObjectPackagePathToColumnQueries)

namespace UE::Editor::RevisionControl::Private
{
	using namespace UE::Editor::DataStorage;

	static bool bAutoSyncPackageInformation = true;
    FAutoConsoleVariableRef CVarSyncPackageInformation(
    	TEXT("TEDS.RevisionControl.SyncPackageInformation"),
    	bAutoSyncPackageInformation,
    	TEXT("Automatically query package information and fill information into TEDS")
    );

	static void ResolvePackageReference(IQueryContext& Context, UPackage* Package, const UObject* Object, RowHandle Row, RowHandle PackageRow, const FString& InFullFilePathOnDisk)
	{
		FTypedElementPackageReference PackageReference;
		PackageReference.Row = PackageRow;
		Context.AddColumn(Row, MoveTemp(PackageReference));

		FTypedElementPackagePathColumn PathColumn;
		FTypedElementPackageLoadedPathColumn LoadedPathColumn;
		FTypedElementPackageFilePathOnDiskColumn FilePathOnDiskColumn;

		Package->GetPathName(nullptr, PathColumn.Path);
		LoadedPathColumn.LoadedPath = Package->GetLoadedPath();
		FilePathOnDiskColumn.FilePathOnDisk = InFullFilePathOnDisk;

		Context.AddColumn(PackageRow, MoveTemp(PathColumn));
		Context.AddColumn(PackageRow, MoveTemp(LoadedPathColumn));
		Context.AddColumn(PackageRow, MoveTemp(FilePathOnDiskColumn));

		// If the Package is Transient we can skip finding the primary asset as that will loop over all assets in the package to find nothing
		if (Package != GetTransientPackage() && !Package->HasAnyFlags(RF_Transient))
		{
			if (Object->HasAnyFlags(RF_HasExternalPackage))
			{
				if (const UObject* ExternalPackage = Object->GetExternalPackage(); Package == ExternalPackage)
				{
					Context.AddColumns<FTedsPrimaryPackageObjectTag>(Row);
				}
			}
			// If the Row Object is a CDO, we have to compare the CDO of the primary package object instead of the object itself
			else if (Context.HasColumn<FTypedElementClassDefaultObjectTag>())
			{
				const UObject* PackageObjectCDO = nullptr;
				// Check if it was BP generated and get the Default Object of that generated class if so
				if(const UBlueprint* PackageObjectBP = Cast<UBlueprint>(Object))
				{
					if (const TSubclassOf<UObject> PackageObjectBPGeneratedClass = PackageObjectBP->GeneratedClass)
					{
						PackageObjectCDO = PackageObjectBPGeneratedClass->GetDefaultObject();
					}
				}
				else
				{
					PackageObjectCDO = Object->GetClass()->GetDefaultObject();
				}
			
				if (Object == PackageObjectCDO)
				{
					Context.AddColumns<FTedsPrimaryPackageObjectTag>(Row);
				}
			}
		}

		if (Package->IsDirty())
		{
			Context.AddColumns<FTedsPackageDirtyTag>(Row);
		}

		Context.SetParentRow(Row, PackageRow);
	};
} // namespace UE::Editor::RevisionControl::Private


void UTypedElementUObjectPackagePathFactory::RegisterHierarchies(UE::Editor::DataStorage::ICoreProvider& DataStorage)
{
	using namespace UE::Editor::DataStorage;

	const FHierarchyRegistrationParams Params
	{
		.Name = PackageHierarchyName,
	};

	DataStorage.RegisterHierarchy(Params);
}

void UTypedElementUObjectPackagePathFactory::RegisterQueries(UE::Editor::DataStorage::ICoreProvider& DataStorage)
{
	using namespace UE::Editor::DataStorage::Queries;
	
	UE::Editor::RevisionControl::Private::CVarSyncPackageInformation->AsVariable()->OnChangedDelegate().AddLambda(
		[this, &DataStorage](IConsoleVariable* AutoPopulate)
		{
			if (AutoPopulate->GetBool())
			{
				RegisterTryAddPackageRef(DataStorage);
			}
			else
			{
				DataStorage.UnregisterQuery(TryAddPackageRef);
				DataStorage.UnregisterQuery(TryReplacePackageRef);
			}
		}
	);
	DataStorage.RegisterQuery(
		Select(
			TEXT("Resolve package references"),
			FProcessor(EQueryTickPhase::FrameEnd, DataStorage.GetQueryTickGroupName(EQueryTickGroups::SyncExternalToDataStorage)),
			[](IQueryContext& Context, RowHandle Row, const FTypedElementUObjectColumn& Object, const FTypedElementPackageUnresolvedReference& UnresolvedPackageReference)
			{
				const RowHandle PackageRow = Context.LookupMappedRow(
					UE::Editor::RevisionControl::MappingDomain, FMapKeyView(UnresolvedPackageReference.PathOnDisk));
				if (!Context.IsRowAvailable(PackageRow))
				{
					return;
				}
				if (const UObject* ObjectInstance = Object.Object.Get(); IsValid(ObjectInstance))
				{
					if (UPackage* Package = ObjectInstance->GetPackage())
					{
						Context.RemoveColumns(Row, { FTypedElementPackageUnresolvedReference::StaticStruct() });

						UE::Editor::RevisionControl::Private::ResolvePackageReference(Context, Package, ObjectInstance, Row, PackageRow, UnresolvedPackageReference.PathOnDisk);
					}
				}
			})
		.AccessesHierarchy(UE::Editor::DataStorage::PackageHierarchyName)
		.Compile());

	if (UE::Editor::RevisionControl::Private::CVarSyncPackageInformation->GetBool())
	{
		RegisterTryAddPackageRef(DataStorage);
	}
}

void UTypedElementUObjectPackagePathFactory::RegisterTryAddPackageRef(UE::Editor::DataStorage::ICoreProvider& DataStorage)
{
	using namespace UE::Editor::DataStorage;
	using namespace UE::Editor::DataStorage::Queries;

	struct FRegisterPackageRowCommand
	{
		void operator()()
		{
			if (CoreProvider)
			{
				// Check in case this row was mapped before this command was pushed 
				if (const RowHandle PackageRow = CoreProvider->LookupMappedRow(UE::Editor::RevisionControl::MappingDomain, FMapKey(PackagePathname)); !CoreProvider->IsRowAvailable(PackageRow))
				{
					if (const TableHandle PackageTable = CoreProvider->FindTable(PackageTableName); PackageTable != InvalidTableHandle)
					{
						const RowHandle Row = CoreProvider->AddRow(PackageTable);
						CoreProvider->MapRow(UE::Editor::RevisionControl::MappingDomain, FMapKey(PackagePathname), Row);
					}
				}
			}
		}
		FString PackagePathname;
		ICoreProvider* CoreProvider = nullptr;
	};

	auto PackageSyncCallback = [](IQueryContext& Context, RowHandle Row, const FTypedElementUObjectColumn& Object)
	{
		// Don't monitor transient objects since they will never exist on disk. And don't monitor objects in cooked packages since
		// they are (likely) not under the source control root and even if they happen to be, we didn't load the loose asset on disk.
		if (const UObject* ObjectInstance = Object.Object.Get(); IsValid(ObjectInstance) && !ObjectInstance->HasAnyFlags(RF_Transient | RF_BeginDestroyed))
		{
			if (UPackage* Target = ObjectInstance->GetPackage(); !Target->HasAnyPackageFlags(PKG_Cooked))
			{
				FString Path;
				Target->GetPathName(nullptr, Path);
				FString Extension = Target->ContainsMap() ? TEXT(".umap") : TEXT(".uasset");
				if (FString PackageFilename; FPackageName::TryConvertLongPackageNameToFilename(Path, PackageFilename, Extension))
				{
					FPaths::NormalizeFilename(PackageFilename);
					FString FullPackageFilename = FPaths::ConvertRelativePathToFull(PackageFilename);
					const RowHandle PackageRow = Context.LookupMappedRow(UE::Editor::RevisionControl::MappingDomain, FMapKeyView(FullPackageFilename));
					if (Context.IsRowAvailable(PackageRow))
					{
						UE::Editor::RevisionControl::Private::ResolvePackageReference(Context, Target, ObjectInstance, Row, PackageRow, FullPackageFilename);
					}
					else
					{
						FTypedElementPackageUnresolvedReference UnresolvedPackageReference;
						UnresolvedPackageReference.PathOnDisk = FullPackageFilename;
						Context.AddColumn(Row, MoveTemp(UnresolvedPackageReference));

						FRegisterPackageRowCommand Command
						{
							.PackagePathname = FullPackageFilename,
							.CoreProvider = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName),
						};
						Context.PushCommand(Command);
					}
				}
			}
		}
	};
	
	TryAddPackageRef = DataStorage.RegisterQuery(
		Select(
			TEXT("Sync UObject package info to new rows"),
			FObserver::OnAdd<FTypedElementUObjectColumn>()
				.SetExecutionMode(EExecutionMode::GameThread),
			PackageSyncCallback)
		.AccessesHierarchy(PackageHierarchyName)
		.Compile());

	TryReplacePackageRef = DataStorage.RegisterQuery(
		Select(
			TEXT("Resync UObject package info to columns on package removal"),
			FObserver::OnRemove<FTypedElementPackageReference>()
				.SetExecutionMode(EExecutionMode::GameThread),
			PackageSyncCallback)
		.AccessesHierarchy(PackageHierarchyName)
		.Compile());
}
