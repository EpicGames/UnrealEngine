// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataStorage/Handles.h"
#include "Elements/Interfaces/TypedElementDataStorageFactory.h"

#include "ObjectPackagePathToColumnQueries.generated.h"

namespace UE::Editor::DataStorage
{
	class ICoreProvider;
} // namespace UE::Editor::DataStorage

UCLASS()
class UTypedElementUObjectPackagePathFactory : public UEditorDataStorageFactory
{
	GENERATED_BODY()

public:
	~UTypedElementUObjectPackagePathFactory() override = default;

	void RegisterHierarchies(UE::Editor::DataStorage::ICoreProvider& DataStorage) override;
	void RegisterQueries(UE::Editor::DataStorage::ICoreProvider& DataStorage) override;

private:
	void RegisterTryAddPackageRef(UE::Editor::DataStorage::ICoreProvider& DataStorage);
	UE::Editor::DataStorage::QueryHandle TryAddPackageRef = UE::Editor::DataStorage::InvalidQueryHandle;
	UE::Editor::DataStorage::QueryHandle TryReplacePackageRef = UE::Editor::DataStorage::InvalidQueryHandle;
};
