// Copyright Epic Games, Inc. All Rights Reserved.

#include "TedsRevisionControlModule.h"

#include "Processors/RevisionControlProcessors.h"
#include "RevisionControlOverlaySettings.h"

#include "DataStorage/Features.h"
#include "Elements/Interfaces/TypedElementDataStorageInterface.h"
#include "Elements/Interfaces/TypedElementQueryStorageInterfaces.h"

namespace UE::Editor::RevisionControl::Private
{
	static void RefreshOverlayStates()
	{
		if (!FModuleManager::Get().IsModuleLoaded("TypedElementFramework"))
		{
			return;
		}

		using namespace UE::Editor::DataStorage;
		ICoreProvider* DataStorage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName);
		if (const URevisionControlDataStorageFactory* Factory = DataStorage->FindFactory<URevisionControlDataStorageFactory>())
		{
			Factory->UpdateOverlaysForSCCState(DataStorage);
		}
	}

	static void RefreshOverlayColors()
	{
		if (!FModuleManager::Get().IsModuleLoaded("TypedElementFramework"))
		{
			return;
		}

		using namespace UE::Editor::DataStorage;
		ICoreProvider* DataStorage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName);
		if (const URevisionControlDataStorageFactory* Factory = DataStorage->FindFactory<URevisionControlDataStorageFactory>())
		{
			Factory->UpdateOverlayColors(DataStorage);
		}
	}
}

void FTedsRevisionControlModule::StartupModule()
{
	OverlayStatesChangedHandle = URevisionControlOverlaySettings::OnOverlayStatesChanged.AddStatic(
		&UE::Editor::RevisionControl::Private::RefreshOverlayStates);
	OverlayColorsChangedHandle = URevisionControlOverlaySettings::OnOverlayColorsChanged.AddStatic(
		&UE::Editor::RevisionControl::Private::RefreshOverlayColors);
}

void FTedsRevisionControlModule::ShutdownModule()
{
	URevisionControlOverlaySettings::OnOverlayStatesChanged.Remove(OverlayStatesChangedHandle);
	URevisionControlOverlaySettings::OnOverlayColorsChanged.Remove(OverlayColorsChangedHandle);
}

void FTedsRevisionControlModule::AddReferencedObjects(FReferenceCollector& Collector)
{
}

FString FTedsRevisionControlModule::GetReferencerName() const
{
	return TEXT("Teds: Revision Control Module");
}

IMPLEMENT_MODULE(FTedsRevisionControlModule, TedsRevisionControl)
