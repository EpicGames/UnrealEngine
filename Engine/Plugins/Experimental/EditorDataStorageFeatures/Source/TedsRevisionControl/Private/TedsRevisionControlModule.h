// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/IDelegateInstance.h"
#include "Modules/ModuleManager.h"
#include "UObject/GCObject.h"

class FReferenceCollector;

class FTedsRevisionControlModule : public IModuleInterface, public FGCObject
{
public:
	~FTedsRevisionControlModule() override = default;

	//
	// IModuleInterface
	//

	virtual void StartupModule() override;
	virtual void ShutdownModule() override;


	//
	// FGCObject
	//

	void AddReferencedObjects(FReferenceCollector& Collector) override;
	FString GetReferencerName() const override;

private:
	FDelegateHandle OverlayStatesChangedHandle;
	FDelegateHandle OverlayColorsChangedHandle;
};
