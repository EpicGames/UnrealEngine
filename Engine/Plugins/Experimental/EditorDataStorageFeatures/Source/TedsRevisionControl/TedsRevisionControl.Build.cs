// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class TedsRevisionControl : ModuleRules
{
	public TedsRevisionControl(ReadOnlyTargetRules Target) : base(Target)
	{
		// Enable truncation warnings in this module
		CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Warning;

		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		if (Target.bBuildEditor)
		{
			PublicIncludePaths.AddRange(new string[] {});
			PrivateIncludePaths.AddRange(new string[] {});

			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
					"Engine",
					"SourceControl",
					"TypedElementFramework",
				});

			PrivateDependencyModuleNames.AddRange(new string[] 
			{ 
				"TedsActorCompatibility",
				"SourceControlViewport",
			});

			DynamicallyLoadedModuleNames.AddRange(new string[] {});
		}
	}
}