// Copyright Epic Games, Inc. All Rights Reserved.

#include "TedsSettingsEditorSubsystem.h"

#include "Editor.h"
#include "TedsSettingsLog.h"
#include "TedsSettingsManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TedsSettingsEditorSubsystem)

namespace UE::Editor::Settings::Private
{
	static TAutoConsoleVariable<bool> CVarTedsSettingsEnable(
		TEXT("TEDS.Feature.Settings.Enable"),
		true,
		TEXT("When true, settings objects from the ISettingsModule will be mirrored to rows in the editor data storage."));
} // namespace UE::Editor::Settings::Private

UTedsSettingsEditorSubsystem::UTedsSettingsEditorSubsystem()
	: UEditorSubsystem()
	, SettingsManager{ MakeShared<FTedsSettingsManager>() }
	, EnabledChangedDelegate{ }
{
}

const bool UTedsSettingsEditorSubsystem::IsEnabled() const
{
	return UE::Editor::Settings::Private::CVarTedsSettingsEnable.GetValueOnGameThread();
}

UTedsSettingsEditorSubsystem::FOnEnabledChanged& UTedsSettingsEditorSubsystem::OnEnabledChanged()
{
	return EnabledChangedDelegate;
}

UE::Editor::DataStorage::RowHandle UTedsSettingsEditorSubsystem::FindSettingsContainer(const FName& ContainerName) const
{
	return SettingsManager->FindSettingsContainer(ContainerName);
}

UE::Editor::DataStorage::RowHandle UTedsSettingsEditorSubsystem::FindSettingsCategory(const FName& ContainerName, const FName& CategoryName) const
{
	return SettingsManager->FindSettingsCategory(ContainerName, CategoryName);
}

UE::Editor::DataStorage::RowHandle UTedsSettingsEditorSubsystem::FindSettingsSection(const FName& ContainerName, const FName& CategoryName, const FName& SectionName) const
{
	return SettingsManager->FindSettingsSection(ContainerName, CategoryName, SectionName);
}

bool UTedsSettingsEditorSubsystem::GetSettingsSectionFromRow(UE::Editor::DataStorage::RowHandle SectionRow, FName& OutContainerName, FName& OutCategoryName, FName& OutSectionName)
{
	return SettingsManager->GetSettingsSectionFromRow(SectionRow, OutContainerName, OutCategoryName, OutSectionName);
}

bool UTedsSettingsEditorSubsystem::GetSettingsCategoryFromRow(UE::Editor::DataStorage::RowHandle CategoryRow, FName& OutContainerName, FName& OutCategoryName)
{
	return SettingsManager->GetSettingsCategoryFromRow(CategoryRow, OutContainerName, OutCategoryName);
}

void UTedsSettingsEditorSubsystem::Initialize(FSubsystemCollectionBase& Collection)
{
	Super::Initialize(Collection);

	UE_LOGF(LogTedsSettings, Log, "UTedsSettingsEditorSubsystem::Initialize");

	UE::Editor::Settings::Private::CVarTedsSettingsEnable->SetOnChangedCallback(
		FConsoleVariableDelegate::CreateLambda([this](IConsoleVariable* Variable)
		{
			const bool bIsEnabled = Variable->GetBool();
			
			if (bIsEnabled)
			{
				SettingsManager->Initialize();
			}
			else
			{
				SettingsManager->Shutdown();
			}

			EnabledChangedDelegate.Broadcast();
		}));

	if (UE::Editor::Settings::Private::CVarTedsSettingsEnable.GetValueOnGameThread())
	{
		SettingsManager->Initialize();
	}
}

void UTedsSettingsEditorSubsystem::Deinitialize()
{
	UE_LOGF(LogTedsSettings, Log, "UTedsSettingsEditorSubsystem::Deinitialize");

	if (UE::Editor::Settings::Private::CVarTedsSettingsEnable.GetValueOnGameThread())
	{
		SettingsManager->Shutdown();
	}

	Super::Deinitialize();
}
