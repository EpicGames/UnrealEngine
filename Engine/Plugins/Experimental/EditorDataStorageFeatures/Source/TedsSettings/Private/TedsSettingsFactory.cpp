// Copyright Epic Games, Inc. All Rights Reserved.

#include "TedsSettingsFactory.h"

#include "TedsSettingsColumns.h"
#include "TedsSettingsWidgets.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TedsSettingsFactory)

void UTedsSettingsFactory::RegisterWidgetConstructors(UE::Editor::DataStorage::ICoreProvider& DataStorage, UE::Editor::DataStorage::IUiProvider& DataStorageUi) const
{
	using namespace UE::Editor::DataStorage::Queries;

	RowHandle PurposeRow = DataStorageUi.FindPurpose(DataStorageUi.GetGeneralWidgetPurposeID());
	
	DataStorageUi.RegisterWidgetFactory<FSettingsContainerReferenceWidgetConstructor>(PurposeRow, TColumn<FSettingsContainerReferenceColumn>());
	DataStorageUi.RegisterWidgetFactory<FSettingsCategoryReferenceWidgetConstructor>(PurposeRow, TColumn<FSettingsCategoryReferenceColumn>());
	DataStorageUi.RegisterWidgetFactory<FSettingsSectionWidgetConstructor>(PurposeRow, TColumn<FSettingsSectionTag>());
}
