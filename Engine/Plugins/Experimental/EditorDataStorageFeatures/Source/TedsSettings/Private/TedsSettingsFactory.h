// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/Interfaces/TypedElementDataStorageFactory.h"
#include "UObject/ObjectMacros.h"

#include "TedsSettingsFactory.generated.h"

UCLASS()
class UTedsSettingsFactory : public UEditorDataStorageFactory
{
	GENERATED_BODY()

public:

	~UTedsSettingsFactory() override = default;

	void RegisterWidgetConstructors(UE::Editor::DataStorage::ICoreProvider& DataStorage,
		UE::Editor::DataStorage::IUiProvider& DataStorageUi) const override;

};
