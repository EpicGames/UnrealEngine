// Copyright Epic Games, Inc. All Rights Reserved.

#include "TedsSettingsManager.h"

#include "Containers/Ticker.h"
#include "DataStorage/Features.h"
#include "Elements/Columns/TypedElementCompatibilityColumns.h"
#include "Elements/Columns/TypedElementMiscColumns.h"
#include "Elements/Columns/TypedElementSlateWidgetColumns.h"
#include "Elements/Columns/TypedElementTypeInfoColumns.h"
#include "Elements/Framework/TypedElementQueryBuilder.h"
#include "Elements/Interfaces/TypedElementDataStorageInterface.h"
#include "Engine/DeveloperSettings.h"
#include "ISettingsCategory.h"
#include "ISettingsContainer.h"
#include "ISettingsEditorModule.h"
#include "ISettingsModule.h"
#include "ISettingsSection.h"
#include "Logging/LogMacros.h"
#include "Misc/PackageName.h"
#include "Modules/ModuleManager.h"
#include "ProfilingDebugging/CpuProfilerTrace.h"
#include "TedsSettingsColumns.h"
#include "TedsSettingsLog.h"
#include "UObject/Package.h"
#include "UObject/UnrealType.h"

namespace UE::Editor::Settings::Private
{
	static UE::Editor::DataStorage::FMapKey GenerateContainerIndexKey(const FName& ContainerName)
	{
		FString Key = "ISettingsContainer: ";
		ContainerName.AppendString(Key);

		return UE::Editor::DataStorage::FMapKey(MoveTemp(Key));
	}

	static UE::Editor::DataStorage::FMapKey GenerateCategoryIndexKey(const FName& ContainerName, const FName& CategoryName)
	{
		FString Key = "ISettingsCategory: ";
		ContainerName.AppendString(Key);
		Key.AppendChar(TEXT(','));
		CategoryName.AppendString(Key);

		return UE::Editor::DataStorage::FMapKey(MoveTemp(Key));
	}

	static UE::Editor::DataStorage::FMapKey GenerateSectionIndexKey(const FName& ContainerName, const FName& CategoryName, const FName& SectionName)
	{
		FString Key = "ISettingsSection: ";
		ContainerName.AppendString(Key);
		Key.AppendChar(TEXT(','));
		CategoryName.AppendString(Key);
		Key.AppendChar(TEXT(','));
		SectionName.AppendString(Key);
	
		return UE::Editor::DataStorage::FMapKey(MoveTemp(Key));
	}
}

FTedsSettingsManager::FTedsSettingsManager()
	: bIsInitialized{ false }
	, SelectAllSettingsQuery{ UE::Editor::DataStorage::InvalidQueryHandle }
	, SettingsContainerTable{ UE::Editor::DataStorage::InvalidTableHandle }
	, SettingsCategoryTable{ UE::Editor::DataStorage::InvalidTableHandle }
	, SettingsPropertyTable{ UE::Editor::DataStorage::InvalidTableHandle }
	, SettingsHierarchy{ }
	, SelectAllSettingsInModuleQueryMap{ }
{
}

void FTedsSettingsManager::Initialize()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FTedsSettingsManager::Initialize);

	if (!bIsInitialized)
	{
		using namespace UE::Editor::DataStorage;
		auto OnDataStorage = [this]
			{
				ICoreProvider* DataStorage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName);
				check(DataStorage);

				RegisterTables(*DataStorage);
				RegisterHierarchies(*DataStorage);
				RegisterQueries(*DataStorage);
				AddSettings();

				////////////////////////////////////////////////////////////////////////////////////////////////////
				// Take ownership of auto-discovered settings (UDeveloperSettings) registration from the SettingsEditorModule.
				// First, force the SettingsEditorModule to register any pending settings from modules that may have already loaded.
				// Then, disable the SettingsEditorModule's ability to register settings so that we can take over with incremental registration.
				ISettingsModule& SettingsModule = FModuleManager::LoadModuleChecked<ISettingsModule>("Settings");
				ISettingsEditorModule& SettingsEditorModule = FModuleManager::LoadModuleChecked<ISettingsEditorModule>("SettingsEditor");

				// During the call to SettingsEditorModule.UpdateSettings we want to register any pending settings
				// but we don't want the SettingsEditorModule to remember those (if it did then it might try to unregister them while we're in charge)
				SettingsEditorModule.SetShouldRegisterSettingCallback(FShouldRegisterSettingsDelegate::CreateLambda([this, &SettingsModule](UDeveloperSettings* Settings)
					{
						RegisterSettings(SettingsModule, Settings);
						return false;
					}));

				constexpr bool bForce = true;
				SettingsEditorModule.UpdateSettings(bForce);

				SettingsEditorModule.SetShouldRegisterSettingCallback(FShouldRegisterSettingsDelegate::CreateLambda([](UDeveloperSettings* Settings)
					{
						return false;
					}));

				FModuleManager::Get().OnModulesChanged().AddSP(this, &FTedsSettingsManager::OnModulesChanged);
				////////////////////////////////////////////////////////////////////////////////////////////////////
			};

		if (AreEditorDataStorageFeaturesEnabled())
		{
			OnDataStorage();
		}
		else
		{
			OnEditorDataStorageFeaturesEnabled().AddSPLambda(this, OnDataStorage);
		}

		bIsInitialized = true;
	}
}

void FTedsSettingsManager::Shutdown()
{
	if (bIsInitialized)
	{
		using namespace UE::Editor::DataStorage;
		OnEditorDataStorageFeaturesEnabled().RemoveAll(this);

		if (AreEditorDataStorageFeaturesEnabled())
		{
			FModuleManager::Get().OnModulesChanged().RemoveAll(this);

			if (ISettingsEditorModule* SettingsEditorModule = FModuleManager::GetModulePtr<ISettingsEditorModule>("SettingsEditor"))
			{
				SettingsEditorModule->SetShouldRegisterSettingCallback(nullptr);
			}

			ICoreProvider* DataStorage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName);
			check(DataStorage);

			RemoveSettings();
			UnregisterQueries(*DataStorage);
		}	

		bIsInitialized = false;
	}
}

UE::Editor::DataStorage::RowHandle FTedsSettingsManager::FindSettingsContainer(const FName& ContainerName) const
{
	using namespace UE::Editor::DataStorage;
	using namespace UE::Editor::Settings::Private;

	if (bIsInitialized)
	{
		if (const ICoreProvider* DataStorage = GetDataStorageFeature<ICoreProvider>(StorageFeatureName))
		{
			FMapKey ContainerIndexKey = GenerateContainerIndexKey(ContainerName);

			return DataStorage->LookupMappedRow(Settings::MappingDomain, ContainerIndexKey);
		}
	}

	return InvalidRowHandle;
}

UE::Editor::DataStorage::RowHandle FTedsSettingsManager::FindSettingsCategory(const FName& ContainerName, const FName& CategoryName) const
{
	using namespace UE::Editor::DataStorage;
	using namespace UE::Editor::Settings::Private;

	if (bIsInitialized)
	{
		if (const ICoreProvider* DataStorage = GetDataStorageFeature<ICoreProvider>(StorageFeatureName))
		{
			FMapKey CategoryIndexKey = GenerateCategoryIndexKey(ContainerName, CategoryName);

			return DataStorage->LookupMappedRow(Settings::MappingDomain, CategoryIndexKey);
		}
	}

	return InvalidRowHandle;
}

UE::Editor::DataStorage::RowHandle FTedsSettingsManager::FindSettingsSection(const FName& ContainerName, const FName& CategoryName, const FName& SectionName) const
{
	using namespace UE::Editor::DataStorage;
	using namespace UE::Editor::Settings::Private;

	if (bIsInitialized)
	{
		if (const ICoreProvider* DataStorage = GetDataStorageFeature<ICoreProvider>(StorageFeatureName))
		{
			FMapKey SectionIndexKey = GenerateSectionIndexKey(ContainerName, CategoryName, SectionName);

			return DataStorage->LookupMappedRow(Settings::MappingDomain, SectionIndexKey);
		}
	}

	return InvalidRowHandle;
}

bool FTedsSettingsManager::GetSettingsSectionFromRow(UE::Editor::DataStorage::RowHandle Row, FName& OutContainerName, FName& OutCategoryName, FName& OutSectionName)
{
	using namespace UE::Editor::DataStorage;

	if (bIsInitialized)
	{
		if (const ICoreProvider* DataStorage = GetDataStorageFeature<ICoreProvider>(StorageFeatureName))
		{
			if (DataStorage->IsRowAvailable(Row) && (DataStorage->HasColumns<FSettingsSectionTag>(Row)))
			{
				const FSettingsContainerReferenceColumn* ContainerReferenceColumn = DataStorage->GetColumn<FSettingsContainerReferenceColumn>(Row);
				const FSettingsCategoryReferenceColumn* CategoryReferenceColumn = DataStorage->GetColumn<FSettingsCategoryReferenceColumn>(Row);
				const FSettingsNameColumn* NameColumn = DataStorage->GetColumn<FSettingsNameColumn>(Row);

				if (ContainerReferenceColumn && CategoryReferenceColumn && NameColumn)
				{
					OutContainerName = ContainerReferenceColumn->ContainerName;
					OutCategoryName = CategoryReferenceColumn->CategoryName;
					OutSectionName = NameColumn->Name;

					return true;
				}
			}
		}
	}

	return false;
}

bool FTedsSettingsManager::GetSettingsCategoryFromRow(UE::Editor::DataStorage::RowHandle Row, FName& OutContainerName, FName& OutCategoryName)
{
	using namespace UE::Editor::DataStorage;

	if (bIsInitialized)
	{
		if (const ICoreProvider* DataStorage = GetDataStorageFeature<ICoreProvider>(StorageFeatureName))
		{
			if (DataStorage->IsRowAvailable(Row) && DataStorage->HasColumns<FSettingsCategoryTag>(Row))
			{
				const FSettingsContainerReferenceColumn* ContainerReferenceColumn = DataStorage->GetColumn<FSettingsContainerReferenceColumn>(Row);
				const FSettingsNameColumn* NameColumn = DataStorage->GetColumn<FSettingsNameColumn>(Row);

				if (ContainerReferenceColumn && NameColumn)
				{
					OutContainerName = ContainerReferenceColumn->ContainerName;
					OutCategoryName = NameColumn->Name;

					return true;
				}
			}
		}
	}

	return false;
}

void FTedsSettingsManager::RegisterTables(UE::Editor::DataStorage::ICoreProvider& DataStorage)
{
	using namespace UE::Editor::DataStorage;

	if (SettingsContainerTable == InvalidTableHandle)
	{
		SettingsContainerTable = DataStorage.RegisterTable(
			TTypedElementColumnTypeList<FSettingsNameColumn, FDisplayNameColumn, FDescriptionColumn, FSettingsContainerTag>(),
			FName(TEXT("Editor_SettingsContainerTable")));
	}

	if (SettingsCategoryTable == InvalidTableHandle)
	{
		SettingsCategoryTable = DataStorage.RegisterTable(
			TTypedElementColumnTypeList<FSettingsContainerReferenceColumn, FSettingsNameColumn, FDisplayNameColumn, FDescriptionColumn, FSettingsCategoryTag>(),
			FName(TEXT("Editor_SettingsCategoryTable")));
	}

	if (SettingsPropertyTable == InvalidTableHandle)
	{
		SettingsPropertyTable = DataStorage.RegisterTable(
			TTypedElementColumnTypeList<FSettingsNameColumn, FDisplayNameColumn, FDescriptionColumn, FSettingsPropertyTag>(),
			FName(TEXT("Editor_SettingsPropertyTable")));
	}
}

void FTedsSettingsManager::RegisterHierarchies(UE::Editor::DataStorage::ICoreProvider& DataStorage)
{
	using namespace UE::Editor::DataStorage;

	FHierarchyRegistrationParams HierarchyRegistrationParams
	{
		.Name = UE::Editor::DataStorage::Settings::HierarchyName
	};

	SettingsHierarchy = DataStorage.RegisterHierarchy(HierarchyRegistrationParams);
}

void FTedsSettingsManager::RegisterQueries(UE::Editor::DataStorage::ICoreProvider& DataStorage)
{
	using namespace UE::Editor::DataStorage::Queries;

	if (SelectAllSettingsQuery == UE::Editor::DataStorage::InvalidQueryHandle)
	{
		SelectAllSettingsQuery = DataStorage.RegisterQuery(
			Select()
				.ReadOnly<FSettingsContainerReferenceColumn, FSettingsCategoryReferenceColumn, FSettingsNameColumn>()
			.Where()
				.All<FSettingsSectionTag>()
			.Compile());
	}
}

void FTedsSettingsManager::UnregisterQueries(UE::Editor::DataStorage::ICoreProvider& DataStorage)
{
	using namespace UE::Editor::DataStorage;

	DataStorage.UnregisterQuery(SelectAllSettingsQuery);
	SelectAllSettingsQuery = InvalidQueryHandle;

	for (const TPair<FName, QueryHandle>& Pairs : SelectAllSettingsInModuleQueryMap)
	{
		DataStorage.UnregisterQuery(Pairs.Value);
	}
	SelectAllSettingsInModuleQueryMap.Reset();
}

void FTedsSettingsManager::AddSettings()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(TedsSettingsManager.AddSettings);

	ISettingsModule& SettingsModule = FModuleManager::GetModuleChecked<ISettingsModule>("Settings");

	TArray<FName> ContainerNames;
	SettingsModule.GetContainerNames(ContainerNames);

	for (FName ContainerName : ContainerNames)
	{
		AddSettingsContainer(ContainerName);
	}

	SettingsModule.OnContainerAdded().AddSP(this, &FTedsSettingsManager::AddSettingsContainer);
}

void FTedsSettingsManager::AddSettingsContainer(const FName& ContainerName)
{
	using namespace UE::Editor::DataStorage;
	using namespace UE::Editor::Settings::Private;

	TRACE_CPUPROFILER_EVENT_SCOPE(TedsSettingsManager.AddSettingsContainer);

	ISettingsModule& SettingsModule = FModuleManager::GetModuleChecked<ISettingsModule>("Settings");

	ICoreProvider* DataStorage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName);
	check(DataStorage);

	UE_LOGF(LogTedsSettings, Verbose, "Add Settings Container : '%ls'", *ContainerName.ToString());

	ISettingsContainerPtr ContainerPtr = SettingsModule.GetContainer(ContainerName);

	FMapKey ContainerIndexKey = GenerateContainerIndexKey(ContainerName);
	RowHandle ContainerRow = DataStorage->LookupMappedRow(Settings::MappingDomain, ContainerIndexKey);
	if (ContainerRow == InvalidRowHandle)
	{
		ContainerRow = DataStorage->AddRow(SettingsContainerTable);
		DataStorage->AddColumn<FSettingsNameColumn>(ContainerRow, { .Name = ContainerName });
		DataStorage->AddColumn<FDisplayNameColumn>(ContainerRow, { .DisplayName = ContainerPtr->GetDisplayName() });
		DataStorage->AddColumn<FDescriptionColumn>(ContainerRow, { .Description = ContainerPtr->GetDescription() });
		DataStorage->AddColumn<FSettingsContainerTag>(ContainerRow);

		DataStorage->MapRow(Settings::MappingDomain, MoveTemp(ContainerIndexKey), ContainerRow);
	}

	TArray<ISettingsCategoryPtr> Categories;
	ContainerPtr->GetCategories(Categories);

	for (ISettingsCategoryPtr CategoryPtr : Categories)
	{
		const bool bQueryExistingRows = false;
		UpdateSettingsCategory(CategoryPtr, ContainerRow, bQueryExistingRows);
	}

	// OnCategoryModified is called at the same time as OnSectionRemoved so we only bind to OnCategoryModified for add / update / remove
	ContainerPtr->OnCategoryModified().AddSPLambda(this, [this, ContainerPtr, ContainerRow](const FName& ModifiedCategoryName)
		{
			UE_LOGF(LogTedsSettings, Verbose, "Settings Category modified : '%ls->%ls'", *ContainerPtr->GetName().ToString(), *ModifiedCategoryName.ToString());

			ISettingsCategoryPtr CategoryPtr = ContainerPtr->GetCategory(ModifiedCategoryName);

			UpdateSettingsCategory(CategoryPtr, ContainerRow);
		});
}

void FTedsSettingsManager::RemoveSettings()
{
	using namespace UE::Editor::DataStorage;
	using namespace UE::Editor::Settings::Private;

	TRACE_CPUPROFILER_EVENT_SCOPE(TedsSettingsManager.RemoveSettings);

	ISettingsModule& SettingsModule = FModuleManager::GetModuleChecked<ISettingsModule>("Settings");

	ICoreProvider* DataStorage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName);
	check(DataStorage);

	ICompatibilityProvider* DataStorageCompatibility = GetMutableDataStorageFeature<ICompatibilityProvider>(CompatibilityFeatureName);
	check(DataStorageCompatibility);

	SettingsModule.OnContainerAdded().RemoveAll(this);

	TArray<FName> ContainerNames;
	SettingsModule.GetContainerNames(ContainerNames);

	for (FName ContainerName : ContainerNames)
	{
		UE_LOGF(LogTedsSettings, Verbose, "Remove Settings Container : '%ls'", *ContainerName.ToString());

		ISettingsContainerPtr ContainerPtr = SettingsModule.GetContainer(ContainerName);

		ContainerPtr->OnCategoryModified().RemoveAll(this);

		FMapKey ContainerIndexKey = GenerateContainerIndexKey(ContainerName);
		RowHandle ContainerRow = DataStorage->LookupMappedRow(Settings::MappingDomain, ContainerIndexKey);
		if (ContainerRow != InvalidRowHandle)
		{
			TArray<TWeakObjectPtr<UObject>> ObjectsToRemove;
			FRowHandleArray RowsToRemove;
			auto VisitRow = [DataStorage, DataStorageCompatibility, &RowsToRemove, &ObjectsToRemove](const ICoreProvider&, RowHandle, RowHandle TargetRow)
			{
				if (FTypedElementUObjectColumn* UObjectColumn = DataStorage->GetColumn<FTypedElementUObjectColumn>(TargetRow))
				{
					ObjectsToRemove.Add(UObjectColumn->Object);
				}
				else
				{
					RowsToRemove.Add(TargetRow);
				}
			};

			DataStorage->WalkDepthFirst(SettingsHierarchy, ContainerRow, VisitRow);
			DataStorage->BatchRemoveRows(RowsToRemove.GetRows());

			for (TWeakObjectPtr<UObject> ObjectToRemove : ObjectsToRemove)
			{
				DataStorageCompatibility->RemoveCompatibleObject(ObjectToRemove);
			}
		}
	}
}

void FTedsSettingsManager::UpdateSettingsCategory(TSharedPtr<ISettingsCategory> SettingsCategory, UE::Editor::DataStorage::RowHandle ContainerRow, const bool bQueryExistingRows)
{
	using namespace UE::Editor::DataStorage;
	using namespace UE::Editor::Settings::Private;

	TRACE_CPUPROFILER_EVENT_SCOPE(TedsSettingsManager.UpdateSettingsCategory);

	ICoreProvider* DataStorage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName);
	check(DataStorage);

	ICompatibilityProvider* DataStorageCompatibility = GetMutableDataStorageFeature<ICompatibilityProvider>(CompatibilityFeatureName);
	check(DataStorageCompatibility);

	const FSettingsNameColumn* ContainerNameColumn = DataStorage->GetColumn<FSettingsNameColumn>(ContainerRow);
	checkf(ContainerNameColumn, TEXT("FSettingsNameColumn must exist on ContainerRow: %llu"), ContainerRow);

	const FName ContainerName = ContainerNameColumn->Name;
	const FName CategoryName = SettingsCategory->GetName();

	UE_LOGF(LogTedsSettings, Verbose, "Update Settings Category: '%ls->%ls'", *ContainerName.ToString(), *CategoryName.ToString());

	FMapKey CategoryIndexKey = GenerateCategoryIndexKey(ContainerName, CategoryName);

	RowHandle CategoryRow = DataStorage->LookupMappedRow(Settings::MappingDomain, CategoryIndexKey);
	if (CategoryRow == InvalidRowHandle)
	{
		CategoryRow = DataStorage->AddRow(SettingsCategoryTable);

		DataStorage->AddColumn<FSettingsContainerReferenceColumn>(CategoryRow, { .ContainerName = ContainerName, .ContainerRow = ContainerRow });
		DataStorage->AddColumn<FSettingsNameColumn>(CategoryRow, { .Name = CategoryName });
		DataStorage->AddColumn<FDisplayNameColumn>(CategoryRow, { .DisplayName = SettingsCategory->GetDisplayName() });
		DataStorage->AddColumn<FDescriptionColumn>(CategoryRow, { .Description = SettingsCategory->GetDescription() });
		DataStorage->AddColumn<FSettingsCategoryTag>(CategoryRow);

		DataStorage->SetParentRow(SettingsHierarchy, CategoryRow, ContainerRow);

		DataStorage->MapRow(Settings::MappingDomain, MoveTemp(CategoryIndexKey), CategoryRow);
	}

	TArray<RowHandle> OldRowHandles;
	TArray<FName> OldSectionNames;

	// Gather all existing section rows for the given { ContainerName, CategoryName } pair.
	if (bQueryExistingRows)
	{
		using namespace UE::Editor::DataStorage::Queries;

		DataStorage->RunQuery(SelectAllSettingsQuery, CreateDirectQueryCallbackBinding(
			[&OldRowHandles, &OldSectionNames, ContainerName, CategoryName](
				IDirectQueryContext& Context,
				const FSettingsContainerReferenceColumn* ContainerColumns,
				const FSettingsCategoryReferenceColumn* CategoryColumns,
				const FSettingsNameColumn* SectionNameColumns)
			{
				const uint32 RowCount = Context.GetRowCount();

				for (uint32 RowIndex = 0; RowIndex < RowCount; ++RowIndex)
				{
					const FName& TempContainerName = ContainerColumns[RowIndex].ContainerName;
					const FName& TempCategoryName = CategoryColumns[RowIndex].CategoryName;
					if (TempContainerName == ContainerName &&
						TempCategoryName == CategoryName)
					{
						OldRowHandles.Emplace(Context.GetRowHandles()[RowIndex]);
						OldSectionNames.Emplace(SectionNameColumns[RowIndex].Name);
					}
				}
			}));
	}

	TArray<FName> NewSectionNames;
	TArray<ISettingsSectionPtr> NewSections;

	const bool bIgnoreVisibility = true;
	SettingsCategory->GetSections(NewSections, bIgnoreVisibility);

	// Iterate the category and add rows for all sections ( replace any existing row for the section as its object may have changed )
	for (ISettingsSectionPtr SectionPtr : NewSections)
	{
		const FName& SectionName = SectionPtr->GetName();

		if (TStrongObjectPtr<UObject> SettingsObjectPtr = SectionPtr->GetSettingsObject().Pin(); SettingsObjectPtr)
		{
			NewSectionNames.Emplace(SectionName);

			FMapKey SectionIndexKey = GenerateSectionIndexKey(ContainerName, CategoryName, SectionName);

			RowHandle OldSectionRow = DataStorage->LookupMappedRow(Settings::MappingDomain, SectionIndexKey);
			if (OldSectionRow != InvalidRowHandle)
			{
				UE_LOGF(LogTedsSettings, Verbose, "Settings Section : '%ls' is already in data storage", *SectionName.ToString());
			}

			RowHandle NewSectionRow = DataStorageCompatibility->AddCompatibleObject(SettingsObjectPtr);

			DataStorage->AddColumn<FSettingsContainerReferenceColumn>(NewSectionRow, { .ContainerName = ContainerName, .ContainerRow = ContainerRow });
			DataStorage->AddColumn<FSettingsCategoryReferenceColumn>(NewSectionRow, { .CategoryName = CategoryName, .CategoryRow = CategoryRow });
			DataStorage->AddColumn<FSettingsNameColumn>(NewSectionRow, { .Name = SectionName });
			DataStorage->AddColumn<FDisplayNameColumn>(NewSectionRow, { .DisplayName = SectionPtr->GetDisplayName() });
			DataStorage->AddColumn<FDescriptionColumn>(NewSectionRow, { .Description = SectionPtr->GetDescription() });
			
			if (UPackage* ClassPackage = SettingsObjectPtr->GetClass()->GetOuterUPackage())
			{
				FName ModuleName = FPackageName::GetShortFName(ClassPackage->GetFName());
				DataStorage->AddColumn<FSettingsModuleTag>(NewSectionRow, ModuleName);
			}

			if (OldSectionRow != InvalidRowHandle && OldSectionRow != NewSectionRow)
			{
				// Remove the old section row and any child property rows.
				FRowHandleArray RowsToRemove;
				auto VisitRow = [DataStorage, OldSectionRow, &RowsToRemove](const ICoreProvider&, RowHandle, RowHandle TargetRow)
				{
					if (TargetRow != OldSectionRow)
					{
						RowsToRemove.Add(TargetRow);
					}
				};

				DataStorage->WalkDepthFirst(SettingsHierarchy, OldSectionRow, VisitRow);
				DataStorage->BatchRemoveRows(RowsToRemove.GetRows());

				if (FTypedElementUObjectColumn* UObjectColumn = DataStorage->GetColumn<FTypedElementUObjectColumn>(OldSectionRow))
				{
					DataStorageCompatibility->RemoveCompatibleObject(UObjectColumn->Object);
				}
				else
				{
					DataStorage->RemoveRow(OldSectionRow);
				}
			}
			
			if (OldSectionRow != NewSectionRow)
			{
				DataStorage->SetParentRow(SettingsHierarchy, NewSectionRow, CategoryRow);

				DataStorage->MapRow(Settings::MappingDomain, MoveTemp(SectionIndexKey), NewSectionRow);

				for (FProperty* Property = SettingsObjectPtr->GetClass()->PropertyLink; Property; Property = Property->PropertyLinkNext)
				{
					if (Property->HasAnyPropertyFlags(CPF_Transient) || !Property->HasAnyPropertyFlags(CPF_Edit))
					{
						continue;
					}

					RowHandle NewPropertyRow = DataStorage->AddRow(SettingsPropertyTable);

					DataStorage->AddColumn<FSettingsPropertyTag>(NewPropertyRow);
					DataStorage->AddColumn<FSettingsNameColumn>(NewPropertyRow, { .Name = Property->GetFName() });
					DataStorage->AddColumn<FDisplayNameColumn>(NewPropertyRow, { .DisplayName = Property->GetDisplayNameText() });
					DataStorage->AddColumn<FDescriptionColumn>(NewPropertyRow, { .Description = Property->GetToolTipText() });

					DataStorage->SetParentRow(SettingsHierarchy, NewPropertyRow, NewSectionRow);
				}
			}

			// Add the FSettingsSectionTag last so that any FObserver::OnAdd<FSettingsSectionTag>() can read the other columns.
			DataStorage->AddColumn<FSettingsSectionTag>(NewSectionRow);

			UE_CLOGF(OldSectionRow == InvalidRowHandle, LogTedsSettings, Verbose, "Added Settings Section : '%ls'", *SectionName.ToString());
			UE_CLOGF(OldSectionRow != InvalidRowHandle, LogTedsSettings, Verbose, "Updated Settings Section : '%ls'", *SectionName.ToString());
		}
	}

	// Iterate the old sections and remove rows not in the new sections list.
	for (int32 RowIndex = 0; RowIndex < OldSectionNames.Num(); ++RowIndex)
	{
		const FName& OldSectionName = OldSectionNames[RowIndex];

		if (NewSectionNames.Contains(OldSectionName))
		{
			continue;
		}

		RowHandle OldSectionRow = OldRowHandles[RowIndex];
		check(OldSectionRow != InvalidRowHandle);

		// Remove the old section row and any child property rows.
		FRowHandleArray RowsToRemove;
		auto VisitRow = [DataStorage, OldSectionRow, &RowsToRemove](const ICoreProvider&, RowHandle, RowHandle TargetRow)
			{
				if (TargetRow != OldSectionRow)
				{
					RowsToRemove.Add(TargetRow);
				}
			};

		DataStorage->WalkDepthFirst(SettingsHierarchy, OldSectionRow, VisitRow);
		DataStorage->BatchRemoveRows(RowsToRemove.GetRows());

		if (FTypedElementUObjectColumn* UObjectColumn = DataStorage->GetColumn<FTypedElementUObjectColumn>(OldSectionRow))
		{
			DataStorageCompatibility->RemoveCompatibleObject(UObjectColumn->Object);
		}
		else
		{
			DataStorage->RemoveRow(OldSectionRow);
		}

		UE_LOGF(LogTedsSettings, Verbose, "Removed Settings Section : '%ls'", *OldSectionName.ToString());
	}
}

void FTedsSettingsManager::OnModulesChanged(FName ModuleName, EModuleChangeReason ChangeReason)
{
	if (ChangeReason == EModuleChangeReason::ModuleLoaded)
	{
		RegisterSettingsForModule(ModuleName);
	}
	else if (ChangeReason == EModuleChangeReason::ModuleUnloaded)
	{
		UnregisterSettingsForModule(ModuleName);
	}
}

void FTedsSettingsManager::RegisterSettings(ISettingsModule& SettingsModule, UDeveloperSettings* Settings)
{
	TSharedPtr<SWidget> CustomWidget = Settings->GetCustomSettingsWidget();

	const FName ContainerName = Settings->GetContainerName();
	const FName CategoryName = Settings->GetCategoryName();
	const FName SectionName = Settings->GetSectionName();

	UE_LOGF(LogTedsSettings, Verbose, "Register Settings '%ls->%ls->%ls' with the ISettingsModule",
		*ContainerName.ToString(), *CategoryName.ToString(), *SectionName.ToString());

	if (CustomWidget.IsValid())
	{
		SettingsModule.RegisterSettings(
			ContainerName,
			CategoryName,
			SectionName,
			Settings->GetSectionText(),
			Settings->GetSectionDescription(),
			CustomWidget.ToSharedRef());
	}
	else
	{
		SettingsModule.RegisterSettings(
			ContainerName,
			CategoryName,
			SectionName,
			Settings->GetSectionText(),
			Settings->GetSectionDescription(),
			Settings);
	}
}

void FTedsSettingsManager::RegisterSettingsForModule(FName ModuleName)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(TedsSettingsManager.RegisterSettingsForModule);

	// Find the class package for this module
	const UPackage* const ClassPackage = FindPackage(nullptr, *(FString("/Script/") + ModuleName.ToString()));
	if (!ClassPackage)
	{
		return;
	}

	TArray<UObject*> PackageObjects;
	GetObjectsWithPackage(ClassPackage, PackageObjects, EGetObjectsFlags::None);

	TArray<UDeveloperSettings*> SettingsObjects;
	for (UObject* Object : PackageObjects)
	{
		const UClass* CurrentClass = Cast<UClass>(Object);

		if (CurrentClass &&
			!CurrentClass->HasAnyClassFlags(CLASS_Deprecated | CLASS_Abstract) &&
			CurrentClass->IsChildOf(UDeveloperSettings::StaticClass()))
		{
			UDeveloperSettings* SettingsObject = Cast<UDeveloperSettings>(CurrentClass->GetDefaultObject());
			if (SettingsObject && SettingsObject->SupportsAutoRegistration())
			{
				SettingsObjects.Add(SettingsObject);
			}
		}
	}

	if (!SettingsObjects.IsEmpty())
	{
		UE_LOGF(LogTedsSettings, Verbose, "Register Settings from module : '%ls'", *ModuleName.ToString());

		ISettingsModule& SettingsModule = FModuleManager::GetModuleChecked<ISettingsModule>("Settings");

		for (UDeveloperSettings* Settings : SettingsObjects)
		{
			RegisterSettings(SettingsModule, Settings);
		}
	}
}

void FTedsSettingsManager::UnregisterSettingsForModule(FName ModuleName)
{
	using namespace UE::Editor::DataStorage::Queries;

	TRACE_CPUPROFILER_EVENT_SCOPE(TedsSettingsManager.UnregisterSettingsForModule);

	ICoreProvider* DataStorage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName);
	checkf(DataStorage, TEXT("DataStorage is not available"));

	QueryHandle* SelectAllSettingsInModuleQuery = SelectAllSettingsInModuleQueryMap.Find(ModuleName);

	if (!SelectAllSettingsInModuleQuery)
	{
		SelectAllSettingsInModuleQuery = &SelectAllSettingsInModuleQueryMap.Emplace(ModuleName, DataStorage->RegisterQuery(
			Select()
				.ReadOnly<FSettingsContainerReferenceColumn, FSettingsCategoryReferenceColumn, FSettingsNameColumn>()
			.Where()
				.All<FSettingsSectionTag>()
				.All<FSettingsModuleTag>(ModuleName)
			.Compile()));
	}

	TArray<FName> ContainerNames;
	TArray<FName> CategoryNames;
	TArray<FName> SectionNames;

	DataStorage->RunQuery(*SelectAllSettingsInModuleQuery, CreateDirectQueryCallbackBinding(
		[&ContainerNames, &CategoryNames, &SectionNames](IDirectQueryContext& Context,
			const FSettingsContainerReferenceColumn* ContainerColumns,
			const FSettingsCategoryReferenceColumn* CategoryColumns,
			const FSettingsNameColumn* SectionNameColumns)
		{
			const uint32 RowCount = Context.GetRowCount();

			for (uint32 RowIndex = 0; RowIndex < RowCount; ++RowIndex)
			{
				ContainerNames.Emplace(ContainerColumns[RowIndex].ContainerName);
				CategoryNames.Emplace(CategoryColumns[RowIndex].CategoryName);
				SectionNames.Emplace(SectionNameColumns[RowIndex].Name);
			}
		}));

	if (!ContainerNames.IsEmpty())
	{
		UE_LOGF(LogTedsSettings, Verbose, "Unregister Settings from module : '%ls'", *ModuleName.ToString());

		ISettingsModule& SettingsModule = FModuleManager::GetModuleChecked<ISettingsModule>("Settings");

		for (int32 i = 0; i < ContainerNames.Num(); ++i)
		{
			SettingsModule.UnregisterSettings(ContainerNames[i], CategoryNames[i], SectionNames[i]);
		}
	}
}
