// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/Interfaces/TypedElementDataStorageInterface.h"
#include "Templates/SharedPointer.h"

class ISettingsCategory;
class ISettingsModule;
class UDeveloperSettings;

class FTedsSettingsManager final : public TSharedFromThis<FTedsSettingsManager>
{
public:

	FTedsSettingsManager();

	const bool IsInitialized() const
	{
		return bIsInitialized;
	}

	void Initialize();
	void Shutdown();

	UE::Editor::DataStorage::RowHandle FindSettingsContainer(const FName& ContainerName) const;

	UE::Editor::DataStorage::RowHandle FindSettingsCategory(const FName& ContainerName, const FName& CategoryName) const;

	UE::Editor::DataStorage::RowHandle FindSettingsSection(const FName& ContainerName, const FName& CategoryName, const FName& SectionName) const;

	bool GetSettingsSectionFromRow(UE::Editor::DataStorage::RowHandle Row, FName& OutContainerName, FName& OutCategoryName, FName& OutSectionName);

	bool GetSettingsCategoryFromRow(UE::Editor::DataStorage::RowHandle Row, FName& OutContainerName, FName& OutCategoryName);

private:

	void RegisterTables(UE::Editor::DataStorage::ICoreProvider& DataStorage);
	void RegisterHierarchies(UE::Editor::DataStorage::ICoreProvider& DataStorage);

	void RegisterQueries(UE::Editor::DataStorage::ICoreProvider& DataStorage);
	void UnregisterQueries(UE::Editor::DataStorage::ICoreProvider& DataStorage);

	void AddSettings();
	void RemoveSettings();

	void AddSettingsContainer(const FName& ContainerName);

	void UpdateSettingsCategory(TSharedPtr<ISettingsCategory> SettingsCategory, UE::Editor::DataStorage::RowHandle ContainerRow, const bool bQueryExistingRows = true);

	void OnModulesChanged(FName ModuleName, EModuleChangeReason ChangeReason);

	void RegisterSettings(ISettingsModule& SettingsModule, UDeveloperSettings* Settings);
	void RegisterSettingsForModule(FName ModuleName);
	void UnregisterSettingsForModule(FName ModuleName);

	bool bIsInitialized;
	UE::Editor::DataStorage::QueryHandle SelectAllSettingsQuery;
	UE::Editor::DataStorage::TableHandle SettingsContainerTable;
	UE::Editor::DataStorage::TableHandle SettingsCategoryTable;
	UE::Editor::DataStorage::TableHandle SettingsPropertyTable;
	UE::Editor::DataStorage::FHierarchyHandle SettingsHierarchy;
	TMap<FName, UE::Editor::DataStorage::QueryHandle> SelectAllSettingsInModuleQueryMap;
};
