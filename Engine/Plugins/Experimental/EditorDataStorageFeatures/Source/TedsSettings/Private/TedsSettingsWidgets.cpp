// Copyright Epic Games, Inc. All Rights Reserved.

#include "TedsSettingsWidgets.h"

#include "Elements/Columns/TypedElementMiscColumns.h"
#include "Elements/Columns/TypedElementSlateWidgetColumns.h"
#include "Elements/Framework/TypedElementAttributeBinding.h"
#include "TedsSettingsColumns.h"
#include "Widgets/SToolTip.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Text/STextBlock.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TedsSettingsWidgets)

FSettingsContainerReferenceWidgetConstructor::FSettingsContainerReferenceWidgetConstructor()
	: FSimpleWidgetConstructor(FSettingsContainerReferenceWidgetConstructor::StaticStruct())
{
}

TSharedPtr<SWidget> FSettingsContainerReferenceWidgetConstructor::CreateWidget(
	UE::Editor::DataStorage::ICoreProvider* DataStorage,
	UE::Editor::DataStorage::IUiProvider* DataStorageUi,
	UE::Editor::DataStorage::RowHandle TargetRow,
	UE::Editor::DataStorage::RowHandle WidgetRow,
	const UE::Editor::DataStorage::FMetaDataView& Arguments)
{
	using namespace UE::Editor::DataStorage;

	if (DataStorage->IsRowAvailable(TargetRow))
	{
		RowHandle ContainerRow = DataStorage->GetColumn<FSettingsContainerReferenceColumn>(TargetRow)->ContainerRow;

		if (DataStorage->IsRowAvailable(ContainerRow))
		{
			FAttributeBinder Binder(ContainerRow, DataStorage);

			return SNew(SBox)
				.Padding(4.0f, 0.0f)
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
						.Text(Binder.BindData(&FDisplayNameColumn::DisplayName))
						.ToolTip(SNew(SToolTip).Text(Binder.BindData(&FDescriptionColumn::Description)))
				];
		}
		else // Fallback to render just the FName if we can't follow the reference
		{
			FAttributeBinder Binder(TargetRow, DataStorage);

			return SNew(SBox)
				.Padding(4.0f, 0.0f)
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
						.ColorAndOpacity(FSlateColor::UseSubduedForeground())
						.Text(Binder.BindText(&FSettingsContainerReferenceColumn::ContainerName))
				];
		}
	}

	return SNullWidget::NullWidget;
}

FText FSettingsContainerReferenceWidgetConstructor::CreateWidgetDisplayNameText(
	UE::Editor::DataStorage::ICoreProvider* DataStorage,
	UE::Editor::DataStorage::RowHandle Row) const
{
	return NSLOCTEXT("TedsSettingsWidgets", "SettingsContainerReferenceWidgetDisplayNameText", "Settings Container");
}

FSettingsCategoryReferenceWidgetConstructor::FSettingsCategoryReferenceWidgetConstructor()
	: FSimpleWidgetConstructor(FSettingsCategoryReferenceWidgetConstructor::StaticStruct())
{
}

TSharedPtr<SWidget> FSettingsCategoryReferenceWidgetConstructor::CreateWidget(
	UE::Editor::DataStorage::ICoreProvider* DataStorage,
	UE::Editor::DataStorage::IUiProvider* DataStorageUi,
	UE::Editor::DataStorage::RowHandle TargetRow,
	UE::Editor::DataStorage::RowHandle WidgetRow,
	const UE::Editor::DataStorage::FMetaDataView& Arguments)
{
	using namespace UE::Editor::DataStorage;

	if (DataStorage->IsRowAvailable(TargetRow))
	{
		RowHandle CategoryRow = DataStorage->GetColumn<FSettingsCategoryReferenceColumn>(TargetRow)->CategoryRow;

		if (DataStorage->IsRowAvailable(CategoryRow))
		{
			FAttributeBinder Binder(CategoryRow, DataStorage);

			return SNew(SBox)
				.Padding(4.0f, 0.0f)
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
						.Text(Binder.BindData(&FDisplayNameColumn::DisplayName))
						.ToolTip(SNew(SToolTip).Text(Binder.BindData(&FDescriptionColumn::Description)))
				];
		}
		else // Fallback to render just the FName if we can't follow the reference
		{
			FAttributeBinder Binder(TargetRow, DataStorage);

			return SNew(SBox)
				.Padding(4.0f, 0.0f)
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
						.ColorAndOpacity(FSlateColor::UseSubduedForeground())
						.Text(Binder.BindText(&FSettingsCategoryReferenceColumn::CategoryName))
				];
		}
	}

	return SNullWidget::NullWidget;
}

FText FSettingsCategoryReferenceWidgetConstructor::CreateWidgetDisplayNameText(
	UE::Editor::DataStorage::ICoreProvider* DataStorage,
	UE::Editor::DataStorage::RowHandle Row) const
{
	return NSLOCTEXT("TedsSettingsWidgets", "SettingsCategoryReferenceWidgetDisplayNameText", "Settings Category");
}

FSettingsSectionWidgetConstructor::FSettingsSectionWidgetConstructor()
	: FSimpleWidgetConstructor(FSettingsSectionWidgetConstructor::StaticStruct())
{
}

TSharedPtr<SWidget> FSettingsSectionWidgetConstructor::CreateWidget(
	UE::Editor::DataStorage::ICoreProvider* DataStorage,
	UE::Editor::DataStorage::IUiProvider* DataStorageUi,
	UE::Editor::DataStorage::RowHandle TargetRow,
	UE::Editor::DataStorage::RowHandle WidgetRow,
	const UE::Editor::DataStorage::FMetaDataView& Arguments)
{
	using namespace UE::Editor::DataStorage;

	if (DataStorage->IsRowAvailable(TargetRow))
	{
		FAttributeBinder Binder(TargetRow, DataStorage);

		return SNew(SBox)
			.Padding(4.0f, 0.0f)
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
					.Text(Binder.BindData(&FDisplayNameColumn::DisplayName))
					.ToolTip(SNew(SToolTip).Text(Binder.BindData(&FDescriptionColumn::Description)))
			];
	}

	return SNullWidget::NullWidget;
}
