// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/Interfaces/TypedElementDataStorageInterface.h"
#include "UObject/ObjectMacros.h"

#include "TedsSettingsColumns.generated.h"

namespace UE::Editor::DataStorage::Settings
{
	inline static const FName MappingDomain = "Settings";
	inline static const FName HierarchyName = "SettingsHierarchy";
}

USTRUCT(meta = (DisplayName = "Settings Container Reference"))
struct FSettingsContainerReferenceColumn final : public FEditorDataStorageColumn
{
	GENERATED_BODY()

	// Store the name as it is frequently accessed to avoid indirection to the row.
	UPROPERTY(meta = (Searchable, Sortable))
	FName ContainerName;

	UE::Editor::DataStorage::RowHandle ContainerRow;
};

USTRUCT(meta = (DisplayName = "Settings Category Reference"))
struct FSettingsCategoryReferenceColumn final : public FEditorDataStorageColumn
{
	GENERATED_BODY()

	// Store the name as it is frequently accessed to avoid indirection to the row.
	UPROPERTY(meta = (Searchable, Sortable))
	FName CategoryName;

	UE::Editor::DataStorage::RowHandle CategoryRow;
};

// A generic name column for use within the settings domain.
// This is added to all container, category, and section rows.
USTRUCT(meta = (DisplayName = "Name"))
struct FSettingsNameColumn final : public FEditorDataStorageColumn
{
	GENERATED_BODY()

	UPROPERTY(meta = (Searchable, Sortable))
	FName Name;
};

// A column whose dynamic identifier specifies the module that contains the class of this row's settings section object.
USTRUCT(meta = (DisplayName = "Settings Module", EditorDataStorage_DynamicColumnTemplate))
struct FSettingsModuleTag final : public FEditorDataStorageTag
{
	GENERATED_BODY()
};

USTRUCT(meta = (DisplayName = "Settings Container"))
struct FSettingsContainerTag final : public FEditorDataStorageTag
{
	GENERATED_BODY()
};

USTRUCT(meta = (DisplayName = "Settings Category"))
struct FSettingsCategoryTag final : public FEditorDataStorageTag
{
	GENERATED_BODY()
};

USTRUCT(meta = (DisplayName = "Settings Section"))
struct FSettingsSectionTag final : public FEditorDataStorageTag
{
	GENERATED_BODY()
};

USTRUCT(meta = (DisplayName = "Settings Property"))
struct FSettingsPropertyTag final : public FEditorDataStorageTag
{
	GENERATED_BODY()
};
