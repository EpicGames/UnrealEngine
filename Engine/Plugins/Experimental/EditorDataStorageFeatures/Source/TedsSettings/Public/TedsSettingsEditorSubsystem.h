// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EditorSubsystem.h"
#include "Elements/Interfaces/TypedElementDataStorageInterface.h"

#include "TedsSettingsEditorSubsystem.generated.h"

#define UE_API TEDSSETTINGS_API

class FTedsSettingsManager;

UCLASS(MinimalAPI)
class UTedsSettingsEditorSubsystem : public UEditorSubsystem
{
	GENERATED_BODY()

public:

	UE_API UTedsSettingsEditorSubsystem();

	UE_API const bool IsEnabled() const;

	DECLARE_MULTICAST_DELEGATE(FOnEnabledChanged)
	UE_API FOnEnabledChanged& OnEnabledChanged();

	/**
	 * Finds an existing container row.
	 *
	 * @param	ContainerName	The ContainerName to search for.
	 *
	 * @return	The found row or InvalidRowHandle if no existing row is found.
	 */
	UE_API UE::Editor::DataStorage::RowHandle FindSettingsContainer(const FName& ContainerName) const;

	/**
	 * Finds an existing category row.
	 *
	 * @param	ContainerName	The ContainerName to search for.
	 * @param	CategoryName	The CategoryName to search for.
	 *
	 * @return	The found row or InvalidRowHandle if no existing row is found.
	 */
	UE_API UE::Editor::DataStorage::RowHandle FindSettingsCategory(const FName& ContainerName, const FName& CategoryName) const;

	/** 
	 * Finds an existing section row.
	 * 
	 * @param	ContainerName	The ContainerName to search for.
	 * @param	CategoryName	The CategoryName to search for.
	 * @param	SectionName		The SectionName to search for.
	 * 
	 * @return	The found row or InvalidRowHandle if no existing row is found.
	 */
	UE_API UE::Editor::DataStorage::RowHandle FindSettingsSection(const FName& ContainerName, const FName& CategoryName, const FName& SectionName) const;

	/**
	 * Gets the settings section details for the given section row.
	 *
	 * @param	SectionRow			The section row from which to read settings details.
	 * @param	OutContainerName	Reference to the ContainerName for the given row, only set if this method returns true.
	 * @param	OutCategoryName		Reference to the CategoryName for the given row, only set if this method returns true.
	 * @param	OutSectionName		Reference to the SectionName for the given row, only set if this method returns true.
	 *
	 * @return	True if the settings section details are successfully returned in the out parameters.
	 */
	UE_API bool GetSettingsSectionFromRow(UE::Editor::DataStorage::RowHandle SectionRow, FName& OutContainerName, FName& OutCategoryName, FName& OutSectionName);

	/**
	 * Gets the settings category details for the given category row.
	 *
	 * @param	CategoryRow			The category row from which to read settings details.
	 * @param	OutContainerName	Reference to the ContainerName for the given row, only set if this method returns true.
	 * @param	OutCategoryName		Reference to the CategoryName for the given row, only set if this method returns true.
	 *
	 * @return	True if the settings category details are successfully returned in the out parameters.
	 */
	UE_API bool GetSettingsCategoryFromRow(UE::Editor::DataStorage::RowHandle CategoryRow, FName& OutContainerName, FName& OutCategoryName);

protected: // USubsystem

	UE_API virtual void Initialize(FSubsystemCollectionBase& Collection) override;
	UE_API virtual void Deinitialize() override;

private:

	TSharedPtr<FTedsSettingsManager> SettingsManager;
	FOnEnabledChanged EnabledChangedDelegate;

};

#undef UE_API
