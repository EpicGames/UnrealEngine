// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class TedsSettings : ModuleRules
{
	public TedsSettings(ReadOnlyTargetRules Target) : base(Target)
	{
		// Enable truncation warnings in this module
		CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Warning;

		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		if (Target.bBuildEditor)
		{
			PublicIncludePaths.AddRange(new string[] { });
			PrivateIncludePaths.AddRange(new string[] { });

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
					"DeveloperSettings",
					"Engine",
					"Settings",
					"SettingsEditor",
					"Slate",
					"SlateCore",
					"UnrealEd",
				});

			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"EditorSubsystem",
					"TypedElementFramework",
				});
		}
	}
}
