// Copyright Epic Games, Inc. All Rights Reserved.

#include "Misc/AutomationTest.h"

#if WITH_AUTOMATION_TESTS

#include "AutoRTFM/Defines.h"
#include "Containers/Ticker.h"
#include "DataStorage/Features.h"
#include "Editor.h"
#include "Elements/Columns/TypedElementCompatibilityColumns.h"
#include "Elements/Columns/TypedElementMiscColumns.h"
#include "Elements/Framework/TypedElementQueryBuilder.h"
#include "Elements/Interfaces/TypedElementDataStorageInterface.h"
#include "ISettingsModule.h"
#include "Modules/ModuleManager.h"
#include "TedsSettingsColumns.h"
#include "TedsSettingsEditorSubsystem.h"
#include "TestSettings.h"

namespace UE::Editor::Settings::Tests
{
	BEGIN_DEFINE_SPEC(FTedsSettingsTestFixture, "Editor.DataStorage.Settings", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
	ISettingsModule* SettingsModule = nullptr;
	DataStorage::ICoreProvider* DataStorage = nullptr;
	DataStorage::ICompatibilityProvider* DataStorageCompatibility = nullptr;
	DataStorage::QueryHandle CountAllSettingsQuery = DataStorage::InvalidQueryHandle;

	uint32 BeforeRowCount = 0;
	TArray<DataStorage::RowHandle> TestRowHandles{};

	uint32 CountSettingsRowsInDataStorage()
	{
		DataStorage::FQueryResult Result = DataStorage->RunQuery(CountAllSettingsQuery);

		return Result.Count;
	}

	template<typename Func>
	AUTORTFM_DISABLE void AwaitRowHandleThenVerify(DataStorage::RowHandle RowHandle, const FDoneDelegate& Done, Func&& OnVerify)
	{
		auto OnTick = [this, RowHandle, Done, OnVerify = Forward<Func>(OnVerify)](float FrameTime)
		{
			if (DataStorage->IsRowAssigned(RowHandle))
			{
				ON_SCOPE_EXIT{ Done.Execute(); };

				OnVerify();

				return false;
			}
			return true;
		};

		FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateLambda(OnTick));
	}

	END_DEFINE_SPEC(FTedsSettingsTestFixture)

	void FTedsSettingsTestFixture::Define()
	{
		check(GEditor);
		UTedsSettingsEditorSubsystem* SettingsEditorSubsystem = GEditor->GetEditorSubsystem<UTedsSettingsEditorSubsystem>();

		SettingsEditorSubsystem->OnEnabledChanged().RemoveAll(this);
		SettingsEditorSubsystem->OnEnabledChanged().AddRaw(this, &FTedsSettingsTestFixture::Redefine);

		if (!SettingsEditorSubsystem->IsEnabled())
		{
			return;
		}

		BeforeEach([this]()
		{
			using namespace UE::Editor::DataStorage;
			SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings");
			check(SettingsModule != nullptr);

			DataStorage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName);
			check(DataStorage != nullptr);

			DataStorageCompatibility = GetMutableDataStorageFeature<ICompatibilityProvider>(CompatibilityFeatureName);
			check(DataStorageCompatibility != nullptr);

			{
				using namespace UE::Editor::DataStorage::Queries;
				CountAllSettingsQuery = DataStorage->RegisterQuery(
					Count()
					.Where()
						.Any<FSettingsSectionTag>()
					.Compile());
			}

			BeforeRowCount = CountSettingsRowsInDataStorage();
		});

		AfterEach([this]()
		{
			for (DataStorage::RowHandle RowHandle : TestRowHandles)
			{
				DataStorage->RemoveRow(RowHandle);
			}
			TestRowHandles.Empty();
			CountAllSettingsQuery = DataStorage::InvalidQueryHandle;
			SettingsModule = nullptr;
			DataStorage = nullptr;
			DataStorageCompatibility = nullptr;
		});

		Describe("RegisterSettings", [this]()
		{
			LatentIt("Should add a row to editor data storage", [this](const FDoneDelegate& Done)
			{
				const FName& ContainerName = FName(TEXT("TestContainer"));
				const FName& CategoryName = FName(TEXT("TestCategory"));
				const FName& SectionName = FName(TEXT("TestSection"));

				TObjectPtr<UTestSettings> TestSettingsObject = NewObject<UTestSettings>();

				SettingsModule->RegisterSettings(ContainerName, CategoryName, SectionName, FText(), FText(), TestSettingsObject);

				DataStorage::RowHandle RowHandle = DataStorageCompatibility->FindRowWithCompatibleObject(TestSettingsObject);
				TestNotEqual(TEXT("RowHandle"), RowHandle, DataStorage::InvalidRowHandle);

				if (RowHandle == DataStorage::InvalidRowHandle)
				{
					Done.Execute();
					return;
				}

				TestRowHandles.Push(RowHandle);

				AwaitRowHandleThenVerify(RowHandle, Done, [this, RowHandle, ContainerName, CategoryName, SectionName]()
				{
					uint32 AfterRowCount = CountSettingsRowsInDataStorage();

					TestEqual(TEXT("RowCount"), AfterRowCount, BeforeRowCount + 1);

					TestEqual(TEXT("ContainerName"), DataStorage->GetColumn<FSettingsContainerReferenceColumn>(RowHandle)->ContainerName, ContainerName);
					TestEqual(TEXT("CategoryName"), DataStorage->GetColumn<FSettingsCategoryReferenceColumn>(RowHandle)->CategoryName, CategoryName);
					TestEqual(TEXT("SectionName"), DataStorage->GetColumn<FSettingsNameColumn>(RowHandle)->Name, SectionName);
				});
			});
		});

		Describe("UnregisterSettings", [this]()
		{
			LatentIt("Should remove a row from editor data storage", [this](const FDoneDelegate& Done)
			{
				const FName& ContainerName = FName(TEXT("TestContainer"));
				const FName& CategoryName = FName(TEXT("TestCategory"));
				const FName& SectionName = FName(TEXT("TestSection"));

				TObjectPtr<UTestSettings> TestSettingsObject = NewObject<UTestSettings>();

				SettingsModule->RegisterSettings(ContainerName, CategoryName, SectionName, FText(), FText(), TestSettingsObject);

				DataStorage::RowHandle RowHandle = DataStorageCompatibility->FindRowWithCompatibleObject(TestSettingsObject);
				TestNotEqual(TEXT("RowHandle"), RowHandle, DataStorage::InvalidRowHandle);

				if (RowHandle == DataStorage::InvalidRowHandle)
				{
					Done.Execute();
					return;
				}

				TestRowHandles.Push(RowHandle);

				AwaitRowHandleThenVerify(RowHandle, Done, [this, RowHandle, TestSettingsObject, ContainerName, CategoryName, SectionName]()
				{
					uint32 AfterRegisterRowCount = CountSettingsRowsInDataStorage();

					TestEqual(TEXT("RowCount"), AfterRegisterRowCount, BeforeRowCount + 1);

					SettingsModule->UnregisterSettings(ContainerName, CategoryName, SectionName);

					uint32 AfterUnregisterRowCount = CountSettingsRowsInDataStorage();
					TestEqual(TEXT("RowCount"), AfterUnregisterRowCount, BeforeRowCount);

					TestFalse(TEXT("IsRowAssigned"), DataStorage->IsRowAssigned(RowHandle));

					DataStorage::RowHandle TestInvalidRowHandle = DataStorageCompatibility->FindRowWithCompatibleObject(TestSettingsObject);
					TestEqual(TEXT("InvalidRowHandle"), TestInvalidRowHandle, DataStorage::InvalidRowHandle);

					UTedsSettingsEditorSubsystem* SettingsEditorSubsystem = GEditor->GetEditorSubsystem<UTedsSettingsEditorSubsystem>();

					TestInvalidRowHandle = SettingsEditorSubsystem->FindSettingsSection(ContainerName, CategoryName, SectionName);
					TestEqual(TEXT("InvalidRowHandle"), TestInvalidRowHandle, DataStorage::InvalidRowHandle);

					uint32 AfterFindOrAddRowCount = CountSettingsRowsInDataStorage();
					TestEqual(TEXT("RowCount"), AfterFindOrAddRowCount, BeforeRowCount);
				});
			});
		});

		Describe("RegisterSettings same container/category/section twice with different UObjects", [this]()
		{
			LatentIt("Should result in only a single row in editor data storage", [this](const FDoneDelegate& Done)
			{
				const FName& ContainerName = FName(TEXT("TestContainer"));
				const FName& CategoryName = FName(TEXT("TestCategory"));
				const FName& SectionName = FName(TEXT("TestSection"));

				TObjectPtr<UTestSettings> TestSettingsObject1 = NewObject<UTestSettings>();
				TObjectPtr<UTestSettings> TestSettingsObject2 = NewObject<UTestSettings>();

				SettingsModule->RegisterSettings(ContainerName, CategoryName, SectionName, FText(), FText(), TestSettingsObject1);
				SettingsModule->RegisterSettings(ContainerName, CategoryName, SectionName, FText(), FText(), TestSettingsObject2);

				DataStorage::RowHandle RowHandle = DataStorageCompatibility->FindRowWithCompatibleObject(TestSettingsObject1);
				TestEqual(TEXT("RowHandle"), RowHandle, DataStorage::InvalidRowHandle);

				if (RowHandle != DataStorage::InvalidRowHandle)
				{
					TestRowHandles.Push(RowHandle);
				}

				RowHandle = DataStorageCompatibility->FindRowWithCompatibleObject(TestSettingsObject2);
				TestNotEqual(TEXT("RowHandle"), RowHandle, DataStorage::InvalidRowHandle);

				if (RowHandle == DataStorage::InvalidRowHandle)
				{
					Done.Execute();
					return;
				}

				TestRowHandles.Push(RowHandle);

				AwaitRowHandleThenVerify(RowHandle, Done, [this, RowHandle, ContainerName, CategoryName, SectionName]()
				{
					uint32 AfterRowCount = CountSettingsRowsInDataStorage();

					TestEqual(TEXT("RowCount"), AfterRowCount, BeforeRowCount + 1);

					TestEqual(TEXT("ContainerName"), DataStorage->GetColumn<FSettingsContainerReferenceColumn>(RowHandle)->ContainerName, ContainerName);
					TestEqual(TEXT("CategoryName"), DataStorage->GetColumn<FSettingsCategoryReferenceColumn>(RowHandle)->CategoryName, CategoryName);
					TestEqual(TEXT("SectionName"), DataStorage->GetColumn<FSettingsNameColumn>(RowHandle)->Name, SectionName);
				});
			});
		});
	}
} // namespace UE::Editor::Settings::Tests

#endif // WITH_AUTOMATION_TESTS
