// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/ObjectMacros.h"

#include "TestSettings.generated.h"

UCLASS()
class UTestSettings : public UObject
{
	GENERATED_BODY()

public:

	UPROPERTY()
	int32 TestInt32 = 42;

};
