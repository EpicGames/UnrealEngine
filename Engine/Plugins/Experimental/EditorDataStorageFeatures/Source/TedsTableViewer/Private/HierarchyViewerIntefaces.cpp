// Copyright Epic Games, Inc. All Rights Reserved.

#include "HierarchyViewerIntefaces.h"

#include "DataStorage/Features.h"
#include "Elements/Framework/TypedElementQueryBuilder.h"

namespace UE::Editor::DataStorage
{
	FHierarchyViewerData::FHierarchyViewerData(ICoreProvider& Storage, FHierarchyHandle InHierarchyHandle)
		: HierarchyHandle(InHierarchyHandle)
	{
		RegisterQueries(Storage);
	}

	FHierarchyViewerData::~FHierarchyViewerData()
	{
		if (ICoreProvider* Storage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName))
		{
			UnRegisterQueries(*Storage);
		}
	}

	RowHandle FHierarchyViewerData::GetParent(const ICoreProvider& Storage, RowHandle InRow) const
	{
		return Storage.GetParentRow(HierarchyHandle, InRow);
	}

	void FHierarchyViewerData::WalkDepthFirst(const ICoreProvider& Storage, RowHandle InRow, ICoreProvider::FHierarchyIterationCallback VisitFn,
		ICoreProvider::ETraversalOrder TraversalOrder) const
	{
		Storage.WalkDepthFirst(HierarchyHandle, InRow, VisitFn, TraversalOrder);
	}

	FHierarchyHandle FHierarchyViewerData::GetHierarchy() const
	{
		return HierarchyHandle;
	}

	void FHierarchyViewerData::RegisterQueries(ICoreProvider& Storage)
	{
		using namespace UE::Editor::DataStorage::Queries;
		
		if (const UScriptStruct* ParentChangedColumn = Storage.GetParentChangedColumnType(HierarchyHandle))
		{
			ParentAddedQuery =
				Storage.RegisterQuery(
					Select(
						TEXT("On parent added to row"),
						FObserver(FObserver::EEvent::Add, ParentChangedColumn).SetExecutionMode(EExecutionMode::GameThread),
						[this](IQueryContext& Context, RowHandle Row)
						{
								ParentChangedEvent.Broadcast(Row);
						})
					.Compile());

			ParentRemovedQuery =
				Storage.RegisterQuery(
					Select(
						TEXT("On parent removed from row"),
						FObserver(FObserver::EEvent::Remove, ParentChangedColumn).SetExecutionMode(EExecutionMode::GameThread),
						[this](IQueryContext& Context, RowHandle Row)
						{
								ParentChangedEvent.Broadcast(Row);
						})
					.Compile());
		}
	}

	void FHierarchyViewerData::UnRegisterQueries(ICoreProvider& Storage)
	{
		Storage.UnregisterQuery(ParentAddedQuery);
		Storage.UnregisterQuery(ParentRemovedQuery);
	}

	FHierarchyViewerMultiData::FHierarchyViewerMultiData(ICoreProvider& Storage, TArray<FHierarchyHandle> OrderedHierarchyHandles)
	{
		for (FHierarchyHandle Handle : OrderedHierarchyHandles)
		{  
			TSharedRef<FHierarchyViewerData> Data = MakeShared<FHierarchyViewerData>(Storage, Handle);
			Data->OnHierarchyChangedEvent().AddLambda([this](RowHandle Row)
				{
					OnHierarchyChangedEvent().Broadcast(Row);
				});

			Hierarchies.Add(MoveTemp(Data));
		}
	}

	RowHandle FHierarchyViewerMultiData::GetParent(const ICoreProvider& Storage, RowHandle InRow) const
	{
		for (const TSharedPtr<FHierarchyViewerData>& Data : Hierarchies)
		{
			RowHandle Parent = Data->GetParent(Storage, InRow);

			if (Storage.IsRowAvailable(Parent))
			{
				return Parent;
			}
		}

		return InvalidRowHandle;
	}

	void FHierarchyViewerMultiData::WalkDepthFirst(const ICoreProvider& Storage, RowHandle InRow,
		ICoreProvider::FHierarchyIterationCallback VisitFn, ICoreProvider::ETraversalOrder TraversalOrder) const
	{
		TSet<RowHandle> Visited;
		
		for (const TSharedPtr<FHierarchyViewerData>& Data : Hierarchies)
		{
			Data->WalkDepthFirst(Storage, InRow, [&Visited, VisitFn](const ICoreProvider& Context, RowHandle Owner, RowHandle Target)
			{
				if (!Visited.Contains(Target))
				{
					VisitFn(Context, Owner, Target);
					Visited.Add(Target);
				}
			}, TraversalOrder);
		}
	}

	void FHierarchyViewerMultiData::ForEachHierarchyData(TFunctionRef<bool(const TSharedRef<FHierarchyViewerData>&)> VisitFn) const
	{
		for (const TSharedPtr<FHierarchyViewerData>& Data : Hierarchies)
		{
			if (Data)
			{
				if (!VisitFn(Data.ToSharedRef()))
				{
					return;
				}
			}
		}
	}

	FRelationTypeHierarchyViewerData::FRelationTypeHierarchyViewerData(RelationTypeHandle InRelationType)
		: RelationType(InRelationType)
	{
	}

	RowHandle FRelationTypeHierarchyViewerData::GetParent(const ICoreProvider& Storage, RowHandle InRow) const
	{
		return Storage.GetRelationObject(RelationType, InRow);
	}

	void FRelationTypeHierarchyViewerData::WalkDepthFirst(const ICoreProvider& Storage, RowHandle InRow,
		ICoreProvider::FHierarchyIterationCallback VisitFn, ICoreProvider::ETraversalOrder TraversalOrder) const
	{
		// TraverseDescendants callback: (Current, Parent, Depth)
		// FHierarchyIterationCallback:  (Context, Owner, Target) where Owner=Parent, Target=Current
		Storage.TraverseDescendants(RelationType, InRow,
			[&Storage, &VisitFn](RowHandle Current, RowHandle Parent, int32 /*Depth*/)
			{
				VisitFn(Storage, Parent, Current);
			},
			TraversalOrder);
	}
}
