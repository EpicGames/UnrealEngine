// Copyright Epic Games, Inc. All Rights Reserved.

#include "QueryStack/AddParentHierarchyRowsNode.h"

#include "HierarchyViewerIntefaces.h"

namespace UE::Editor::DataStorage::QueryStack
{
	FAddParentHierarchyRowNode::FAddParentHierarchyRowNode(const ICoreProvider* InStorage, TSharedPtr<IHierarchyViewerDataInterface> InHierarchyData,
		TSharedPtr<IRowNode> InParent)
			: Storage(InStorage)
			, HierarchyData(MoveTemp(InHierarchyData))
			, Parent(MoveTemp(InParent))
	{
		checkf(HierarchyData, TEXT("This node requires a valid hierarchy to view"));
		UpdateRows();
	}

	FRowHandleArrayView FAddParentHierarchyRowNode::GetRows() const
	{
		return Rows.GetRows();
	}

	FRowHandleArray& FAddParentHierarchyRowNode::GetMutableRows()
	{
		return Rows;
	}

	INode::RevisionId FAddParentHierarchyRowNode::GetRevision() const
	{
		return Revision;
	}

	void FAddParentHierarchyRowNode::Update(FTimespan AllottedTime)
	{
		if (Parent->GetRevision() != CachedParentRevision)
		{
			UpdateRows();
			CachedParentRevision = Parent->GetRevision();
			Revision++;
		}
	}

	void FAddParentHierarchyRowNode::VisitParents(ParentListCallback Callback)
	{
		Callback(Parent);
	}

	void FAddParentHierarchyRowNode::UpdateRows()
	{
		TRACE_CPUPROFILER_EVENT_SCOPE_TEXT(TEXT("[TEDS] FAddParentHierarchyRowNode Update"));
		const FRowHandleArrayView ParentRows = Parent->GetRows();
		
		Rows.Empty();
		Rows.Append(ParentRows);
		Rows.Sort();

		for (const RowHandle Row : ParentRows)
		{
			RowHandle ParentRow = HierarchyData->GetParent(*Storage, Row);
			while (ParentRow != InvalidRowHandle)
			{
				// Since the Rows are sorted, the search for contains will use a binary search
				if(!Rows.Contains(ParentRow))
				{
					Rows.Add(ParentRow);
					ParentRow = HierarchyData->GetParent(*Storage, ParentRow);
				}
				else
				{
					// Break if the Rows already contain the parent row as that means it has already been searched or it will be in the for-loop
					break;
				}
			}
		}
	}
}
