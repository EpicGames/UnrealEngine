// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "TedsQueryStackInterfaces.h"
#include "Elements/Interfaces/TypedElementDataStorageInterface.h"

#define UE_API TEDSTABLEVIEWER_API

namespace UE::Editor::DataStorage
{
	class IHierarchyViewerDataInterface;
}

namespace UE::Editor::DataStorage::QueryStack
{
	// A custom query stack node to add any missing parents of the given node rows in the hierarchy viewer
	// Ex: Filter matches child row and moves that child row to the outer hierarchy level since it doesn't have a parent
	class FAddParentHierarchyRowNode : public IRowNode
	{
	public:
		
		UE_API FAddParentHierarchyRowNode(const ICoreProvider* InStorage, TSharedPtr<IHierarchyViewerDataInterface> InHierarchyData,
			TSharedPtr<IRowNode> InParent);
		virtual ~FAddParentHierarchyRowNode() override = default;
		
		/** Retrieve access to the rows used by this node. */
		virtual FRowHandleArrayView GetRows() const override;
		virtual FRowHandleArray& GetMutableRows() override;
		virtual RevisionId GetRevision() const override;
		virtual void VisitParents(ParentListCallback Callback) override;
		
	protected:
		virtual void Update(FTimespan AllottedTime) override;
		void UpdateRows();
	protected:
		RevisionId Revision = UninitializedRevisionId;
		const ICoreProvider* Storage = nullptr;
		TSharedPtr<IHierarchyViewerDataInterface> HierarchyData;
		TSharedPtr<IRowNode> Parent;
		FRowHandleArray Rows;
		RevisionId CachedParentRevision = UninitializedRevisionId;
	};
}

#undef UE_API