// Copyright Epic Games, Inc. All Rights Reserved.

#include "QueryStack/TopLevelRowsNode.h"

#include "HierarchyViewerIntefaces.h"

namespace UE::Editor::DataStorage::QueryStack
{
	FTopLevelRowsNode::FTopLevelRowsNode(const ICoreProvider* InStorage, TSharedPtr<IHierarchyViewerDataInterface> InHierarchyData, TSharedPtr<IRowNode> InParent)
			: Storage(InStorage)
			, HierarchyData(MoveTemp(InHierarchyData))
			, Parent(MoveTemp(InParent))
	{
		checkf(HierarchyData, TEXT("This node requires a valid hierarchy to view"))
		UpdateRows();
	}

	FRowHandleArrayView FTopLevelRowsNode::GetRows() const
	{
		return Rows.GetRows();
	}

	FRowHandleArray& FTopLevelRowsNode::GetMutableRows()
	{
		return Rows;
	}

	INode::RevisionId FTopLevelRowsNode::GetRevision() const
	{
		return Revision;
	}

	void FTopLevelRowsNode::Update(FTimespan AllottedTime)
	{
		if (Parent->GetRevision() != CachedParentRevision)
		{
			UpdateRows();
			CachedParentRevision = Parent->GetRevision();
			Revision++;
		}
	}

	void FTopLevelRowsNode::VisitParents(ParentListCallback Callback)
	{
		Callback(Parent);
	}

	void FTopLevelRowsNode::UpdateRows()
	{
		Rows.Empty();
		FRowHandleArray RowsToCheck(Parent->GetRows());
		RowsToCheck.Sort();
		
		for (RowHandle Row : Parent->GetRows())
		{
			RowHandle ParentRow = HierarchyData->GetParent(*Storage, Row);
			
			// If the row doesn't have a parent, it is a top level node and can be added
			if (Storage->IsRowAvailable(ParentRow))
			{
				// If the filtered list contains the parent of the row, it has a visible parent and thus is not a top-level node
				if(!RowsToCheck.Contains(ParentRow))
				{
					Rows.Add(Row);
				}
			}
			else
			{
				Rows.Add(Row);
			}
		}
	}
}
