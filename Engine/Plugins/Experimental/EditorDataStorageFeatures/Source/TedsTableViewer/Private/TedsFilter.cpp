// Copyright Epic Games, Inc. All Rights Reserved.

#include "TedsFilter.h"

#include "Widgets/STedsFilterBar.h"

namespace UE::Editor::DataStorage
{
	class STedsFilterBar;

	void FTedsFilter::SetTedsFilterBar(const TSharedPtr<STedsFilterBar>& InTedsFilterBar)
	{
		TedsFilterBar = InTedsFilterBar;
		if(IsActiveByDefault())
		{
			ActiveStateChanged(true);
		}
	}

	void FTedsFilter::ActiveStateChanged(bool bActive)
	{
		using namespace UE::Editor::DataStorage::Queries;
		
		const TSharedPtr<STedsFilterBar> TedsFilterBarPin = TedsFilterBar.Pin();
		if (ensureMsgf(TedsFilterBarPin, TEXT("No TedsFilterBar Context was set for the %s filter."), *FilterDisplayName.ToString()))
		{
			if(bActive)
			{
				if (bIsClassFilter)
				{
					// Class filters are separated since we want to OR them with each other
					TedsFilterBarPin->AddClassQueryFunction(FilterName, FilterQuery.Get<TConstQueryFunction<bool>>());
				}
				else if (FilterQuery.IsType<QueryHandle>())
				{
					TedsFilterBarPin->AddExternalQuery(FilterName, FilterQuery.Get<QueryHandle>());
				}
				else /* if (FilterQuery.IsType<TConstQueryFunction<bool>>()) */
				{
					TedsFilterBarPin->AddExternalQueryFunction(FilterName, FilterQuery.Get<TConstQueryFunction<bool>>());
				}
			}
			else
			{
				if (bIsClassFilter)
				{
					TedsFilterBarPin->RemoveClassQueryFunction(FilterName);
				}
				else if (FilterQuery.IsType<QueryHandle>())
				{
					TedsFilterBarPin->RemoveExternalQuery(FilterName);
				}
				else /* if (FilterQuery.IsType<TQueryFunction<bool>>()) */
				{
					TedsFilterBarPin->RemoveExternalQueryFunction(FilterName);
				}
			}
		}
	}

	bool FTedsFilter::PassesFilter(FTedsRowHandle& InItem) const
	{
		// The filter is applied through a TEDS query and this is just a dummy to activate it, so we can just run a simple valid check on if the
		// TEDSFilterBar is still valid.
		return TedsFilterBar.IsValid();
	}
} // namespace UE::Editor::DataStorage
