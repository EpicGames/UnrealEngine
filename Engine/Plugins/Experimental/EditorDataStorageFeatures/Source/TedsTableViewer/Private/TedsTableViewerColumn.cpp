// Copyright Epic Games, Inc. All Rights Reserved.

#include "TedsTableViewerColumn.h"

#include "EditorAnalytics.h"
#include "Columns/SlateHeaderColumns.h"
#include "DataStorage/Features.h"
#include "Elements/Columns/TypedElementMiscColumns.h"
#include "Elements/Columns/TypedElementSlateWidgetColumns.h"
#include "Elements/Columns/TypedElementTypeInfoColumns.h"
#include "Elements/Interfaces/TypedElementDataStorageInterface.h"
#include "Elements/Interfaces/TypedElementDataStorageCompatibilityInterface.h"
#include "Elements/Interfaces/TypedElementDataStorageUiInterface.h"
#include "Elements/Framework/TypedElementDataStorageWidget.h"
#include "Elements/Framework/TypedElementRegistry.h"
#include "TedsTableViewerUtils.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

namespace UE::Editor::DataStorage
{
	FTedsTableViewerColumn::FTedsTableViewerColumn(const FName& InColumnName, const IUiProvider::FWidgetConstructorPtr& InCellWidgetConstructor,
		const TArray<TWeakObjectPtr<const UScriptStruct>>& InMatchedColumns, const IUiProvider::FWidgetConstructorPtr& InHeaderWidgetConstructor,
		const FMetaData& InWidgetMetaData, const TArray<TSharedPtr<FColumnMetaData>>& InColumnMetaData)
		: ColumnName(InColumnName)
		, CellWidgetConstructor(InCellWidgetConstructor)
		, HeaderWidgetConstructor(InHeaderWidgetConstructor)
		, MatchedColumns(InMatchedColumns)
		, WidgetMetaData(InWidgetMetaData)
		, ColumnMetaData(InColumnMetaData)
	{
		Storage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName);
		StorageUi = GetMutableDataStorageFeature<IUiProvider>(UiFeatureName);
		StorageCompatibility = GetMutableDataStorageFeature<ICompatibilityProvider>(CompatibilityFeatureName);

		// Store the matched columns as a query condition that requires all of them (i.e AND's them)
		for (const TWeakObjectPtr<const UScriptStruct>& Column : MatchedColumns)
		{
			MatchedColumnConditions = MatchedColumnConditions && Queries::TColumn(Column);
		}
		MatchedColumnConditions.Compile(Queries::FEditorStorageQueryConditionCompileContext(Storage));

		// Check if sorting has been explicitly disabled for this column via per-column metadata
		const bool* bColumnDisableSorting = nullptr;
		for (const TWeakObjectPtr<const UScriptStruct>& Column : InMatchedColumns)
		{
			const FMetaDataEntryView DisableSortingEntry = GetMetaData().FindForColumn(Column, "bColumnDisableSorting");
			bColumnDisableSorting = DisableSortingEntry.TryGetExact<bool>();
			if (bColumnDisableSorting)
			{
				break;
			}
		}

		// Retrieve sort and search functors to allow the content of the column to be used for sorting and filtering
		if (HeaderWidgetConstructor)
		{
			if (!bColumnDisableSorting || !*bColumnDisableSorting)
			{
				ColumnSorters = HeaderWidgetConstructor->ConstructColumnSorters(Storage, StorageUi, GetMetaData());
			}
			ColumnSearchers = HeaderWidgetConstructor->ConstructColumnSearchers(Storage, StorageUi, GetMetaData());
		}
		// Prefer to use the sort/search function in the header, but if it doesn't exist, use the one from the cell.
		if (CellWidgetConstructor && ColumnSorters.IsEmpty())
		{
			if (ColumnSorters.IsEmpty() && (!bColumnDisableSorting || !*bColumnDisableSorting))
			{
				ColumnSorters = CellWidgetConstructor->ConstructColumnSorters(Storage, StorageUi, GetMetaData());
			}
			if (ColumnSearchers.IsEmpty())
			{
				ColumnSearchers = CellWidgetConstructor->ConstructColumnSearchers(Storage, StorageUi, GetMetaData());
			}
		}

		RegisterQueries();
	}

	FTedsTableViewerColumn::~FTedsTableViewerColumn()
	{
		UnRegisterQueries();
	}

	TSharedPtr<SWidget> FTedsTableViewerColumn::ConstructRowWidget(RowHandle InRowHandle, TFunction<void(ICoreProvider&, const RowHandle&)> WidgetRowSetupDelegate/* = nullptr*/) const
	{
		TSharedPtr<SWidget> RowWidget;

		if (CellWidgetConstructor && Storage->IsRowAssigned(InRowHandle))
		{
			const RowHandle UiRowHandle = Storage->AddRow(Storage->FindTable(TableViewerUtils::GetWidgetTableName()));

			const TConstArrayView<TWeakObjectPtr<const UScriptStruct>> ColumnTypes = GetMatchedColumns();
			if (ColumnTypes.Num() == 1)
			{
				Storage->AddColumn<FTypedElementScriptStructTypeInfoColumn>(UiRowHandle, FTypedElementScriptStructTypeInfoColumn{ .TypeInfo = *ColumnTypes.begin() });
			}

			if (FTypedElementRowReferenceColumn* RowReference = Storage->GetColumn<FTypedElementRowReferenceColumn>(UiRowHandle))
			{
				RowReference->Row = InRowHandle;
			}

			if (FTypedElementSlateWidgetReferenceColumn* WidgetReferenceColumn = Storage->GetColumn<FTypedElementSlateWidgetReferenceColumn>(UiRowHandle))
			{
				WidgetReferenceColumn->WidgetConstructor = CellWidgetConstructor;
			}

			if (WidgetRowSetupDelegate)
			{
				WidgetRowSetupDelegate(*Storage, UiRowHandle);
			}

			RowWidget = StorageUi->ConstructWidget(UiRowHandle, *CellWidgetConstructor, GetMetaData());
		}

		return RowWidget;
	}

	SHeaderRow::FColumn::FArguments FTedsTableViewerColumn::ConstructHeaderRowColumn()
	{
		const TConstArrayView<TWeakObjectPtr<const UScriptStruct>> ColumnTypes = GetMatchedColumns();

		// Compute the display name up front so it can be used for both the header widget and the tooltip
		FText DefaultHeaderText = CellWidgetConstructor ? CellWidgetConstructor->CreateWidgetDisplayNameText(Storage) : FText::GetEmpty();
		if (DefaultHeaderText.IsEmpty())
		{
			DefaultHeaderText = FText::FromString(ColumnName.ToString());
		}

		TSharedPtr<SWidget> Widget;
		RowHandle UiRowHandle = InvalidRowHandle;
		if (HeaderWidgetConstructor)
		{
			UiRowHandle = Storage->AddRow(Storage->FindTable(FName(TEXT("Editor_WidgetTable"))));

			// TEDS UI TODO: We can't do this from the Widget Constructor because it is a UStruct and does not have access to AsShared(), so we would
			// be forced to store a raw pointer instead of a weak pointer which is unsafe. Once the widget construction pipleline is improved this can
			// probably be moved to a better place
			if (FTypedElementSlateWidgetReferenceColumn* WidgetReferenceColumn = Storage->GetColumn<FTypedElementSlateWidgetReferenceColumn>(UiRowHandle))
			{
				WidgetReferenceColumn->WidgetConstructor = HeaderWidgetConstructor;
			}

			Widget = StorageUi->ConstructWidget(UiRowHandle, *HeaderWidgetConstructor,
				GetMetaData());
		}
		if (!Widget.IsValid())
		{
			Widget = SNew(STextBlock)
				.Text(DefaultHeaderText);
		}

		const bool* bAllowSorting;
		const bool* bColumnInitialSortIsAscending;
		const bool* bUseDisplayNameTooltipPtr;

		{
			UE::Editor::DataStorage::FMetaDataEntryView MetaDataEntryView = GetMetaData().FindGeneric("bAllowSorting");
		    bAllowSorting = MetaDataEntryView.TryGetExact<bool>();
		}
		{
			UE::Editor::DataStorage::FMetaDataEntryView MetaDataEntryView = GetMetaData().FindForColumnsOrGeneric(ColumnTypes, "bColumnInitialSortIsAscending");
			bColumnInitialSortIsAscending = MetaDataEntryView.TryGetExact<bool>();
		}
		{
			UE::Editor::DataStorage::FMetaDataEntryView MetaDataEntryView = GetMetaData().FindGeneric("bUseDisplayNameTooltip");
			bUseDisplayNameTooltipPtr = MetaDataEntryView.TryGetExact<bool>();
		}
		const bool bUseDisplayNameTooltip = bUseDisplayNameTooltipPtr ? *bUseDisplayNameTooltipPtr : true;

		FText TooltipFText;
		if (bUseDisplayNameTooltip)
		{
			TooltipFText = DefaultHeaderText;
		}
		else
		{
			FString TooltipText = TEXT("Data Storage columns:");
			for (const TWeakObjectPtr<const UScriptStruct>& ColumnType : ColumnTypes)
			{
				if (ColumnType.IsValid())
				{
					TooltipText += TEXT("\n    ");
					ColumnType->AppendName(TooltipText);
				}
			}
			TooltipFText = FText::FromString(MoveTemp(TooltipText));
		}

		InitialSortMode = EColumnSortMode::Ascending;
		if (bColumnInitialSortIsAscending)
		{
			InitialSortMode = *bColumnInitialSortIsAscending ? EColumnSortMode::Ascending : EColumnSortMode::Descending;
		}

		SHeaderRow::FColumn::FArguments Column = SHeaderRow::Column(ColumnName)
			.FillWidth(1)
			.HeaderComboVisibility(EHeaderComboVisibility::OnHover)
			.DefaultTooltip(TooltipFText)
			.DefaultLabel(DefaultHeaderText)
			.SortMode_Raw(this, &FTedsTableViewerColumn::GetSortMode)
			.SortPriority_Raw(this, &FTedsTableViewerColumn::GetSortPriority)
			.IsSorting_Raw(this, &FTedsTableViewerColumn::IsSorting)
			.OnSort_Raw(this, &FTedsTableViewerColumn::OnSortCallback)
			.InitialSortMode(InitialSortMode)
			.CanManuallySort(bAllowSorting ? *bAllowSorting : true)
			.HeaderContent()
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
					.FillWidth(1.0f)
					[
						SNew(SBox)
							.MinDesiredHeight(20.0f)
							.VAlign(VAlign_Center)
							[
								Widget.ToSharedRef()
							]
					]
				+SHorizontalBox::Slot()
					.AutoWidth()
					[
						SNew(STextBlock)
							.Text_Raw(this, &FTedsTableViewerColumn::GetSortText)
							.Visibility_Raw(this, &FTedsTableViewerColumn::GetSortTextVisibility)
					]
			];

		if (const FHeaderWidgetSizeColumn* HeaderProperties = Storage->GetColumn<FHeaderWidgetSizeColumn>(UiRowHandle))
		{
			float Width = HeaderProperties->Width;
			switch (HeaderProperties->ColumnSizeMode)
			{
			case EColumnSizeMode::Fill: Column.FillWidth(Width); break;
			case EColumnSizeMode::Fixed: Column.FixedWidth(Width); break;
			case EColumnSizeMode::Manual: Column.ManualWidth(Width); break;
			case EColumnSizeMode::FillSized: Column.FillSized(Width); break;
			}
		}
		return Column;
	}

	void FTedsTableViewerColumn::Tick()
	{
		// Update any rows that could need widget updates
		if (!RowsToUpdate.IsEmpty())
		{
			UpdateWidgets();
			RowsToUpdate.Empty();
		}
	}

	void FTedsTableViewerColumn::SetIsRowVisibleDelegate(FIsRowVisible InIsRowVisibleDelegate)
	{
		IsRowVisibleDelegate = MoveTemp(InIsRowVisibleDelegate);
	}

	void FTedsTableViewerColumn::OnGetSearchStringCallback(const RowHandle& InRowHandle, TArray< FString >& OutSearchStrings)
	{
		if (Storage)
		{
			FString OutString;
			for (const TSharedPtr<const FColumnSearcherInterface>& Searcher : ColumnSearchers )
			{
				OutString.Reset(FName::StringBufferSize);
				Searcher->GetSearchableString(OutString, *Storage, InRowHandle);
				if (!OutString.IsEmpty())
				{
					OutSearchStrings.Add(OutString);
				}
			}
		}
	}

	bool FTedsTableViewerColumn::HasSortInfo() const
	{
		return !ColumnSorters.IsEmpty();
	}

	void FTedsTableViewerColumn::SetInitialSortMode(EColumnSortMode::Type InSortMode)
	{
		InitialSortMode = InSortMode;
		ClearSort();
	}

	void FTedsTableViewerColumn::SetSortDelegates(FSortHandler InSortDelegates)
	{
		SortDelegates = MoveTemp(InSortDelegates);
	}

	void FTedsTableViewerColumn::ClearSort()
	{
		SorterShortName = FText::GetEmpty();
		ColumnSorterIndex = 0;
		bIsSorted = false;
		SortPriority = EColumnSortPriority::None;
	}

	void FTedsTableViewerColumn::RegisterQueries()
	{
		using namespace UE::Editor::DataStorage::Queries;

		const TConstArrayView<TWeakObjectPtr<const UScriptStruct>> ColumnTypes = GetMatchedColumns();
		
		// For each TEDS column this column is matched with, we'll add observers to track addition/removal to update any widgets
		for (const TWeakObjectPtr<const UScriptStruct>& ColumnType : ColumnTypes)
		{
			const FName ColumnAddObserverName = *(FString::Printf(TEXT("Column Add Monitor for %s Table Viewer Column, %s TEDS Column"), *ColumnName.ToString(), *ColumnType->GetName()));
			FObserver AddObserver(FObserver::EEvent::Add, ColumnType.Get());
			AddObserver.SetExecutionMode(EExecutionMode::GameThread);

			// TEDS-Outliner TODO: Long term if we move this into TypedElementOutlinerMode or similar we can get access to the exact
			// types the Outliner is looking at and specify them on .Where() to cut down on the things we are observing
			QueryHandle AddQueryHandle = Storage->RegisterQuery(
				Select(
					ColumnAddObserverName,
					AddObserver,
					[this](IQueryContext& Context, RowHandle Row)
					{
						RowsToUpdate.Add(TPair<RowHandle, bool>(Row, true));
					})
				.Where()
				.All(ColumnType.Get())
				.Compile());

			InternalObserverQueries.Add(AddQueryHandle);

			const FName ColumnRemoveObserverName = *(FString::Printf(TEXT("Column Remove Monitor for %s Table Viewer Column, %s TEDS Column"), *ColumnName.ToString(), *ColumnType->GetName()));
			FObserver RemoveObserver(FObserver::EEvent::Remove, ColumnType.Get());
			RemoveObserver.SetExecutionMode(EExecutionMode::GameThread);

			// Table Viewer TODO: We might be able to cut down on the rows we are querying for in the future by getting the rows from the query stack
			// but we currently have to use a generic query so we can support the TEDS-Outliner as well
			QueryHandle RemoveQueryHandle = Storage->RegisterQuery(
				Select(
					ColumnRemoveObserverName,
					RemoveObserver,
					[this](IQueryContext& Context, RowHandle Row)
					{
						RowsToUpdate.Add(TPair<RowHandle, bool>(Row, false));
					})
				.Where()
				.All(ColumnType.Get())
				.Compile());

			InternalObserverQueries.Add(RemoveQueryHandle);
		}

		// We are looking for widgets that have a row reference
		TArray<const UScriptStruct*> SelectionColumns({ FTypedElementSlateWidgetReferenceColumn::StaticStruct(), FTypedElementRowReferenceColumn::StaticStruct() });

		// Query to get all widgets that were created by this column
		WidgetQuery = Storage->RegisterQuery(
			Select()
				.ReadOnly(SelectionColumns)
			.Compile());
	}

	void FTedsTableViewerColumn::UnRegisterQueries()
	{
		for(const QueryHandle Query : InternalObserverQueries)
		{
			Storage->UnregisterQuery(Query);
		}
		
		Storage->UnregisterQuery(WidgetQuery);

	}

	bool FTedsTableViewerColumn::IsRowVisible(const RowHandle InRowHandle) const
	{
		if(IsRowVisibleDelegate.IsBound())
		{
			return IsRowVisibleDelegate.Execute(InRowHandle);
		}

		// If we don't have a delegate bound we just return true since in the worst case we will just spend time trying to update
		// rows that aren't visible and therefore don't have widgets due to virtualization
		return true; 
	}

	void FTedsTableViewerColumn::UpdateWidgets()
	{
		if (!CellWidgetConstructor)
		{
			return;
		}
		
		// Remove any widget rows that don't actually need an update
		RowsToUpdate = RowsToUpdate.FilterByPredicate([this](const TPair<RowHandle, bool>& Pair) -> bool
		{
			// We don't have a widget for this item visible, so there is nothing to update
			if(!IsRowVisible(Pair.Key))
			{
				return false;
			}
			
			// Check if the row now matches the query conditions for this widget
			bool bMatchesQueryConditions = true;

			// First we try to match against the conditions provided by the widget constructor if possible
			const Queries::FConditions* WidgetConstructorConditions = CellWidgetConstructor->GetQueryConditions(Storage);
			if(WidgetConstructorConditions && WidgetConstructorConditions->IsCompiled())
			{
				bMatchesQueryConditions = bMatchesQueryConditions && Storage->MatchesColumns(Pair.Key, *WidgetConstructorConditions);
			}
			// If the widget constructor didn't provide any conditions, try to match against the columns we were provided on init
			else
			{
				bMatchesQueryConditions = bMatchesQueryConditions && Storage->MatchesColumns(Pair.Key, MatchedColumnConditions);
			}
			
			// If we are adding a column that we are monitoring and it now matches, or if we are removing a column that we are monitoring and it now
			// stops matching, there is a potential need for widget update
			return (bMatchesQueryConditions && Pair.Value) || (!bMatchesQueryConditions && !Pair.Value);
		});

		using namespace UE::Editor::DataStorage::Queries;

		TArray<RowHandle> MatchedWidgetRows;
		DirectQueryCallback RowCollector = CreateDirectQueryCallbackBinding([&MatchedWidgetRows](const IDirectQueryContext& Context, const RowHandle*)
		{
			MatchedWidgetRows.Append(Context.GetRowHandles());
		});

		// Run query to gather all widget rows
		Storage->RunQuery(WidgetQuery, RowCollector);

		// Run the actual logic outside the query because updating the widget can add/remove columns through the DataStorage which is invalid
		// when you are inside a query callback
		for(RowHandle Row : MatchedWidgetRows)
		{
			const FTypedElementSlateWidgetReferenceColumn* WidgetColumn = Storage->GetColumn<FTypedElementSlateWidgetReferenceColumn>(Row);
			const FTypedElementRowReferenceColumn* RowReferenceColumn = Storage->GetColumn<FTypedElementRowReferenceColumn>(Row);

			if(!ensureMsgf(WidgetColumn && RowReferenceColumn,
				TEXT("Expected to have the widget reference and row reference columns since we queried for them")))
			{
				continue;
			}

			// Check if this widget's owning row is in our rows to update
			bool* bColumnAddedPtr = RowsToUpdate.Find(RowReferenceColumn->Row);
			// If not, skip it
			if(!bColumnAddedPtr)
			{
				continue;
			}

			// Check if the container TEDSWidget exists, if not we cannot update this widget
			const TSharedPtr<ITedsWidget> TedsWidget = WidgetColumn->TedsWidget.Pin();
			if(!TedsWidget)
			{
				continue;
			}

			// A row has numerous widgets, make sure we only update the one that was created by our column by checking the constructor
			if(WidgetColumn->WidgetConstructor != CellWidgetConstructor)
			{
				continue;
			}

			// If a column was added and we are here, we need to re-create the widget
			if(*bColumnAddedPtr)
			{
				if(const TSharedPtr<SWidget> RowWidget = StorageUi->ConstructInternalWidget(Row, *CellWidgetConstructor, GetMetaData()))
				{
					TedsWidget->SetContent(RowWidget.ToSharedRef());
				}
			}
			// If a column was removed (and we don't match anymore) delete the internal widget
			else
			{
				TedsWidget->SetContent(SNullWidget::NullWidget);
			}
		}
	}

	FComboMetaDataView<FGenericMetaDataView, FColumnsMetaDataView> FTedsTableViewerColumn::GetMetaData() const
	{
		return FComboMetaDataView(FGenericMetaDataView(WidgetMetaData)).Next(FColumnsMetaDataView(ColumnMetaData));
	}

	FName FTedsTableViewerColumn::GetColumnName() const
	{
		return ColumnName;
	}
	
	TConstArrayView<TWeakObjectPtr<const UScriptStruct>> FTedsTableViewerColumn::GetMatchedColumns() const
	{
		return MatchedColumns;
	}

	ICoreProvider* FTedsTableViewerColumn::GetStorage() const
	{
		return Storage;
	}

	EColumnSortMode::Type FTedsTableViewerColumn::GetSortMode() const
	{
		if (!bIsSorted)
		{
			return EColumnSortMode::None;
		}

		const bool bMatchesInitialDirection = (ColumnSorterIndex & 1) == 1;
		return bMatchesInitialDirection
			? InitialSortMode
			: (InitialSortMode == EColumnSortMode::Ascending
				? EColumnSortMode::Descending
				: EColumnSortMode::Ascending);
	}

	EColumnSortPriority::Type FTedsTableViewerColumn::GetSortPriority() const
	{
		return SortPriority;
	}

	bool FTedsTableViewerColumn::IsSorting() const
	{
		return (bIsSorted && SortDelegates.IsSortingHandler.IsBound()) ? SortDelegates.IsSortingHandler.Execute() : false;
	}

	EVisibility FTedsTableViewerColumn::GetSortTextVisibility() const
	{
		return (bIsSorted && !SorterShortName.IsEmpty()) ? EVisibility::Visible : EVisibility::Collapsed;
	}

	FText FTedsTableViewerColumn::GetSortText() const
	{
		return SorterShortName;
	}

	void FTedsTableViewerColumn::OnSortCallback(
		EColumnSortPriority::Type InSortPriority, const FName& InColumnId, EColumnSortMode::Type InSortDirection)
	{
		if (!ColumnSorters.IsEmpty() && InColumnId == ColumnName)
		{
			bIsSorted = true;

			SortPriority = (InSortPriority == EColumnSortPriority::Secondary) ? EColumnSortPriority::Secondary : EColumnSortPriority::Primary;

			TSharedPtr<const FColumnSorterInterface> Sorter = ColumnSorters[ColumnSorterIndex >> 1];

			if (ColumnSorters.Num() > 1)
			{
				FText ShortName = Sorter->GetShortName();
				SorterShortName = ShortName.IsEmpty()
					? FText::GetEmpty()
					: FText::Format(NSLOCTEXT("TedsTableViewColumn", "SorterName", " - {0}"), ShortName);
			}

			// To ensure consistency, we remove None cases
			if (InSortDirection == EColumnSortMode::None)
			{
				InSortDirection = (InitialSortMode != EColumnSortMode::None) ? InitialSortMode : EColumnSortMode::Ascending;
			}

			// If we are trying to change sort direction, we go to the next column sorter
			const EColumnSortMode::Type CurrentSortDirection = GetSortMode();
			if (CurrentSortDirection != InSortDirection)
			{
				// Each column sorter is used twice, once for ascending and once for descending sorts.
				ColumnSorterIndex = (ColumnSorterIndex + 1) % (ColumnSorters.Num() * 2);
				ensure(GetSortMode() == InSortDirection);
			}

			SortDelegates.OnSortHandler.ExecuteIfBound(ColumnName, Sorter, InSortDirection, SortPriority);
		}
	}
}

