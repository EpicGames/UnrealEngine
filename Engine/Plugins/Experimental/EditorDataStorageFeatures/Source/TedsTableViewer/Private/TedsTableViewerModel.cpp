// Copyright Epic Games, Inc. All Rights Reserved.

#include "TedsTableViewerModel.h"

#include "TedsRowFilterNode.h"
#include "DataStorage/Features.h"
#include "Elements/Columns/TypedElementUIColumns.h"
#include "Elements/Framework/TypedElementQueryCapabilities.h"
#include "Elements/Framework/TypedElementQueryContext.h"
#include "Elements/Interfaces/TypedElementDataStorageInterface.h"
#include "Elements/Interfaces/TypedElementDataStorageUiInterface.h"
#include "TedsRowOrderInversionNode.h"
#include "TedsRowSortNode.h"
#include "TedsTableViewerColumn.h"
#include "TedsTableViewerUtils.h"

DEFINE_LOG_CATEGORY_STATIC(LogTableViewer, Log, All)

namespace UE::Editor::DataStorage
{
	FTedsTableViewerModel::FTedsTableViewerModel(const FArguments& Args)
		: RequestedTedsColumns(Args.RequestedColumns)
		, GenericMetaData(Args.GenericMetaData)
		, ColumnMetaData(Args.ColumnMetaData)
		, CellWidgetPurpose(Args.CellWidgetPurpose)
		, HeaderWidgetPurpose(Args.HeaderWidgetPurpose)
		, IsItemVisible(Args.IsItemVisibleDelegate)
		, PrimaryColumn(Args.PrimaryColumn)
		, bShowHiddenRows(Args.bShowHiddenRows)
	{
		if (!GenericMetaData.IsValid())
		{
			GenericMetaData = MakeShared<FMetaData>();
		}

		Storage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName);
		StorageUi = GetMutableDataStorageFeature<IUiProvider>(UiFeatureName);

		ExecutorInstanceName = FName("TableViewerModel", InstanceCounter++);
		SetQueryStack(Args.RowQueryStack, false);

		TickerHandle = FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateRaw(this, &FTedsTableViewerModel::Tick), 0);

		GenerateColumns();
	}

	FTedsTableViewerModel::~FTedsTableViewerModel()
	{
		FTSTicker::RemoveTicker(TickerHandle);
	}
	
	void FTedsTableViewerModel::Refresh()
	{
		Items.Empty();

		FRowHandleArrayView Rows = RowQueryStack->GetRows();
		
		for(const RowHandle RowHandle : Rows)
		{
			FTedsRowHandle TedsRowHandle{ .RowHandle = RowHandle };
			Items.Add(TedsRowHandle);
		}

		CachedRowQueryStackRevision = RowQueryStack->GetRevision();

		OnModelChanged.Broadcast();
	}

	TSharedPtr<QueryStack::IRowNode> FTedsTableViewerModel::GetRowNode() const
	{
		return RowQueryStack;
	}

	FName FTedsTableViewerModel::GetPrimaryColumnName() const
	{
		return PrimaryUIColumnName;
	}

	void FTedsTableViewerModel::ValidateRequestedColumns()
	{
		RequestedTedsColumns.RemoveAll([](const TWeakObjectPtr<const UScriptStruct>& Column)
		{
			if (ColumnUtils::IsDynamicTemplate(Column.Get()))
			{
				UE_LOGF(LogTableViewer, Log, "%ls Column is a dynamic template which cannot be displayed in the table viewer and has been removed!",
						*Column->GetName());
				return true;
			}

			return false;
		});
	}

	bool FTedsTableViewerModel::Tick(float DeltaTime)
	{
		QueryStackExecutor.Update();

		// If the revision ID has changed, refresh to update our rows
		if(RowQueryStack->GetRevision() != CachedRowQueryStackRevision)
		{
			Refresh();
		}

		// Tick all the individual column views
		for(const TSharedRef<FTedsTableViewerColumn>&Column : ColumnsView)
		{
			Column->Tick();
		}
		
		return true;
	}

	const TArray<TableViewerItemPtr>& FTedsTableViewerModel::GetItems() const
	{
		return Items;
	}

	uint64 FTedsTableViewerModel::GetRowCount() const
	{
		return Items.Num();
	}

	uint64 FTedsTableViewerModel::GetColumnCount() const
	{
		return ColumnsView.Num();
	}

	TSharedPtr<FTedsTableViewerColumn> FTedsTableViewerModel::GetColumn(const FName& ColumnName) const
	{
		const TSharedRef<FTedsTableViewerColumn>* Column = ColumnsView.FindByPredicate([ColumnName]
			(const TSharedRef<FTedsTableViewerColumn>& InColumn)
		{
			return InColumn->GetColumnName() == ColumnName;
		});
		
		if(Column)
		{
			return *Column;
		}

		return nullptr;
	}

	int32 FTedsTableViewerModel::GetColumnIndex(const FName& ColumnName) const
	{
		return ColumnsView.IndexOfByPredicate([ColumnName]
			(const TSharedRef<FTedsTableViewerColumn>& InColumn)
		{
			return InColumn->GetColumnName() == ColumnName;
		});
	}

	void FTedsTableViewerModel::ForEachColumn(const TFunctionRef<void(const TSharedRef<FTedsTableViewerColumn>&)>& Delegate) const
	{
		for(const TSharedRef<FTedsTableViewerColumn>& Column : ColumnsView)
		{
			Delegate(Column);
		}
	}

	FTedsTableViewerModel::FOnModelChanged& FTedsTableViewerModel::GetOnModelChanged()
	{
		return OnModelChanged;
	}

	void FTedsTableViewerModel::SetQueryStack(const TSharedPtr<QueryStack::IRowNode>& InRowQueryStack, bool bUpdateExecutor /*= true*/)
	{
		checkf(InRowQueryStack, TEXT("The TEDS Table Viewer requires a valid query stack to provide it with rows to process."));
				
		using namespace UE::Editor::DataStorage::Queries;
		
		TSharedPtr<QueryStack::IRowNode> QueryStack = InRowQueryStack;

		if (!bShowHiddenRows)
		{
			QueryStack = MakeShared<QueryStack::FRowFilterNode>(Storage, QueryStack, 
			[](TConstQueryContext<SingleRowInfo, CurrentTableInfo> Context)
			{
				return !Context.CurrentTableHasColumns<FHideRowFromUITag>();
			});
		}
		
		// If the sorting nodes have been made and are just being reset to a new query stack, persist the sorting state
		RowSecondarySortingNode = MakeShared<QueryStack::FRowSortNode>(*Storage, QueryStack, RowSecondarySortingNode ? RowSecondarySortingNode->GetColumnSorter() : nullptr);
		RowSecondaryInversionNode = MakeShared<QueryStack::FRowOrderInversionNode>(RowSecondarySortingNode, RowSecondaryInversionNode ? RowSecondaryInversionNode->IsEnabled() : false);
		RowPrimarySortingNode = MakeShared<QueryStack::FRowSortNode>(*Storage, RowSecondaryInversionNode, RowPrimarySortingNode ? RowPrimarySortingNode->GetColumnSorter() : nullptr);
		RowPrimaryInversionNode = MakeShared<QueryStack::FRowOrderInversionNode>(RowPrimarySortingNode, RowPrimaryInversionNode ? RowPrimaryInversionNode->IsEnabled() : false);
		RowQueryStack = RowPrimaryInversionNode;
		
		QueryStackExecutor = QueryStack::FExplicitUpdateExecutor(ExecutorInstanceName, RowQueryStack);

		if (bUpdateExecutor)
		{
			QueryStackExecutor.Update();
		}
		
		Refresh();
	}

	void FTedsTableViewerModel::SetColumns(const TArray<TWeakObjectPtr<const UScriptStruct>>& InColumns)
	{
		RequestedTedsColumns = InColumns;
		GenerateColumns();
	}

	void FTedsTableViewerModel::AddCustomRowWidget(const TSharedRef<FTedsTableViewerColumn>& InColumn)
	{
		// Table Viewer TODO: We should allow users to specify sort order using a TEDS column on the UI row, but for now we put any custom
		// columns on the front
		ColumnsView.Insert(InColumn, 0);
		InColumn->SetSortDelegates(FTedsTableViewerColumn::FSortHandler
			{
				.OnSortHandler = FTedsTableViewerColumn::FOnSort::CreateRaw(this, &FTedsTableViewerModel::OnSort),
				.IsSortingHandler = FTedsTableViewerColumn::FIsSorting::CreateRaw(this, &FTedsTableViewerModel::IsSorting)
			});
	}

	ICoreProvider* FTedsTableViewerModel::GetDataStorageInterface() const
	{
		return Storage;
	}

	IUiProvider* FTedsTableViewerModel::GetDataStorageUiProvider() const
	{
		return StorageUi;
	}

	void FTedsTableViewerModel::GenerateColumns()
	{
		ValidateRequestedColumns();

		RowHandle CellWidgetPurposeRow = StorageUi->FindPurpose(CellWidgetPurpose);
		RowHandle HeaderWidgetPurposeRow = StorageUi->FindPurpose(HeaderWidgetPurpose);
		
		using MatchApproach = IUiProvider::EMatchApproach;
		int32 IndexOffset = 0;

		ColumnsView.Empty();
		
		// A Map of TEDS Columns -> UI columns so we can add them in the same order they were specified
		TMap<TWeakObjectPtr<const UScriptStruct>, TSharedRef<FTedsTableViewerColumn>> NewColumnMap;

		// A copy of the columns to preserve the order since TEDS UI modifies the array directly
		TArray<TWeakObjectPtr<const UScriptStruct>> ColumnsCopy = RequestedTedsColumns;
		
		// Lambda to create the constructor for a given list of columns
		auto ColumnConstructor = [this, &IndexOffset, &NewColumnMap, HeaderWidgetPurposeRow](
			IUiProvider::FWidgetConstructorPtr CellConstructor, TConstArrayView<TWeakObjectPtr<const UScriptStruct>> MatchedColumns)
			{
				IUiProvider::FWidgetConstructorPtr HeaderConstructor =
					TableViewerUtils::CreateHeaderWidgetConstructor(*StorageUi, FColumnsMetaDataView(ColumnMetaData), MatchedColumns, HeaderWidgetPurposeRow);
			
				FName NameId = TableViewerUtils::FindLongestMatchingName(MatchedColumns, IndexOffset);

				TArray<TWeakObjectPtr<const UScriptStruct>> MatchedColumnsCopy(MatchedColumns);

				// Currently we are passing in metadata for all columns to individual columns, we might want to only pass in the metadata for the matched columns
				// for each one
				TSharedRef<FTedsTableViewerColumn> Column = MakeShared<FTedsTableViewerColumn>(NameId, CellConstructor, MoveTemp(MatchedColumnsCopy), HeaderConstructor,
					*GenericMetaData, ColumnMetaData);
			
				Column->SetIsRowVisibleDelegate(FTedsTableViewerColumn::FIsRowVisible::CreateRaw(this, &FTedsTableViewerModel::IsRowVisible));
				Column->SetSortDelegates(FTedsTableViewerColumn::FSortHandler
					{
						.OnSortHandler = FTedsTableViewerColumn::FOnSort::CreateRaw(this, &FTedsTableViewerModel::OnSort),
						.IsSortingHandler = FTedsTableViewerColumn::FIsSorting::CreateRaw(this, &FTedsTableViewerModel::IsSorting)
					});

				for (const TWeakObjectPtr<const UScriptStruct>& ColumnType : MatchedColumns)
				{
					NewColumnMap.Emplace(ColumnType, Column);
				}
			
				++IndexOffset;
				return true;
			};

		// Create the widget constructors for the columns
		StorageUi->CreateWidgetConstructors(CellWidgetPurposeRow, MatchApproach::LongestMatch, ColumnsCopy, 
			FColumnsMetaDataView(ColumnMetaData), ColumnConstructor);
		
		// Add the actual UI columns in the order the Teds Columns were specified
		for (const TWeakObjectPtr<const UScriptStruct>& ColumnType : RequestedTedsColumns)
		{
			if(const TSharedRef<FTedsTableViewerColumn>* FoundColumn = NewColumnMap.Find(ColumnType))
			{
				// If the column already exists, a widget matched it and a previously encountered column together and was already added
				// so we can safely ignore it here
				if(!GetColumn((*FoundColumn)->GetColumnName()))
				{
					ColumnsView.Add(*FoundColumn);
				}
			}
		}

		// Cache the primary UI column name since it can only change when we regenerate the columns
		if (const TSharedRef<FTedsTableViewerColumn>* FoundPrimaryColumn = NewColumnMap.Find(PrimaryColumn))
		{
			PrimaryUIColumnName = (*FoundPrimaryColumn)->GetColumnName();

			// Initialize default sorting with the primary column, if it exists
			// Eventually there will be a use case to make this into a separate parameter, but for now this is sufficient for current user needs
			const FMetaDataEntryView MetaDataEntryView = (*FoundPrimaryColumn)->GetMetaData().FindForColumnOrGeneric(PrimaryColumn, "bColumnInitialSortIsAscending");
			const bool* bColumnInitialSortIsAscending = MetaDataEntryView.TryGetExact<bool>();
			EColumnSortMode::Type SortMode = EColumnSortMode::Ascending;
			if (bColumnInitialSortIsAscending)
			{
				SortMode = *bColumnInitialSortIsAscending ? EColumnSortMode::Ascending : EColumnSortMode::Descending;
			}
			(*FoundPrimaryColumn)->SetInitialSortMode(SortMode);
			(*FoundPrimaryColumn)->OnSortCallback(EColumnSortPriority::Primary, PrimaryUIColumnName, SortMode);
		}
		else
		{
			PrimaryUIColumnName = NAME_None;
		}
	}

	bool FTedsTableViewerModel::IsRowVisible(RowHandle InRowHandle) const
	{
		if(!IsItemVisible.IsBound())
		{
			return true;
		}
		
		// Table Viewer TODO: We can probably store a map of the items instead but this works for now
		for(const TableViewerItemPtr& Item : Items)
		{
			if(Item == InRowHandle)
			{
				return IsItemVisible.Execute(Item);
			}
		}

		return true;
	}

	void FTedsTableViewerModel::OnSort(FName ColumnName, TSharedPtr<const FColumnSorterInterface> ColumnSorter, EColumnSortMode::Type Direction, 
		EColumnSortPriority::Type Priority)
	{
		if (RowPrimarySortingNode && RowSecondarySortingNode && RowPrimaryInversionNode && RowSecondaryInversionNode)
		{
			if (Priority == EColumnSortPriority::Secondary)
			{
				if (RowSecondarySortingNode->GetColumnSorter() != ColumnSorter)
				{
					RowSecondarySortingNode->SetColumnSorter(ColumnSorter);
				}
				RowSecondaryInversionNode->Enable(Direction != EColumnSortMode::Type::Ascending);
				SecondarySortColumnName = ColumnName;
			}
			else
			{
				if (RowPrimarySortingNode->GetColumnSorter() != ColumnSorter)
				{
					RowPrimarySortingNode->SetColumnSorter(ColumnSorter);
				}
				RowPrimaryInversionNode->Enable(Direction != EColumnSortMode::Type::Ascending);
				PrimarySortColumnName = ColumnName;
				
				// Reset the secondary sorter.
				SecondarySortColumnName = NAME_None;
				RowSecondarySortingNode->SetColumnSorter(nullptr);
				RowSecondaryInversionNode->Enable(false);
			}
		}
		else
		{
			// Reset so all columns get cleared.
			PrimarySortColumnName = NAME_None;
			SecondarySortColumnName = NAME_None;
		}

		for (const TSharedRef<FTedsTableViewerColumn>& Column : ColumnsView)
		{
			FName LocalColumnName = Column->GetColumnName();
			if (LocalColumnName != PrimarySortColumnName && LocalColumnName != SecondarySortColumnName)
			{
				Column->ClearSort();
			}
		}
	}

	bool FTedsTableViewerModel::IsSorting() const
	{
		return (RowPrimarySortingNode && RowSecondarySortingNode) 
			? (RowPrimarySortingNode->IsSorting() || RowSecondarySortingNode->IsSorting())
			: false;
	}
} // namespace UE::Editor::DataStorage
