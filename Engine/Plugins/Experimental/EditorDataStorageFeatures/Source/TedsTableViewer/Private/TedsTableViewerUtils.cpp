// Copyright Epic Games, Inc. All Rights Reserved.

#include "TedsTableViewerUtils.h"

#include "GameFramework/Actor.h"
#include "DataStorage/Debug/Log.h"
#include "DataStorage/Features.h"
#include "Elements/Columns/TypedElementCompatibilityColumns.h"
#include "Elements/Columns/TypedElementIconOverrideColumns.h"
#include "Elements/Columns/TypedElementMiscColumns.h"
#include "Elements/Columns/TypedElementTypeInfoColumns.h"
#include "Elements/Columns/TypedElementUIColumns.h"
#include "Elements/Interfaces/TypedElementDataStorageUiInterface.h"
#include "Styling/SlateIconFinder.h"
#include "TedsTableViewerWidgetColumns.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TedsTableViewerUtils)

namespace UE::Editor::DataStorage::TableViewerUtils
{
	static FName TableViewerWidgetTableName("Editor_TableViewerWidgetTable");
	static FName TableViewerWidgetHierarchyName("TableViewerWidgetHierarchy");
	static FName TableViewerRowWidgetTableName("Editor_TableViewerRowWidgetTable");
	static uint32 MaxTableViewerRowWidgetRows = 15000;

	FName GetWidgetTableName()
	{
		return TableViewerWidgetTableName;
	}

	FName GetWidgetHierarchyName()
	{
		return TableViewerWidgetHierarchyName;
	}

	FName GetRowWidgetTableName()
	{
		return TableViewerRowWidgetTableName;
	}
	
	// TEDS UI TODO: Maybe the widget can specify a user facing name derived from the matched columns instead of trying to find the longest matching name
	FName FindLongestMatchingName(TConstArrayView<TWeakObjectPtr<const UScriptStruct>> ColumnTypes, int32 DefaultNameIndex)
	{
		switch (ColumnTypes.Num())
		{
		case 0:
			return FName(TEXT("Column"), DefaultNameIndex);
		case 1:
			return FName(ColumnTypes[0]->GetDisplayNameText().ToString());
		default:
			{
				FText LongestMatchText = ColumnTypes[0]->GetDisplayNameText();
				FStringView LongestMatch = LongestMatchText.ToString();
				const TWeakObjectPtr<const UScriptStruct>* ItEnd = ColumnTypes.end();
				const TWeakObjectPtr<const UScriptStruct>* It = ColumnTypes.begin();
				++It; // Skip the first entry as that's already set.
				for (; It != ItEnd; ++It)
				{
					FText NextMatchText = (*It)->GetDisplayNameText();
					FStringView NextMatch = NextMatchText.ToString();

					int32 MatchSize = 0;
					auto ItLeft = LongestMatch.begin();
					auto ItLeftEnd = LongestMatch.end();
					auto ItRight = NextMatch.begin();
					auto ItRightEnd = NextMatch.end();
					while (
						ItLeft != ItLeftEnd &&
						ItRight != ItRightEnd &&
						*ItLeft == *ItRight)
					{
						++MatchSize;
						++ItLeft;
						++ItRight;
					}

					// At least 3 letters have to match to avoid single or double letter names which typically mean nothing.
					if (MatchSize > 2)
					{
						LongestMatch.LeftInline(MatchSize);
					}
					else
					{
						// There are not enough characters in the string that match. Just return the name of the first column
						return FName(ColumnTypes[0]->GetDisplayNameText().ToString());
					}
				}
				return FName(LongestMatch);
			}
		};
	}
	
	TArray<TWeakObjectPtr<const UScriptStruct>> CreateVerifiedColumnTypeArray(TConstArrayView<TWeakObjectPtr<const UScriptStruct>> ColumnTypes)
	{
		TArray<TWeakObjectPtr<const UScriptStruct>> VerifiedColumnTypes;
		VerifiedColumnTypes.Reserve(ColumnTypes.Num());
		for (const TWeakObjectPtr<const UScriptStruct>& ColumnType : ColumnTypes)
		{
			if (ColumnType.IsValid())
			{
				VerifiedColumnTypes.Add(ColumnType.Get());
			}
			else
			{
				UE_LOGF(LogEditorDataStorage, Verbose, "Invalid column provided to the table viewer");
			}
		}
		return VerifiedColumnTypes;
	}

	IUiProvider::FWidgetConstructorPtr CreateHeaderWidgetConstructor(IUiProvider& StorageUi,
		const FMetaDataView& InMetaData, TConstArrayView<TWeakObjectPtr<const UScriptStruct>> ColumnTypes, RowHandle PurposeRow)
	{
		using MatchApproach = IUiProvider::EMatchApproach;

		TArray<TWeakObjectPtr<const UScriptStruct>> VerifiedColumnTypes = CreateVerifiedColumnTypeArray(ColumnTypes);
		IUiProvider::FWidgetConstructorPtr Constructor = nullptr;

		StorageUi.CreateWidgetConstructors(PurposeRow, MatchApproach::ExactMatch, VerifiedColumnTypes, InMetaData,
				[&Constructor, VerifiedColumnTypes](
					IUiProvider::FWidgetConstructorPtr WidgetConstructor, 
					TConstArrayView<TWeakObjectPtr<const UScriptStruct>> MatchedColumnTypes)
				{
					if (VerifiedColumnTypes.Num() == MatchedColumnTypes.Num())
					{
						Constructor = MoveTemp(WidgetConstructor);
					}
					// Either this was the exact match so no need to search further or the longest possible chain didn't match so the next ones will 
					// always be shorter in both cases just return.
					return false;
				});
		
		return Constructor;
	}

	const FSlateBrush* GetIconForRow(ICoreProvider* DataStorage, RowHandle Row)
	{
		static TMap<FName, const FSlateBrush*> CachedIconMap;

		auto FindCachedIcon = [](const FName& IconName) -> const FSlateBrush*
		{
			if (const FSlateBrush** CachedBrush = CachedIconMap.Find(IconName))
			{
				if (*CachedBrush)
				{
					return *CachedBrush;
				}
			}
			return nullptr;
		};

		// Look for any icon overrides
		if(const FTypedElementIconOverrideColumn* IconOverrideColumn = DataStorage->GetColumn<FTypedElementIconOverrideColumn>(Row))
		{
			const FName IconName = IconOverrideColumn->IconName;

			if(const FSlateBrush* CachedBrush = FindCachedIcon(IconName))
			{
				return CachedBrush;
			}
			else if(const FSlateBrush* CustomBrush = FSlateIconFinder::FindIcon(IconName).GetOptionalIcon())
			{
				CachedIconMap.Add(IconName, CustomBrush);
				return CustomBrush;
			}
		}
		// Otherwise find the icon from the type information if available
		else if (const FTypedElementClassTypeInfoColumn* TypeInfoColumn = DataStorage->GetColumn<FTypedElementClassTypeInfoColumn>(Row))
		{
			if (const UClass* Type = TypeInfoColumn->TypeInfo.Get())
			{
				const FName IconName = Type->GetFName();

				if(const FSlateBrush* CachedBrush = FindCachedIcon(IconName))
				{
					return CachedBrush;
				}
				else if(const FSlateBrush* TypeBrush = FSlateIconFinder::FindIconBrushForClass(Type))
				{
					CachedIconMap.Add(IconName, TypeBrush);
					return TypeBrush;
				}
			}
		}
		
		// Fallback to the regular actor icon if we haven't found any specific icon
		return FSlateIconFinder::FindIconForClass(AActor::StaticClass()).GetOptionalIcon();

	}

	void CreateRowMappingKey(const FName& InHierarchyIdentifier, const TableViewerItemPtr InItem, FMapKey& OutMapKey)
	{
		CreateRowMappingKey(InHierarchyIdentifier, InItem.RowHandle, OutMapKey);
	}
	
	void CreateRowMappingKey(const FName& InHierarchyIdentifier, const RowHandle InItem, FMapKey& OutMapKey)
	{
		OutMapKey = FMapKey(FString::Printf(TEXT("%s.%llu"), *InHierarchyIdentifier.ToString(), InItem));
	}
	
	RowHandle GetTableViewerRowUiRow(const ICoreProvider* DataStorage, const RowHandle ChildWidgetRow)
	{
		const FHierarchyHandle WidgetHierarchy = DataStorage->FindHierarchyByName(GetWidgetHierarchyName());
		RowHandle ParentRow = ChildWidgetRow;
		while (ParentRow != InvalidRowHandle)
		{
			ParentRow = DataStorage->GetParentRow(WidgetHierarchy, ParentRow);
			if (DataStorage->HasColumns<FTableViewerRowTag>(ParentRow))
			{
				break;
			}
		}
		return ParentRow;
	}

	RowHandle GetTableViewerUiRow(const ICoreProvider* DataStorage, const RowHandle ChildWidgetRow)
	{
		const FHierarchyHandle WidgetHierarchy = DataStorage->FindHierarchyByName(GetWidgetHierarchyName());
		RowHandle ParentRow = ChildWidgetRow;
		while (ParentRow != InvalidRowHandle)
		{
			ParentRow = DataStorage->GetParentRow(WidgetHierarchy, ParentRow);
			if (DataStorage->HasColumns<FTableViewerTag>(ParentRow))
			{
				break;
			}
		}
		return ParentRow;
	}

	// Helper to get all UI Rows for a given TableViewer
	void GetAllTableViewerUIRows(const ICoreProvider* DataStorage, const RowHandle TableViewerRowHandle, TArray<RowHandle>& OutRows)
	{
		if (DataStorage)
		{
			DataStorage->WalkDepthFirst(DataStorage->FindHierarchyByName(GetWidgetHierarchyName()), TableViewerRowHandle,
			[&OutRows](const ICoreProvider& Context, RowHandle, const RowHandle Target)
			{
				if (Context.HasColumns<FTedsTableViewerRowTag>(Target))
				{
					OutRows.Add(Target);
				}
			});
		}
	}

	void RemoveAllTableViewerUIRows(ICoreProvider* DataStorage, const RowHandle TableViewerRowHandle)
	{
		// Remove any rows attached to this table and the dynamic table
		if (DataStorage)
		{
			TArray<RowHandle> ChildUIRows;
			GetAllTableViewerUIRows(DataStorage, TableViewerRowHandle, ChildUIRows);
			DataStorage->BatchRemoveRows(ChildUIRows);
		}
	}

	void DeactivateAllTableViewerUIRows(ICoreProvider* DataStorage, const RowHandle TableViewerRowHandle)
	{
		// Deactivate any rows attached to this table and the dynamic table
		if (DataStorage)
		{
			TArray<RowHandle> ChildUIRows;
			GetAllTableViewerUIRows(DataStorage, TableViewerRowHandle, ChildUIRows);
			DataStorage->BatchAddRemoveColumns(ChildUIRows, { FTedsTableViewerInactiveRowTag::StaticStruct() }, {});
		}
	}
	
	void InitializeAllTableViewerUIRows(ICoreProvider* DataStorage, const TArray<TableViewerItemPtr>& Items, 
		const FGetTableViewerRowMapping& GetTableViewerRowMapping, const FName& TableViewerIdentifier, 
		const RowHandle TableViewerRowHandle, const bool bAddHierarchyTags)
	{
		// Have all rows initialize their UI Row counterparts
		if (DataStorage)
		{
			const TableHandle RowWidgetTableHandle = DataStorage->FindTable(GetRowWidgetTableName());
			const FHierarchyHandle WidgetHierarchyHandle = DataStorage->FindHierarchyByName(GetWidgetHierarchyName());

			if (RowWidgetTableHandle != InvalidTableHandle && DataStorage->IsValidHierarchyHandle(WidgetHierarchyHandle))
			{
				for (const TableViewerItemPtr& Item : Items)
				{
					FMapKey MappingKey;
					if (GetTableViewerRowMapping.IsBound())
					{
						GetTableViewerRowMapping.Execute(TableViewerIdentifier, Item, MappingKey);
					}
					if (!MappingKey.IsSet())
					{
						CreateRowMappingKey(TableViewerIdentifier, Item, MappingKey);
					}
					// If the row already existed we will use that, but if not, we want to create a row for each entry so
					// changing the default expansion state doesn't affect rows that come into view but aren't new
					if (const RowHandle ExistingRow = DataStorage->LookupMappedRow(TableViewerUIRowMappingDomain, MappingKey); DataStorage->IsRowAvailable(ExistingRow))
					{
						DataStorage->RemoveColumns<FTedsTableViewerInactiveRowTag>(ExistingRow);
					}
					else
					{
						const RowHandle AddedRow = DataStorage->AddRow(RowWidgetTableHandle);
			
						DataStorage->SetParentRow(WidgetHierarchyHandle, AddedRow, TableViewerRowHandle);
						DataStorage->MapRow(TableViewerUIRowMappingDomain, MappingKey, AddedRow);
						if (bAddHierarchyTags)
						{
							DataStorage->AddColumns<FExpandedInUITag>(AddedRow);
						}
					}
				}
			}
		}
	}
}

void UTedsTableViewerFactory::RegisterQueries(UE::Editor::DataStorage::ICoreProvider& DataStorage)
{
	using namespace UE::Editor::DataStorage;
	using namespace UE::Editor::DataStorage::Queries;

	struct FRemoveAllInactiveTableViewerRowWidgetRows
	{
		void operator()()
		{
			if (Storage)
			{
				Storage->RemoveAllRowsWith<FTableViewerRowTag, FTedsTableViewerInactiveRowTag>();
			}
		}
		ICoreProvider* Storage = nullptr;
	};

	const QueryHandle TableViewerRowWidgetCount = DataStorage.RegisterQuery(
		Count()
		.Where()
			.All<FTableViewerRowTag>()
		.Compile()
	);
	
	DataStorage.RegisterQuery(
		Select(
			TEXT("Check if the max TableViewer Rows has been reached and command to purge all inactive rows"),
			FObserver::OnAdd<FTableViewerTag>(),
			[](IQueryContext& Context, RowHandle Row)
			{
				const FQueryResult QueryResult = Context.RunSubquery(0);
				if (QueryResult.Count > TableViewerUtils::MaxTableViewerRowWidgetRows)
				{
					Context.PushCommand(FRemoveAllInactiveTableViewerRowWidgetRows
					{
						.Storage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName)
					});
				}
			})
		.DependsOn()
			.SubQuery(TableViewerRowWidgetCount)
		.Compile());
}

void UTedsTableViewerFactory::RegisterTables(UE::Editor::DataStorage::ICoreProvider& DataStorage)
{
	using namespace UE::Editor::DataStorage;
	const TableHandle BaseWidgetTable = DataStorage.FindTable(FName(TEXT("Editor_WidgetTable")));
	if (BaseWidgetTable != InvalidTableHandle)
	{
		DataStorage.RegisterTable<FTypedElementRowReferenceColumn>(TableViewerUtils::TableViewerWidgetTableName,
			FTableRegistrationOptions{ .SourceTable = BaseWidgetTable });
	}
	
	DataStorage.RegisterTable< FTableViewerRowTag>(TableViewerUtils::TableViewerRowWidgetTableName);
}

void UTedsTableViewerFactory::RegisterHierarchies(UE::Editor::DataStorage::ICoreProvider& DataStorage)
{
	using namespace UE::Editor::DataStorage;

	const FHierarchyRegistrationParams Params
	{
		.Name = TableViewerUtils::GetWidgetHierarchyName()
	};

	DataStorage.RegisterHierarchy(Params);
}
