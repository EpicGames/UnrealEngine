// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/Composite/STedsCompositeHierarchyViewer.h"

#include "DragAndDrop/Widgets/WidgetDropHandler.h"
#include "TedsQueryStackInterfaces.h"
#include "TedsRowMergeNode.h"
#include "QueryStack/AddParentHierarchyRowsNode.h"
#include "Widgets/STedsFilterBar.h"
#include "Widgets/STedsSearchBox.h"
#include "Widgets/Input/SComboButton.h"

#define LOCTEXT_NAMESPACE "TedsCompositeHierarchyViewer"

namespace UE::Editor::DataStorage
{

	FTedsCompositeHierarchyViewerOption::FTedsCompositeHierarchyViewerOption(const FText& InSettingTitle, const FText& InSettingTitleToolTip, EOptionCategories InOptionCategory, bool bInActive)
		: OptionTitle(InSettingTitle)
		, OptionToolTip(InSettingTitleToolTip)
		, OptionCategory(InOptionCategory)
		, bActive(bInActive)
	{
	}

	bool FTedsCompositeHierarchyViewerOption::IsActive() const
	{
		return bActive;
	}

	void FTedsCompositeHierarchyViewerOption::ToggleIsActive()
	{
		bActive = !bActive;
		OnToggleEvent.Broadcast(bActive);
	}

	void FTedsCompositeHierarchyViewerOption::AddMenu(FMenuBuilder& InMenuBuilder)
	{
		TSharedRef<FTedsCompositeHierarchyViewerOption> SharedThis = AsShared();

		InMenuBuilder.AddMenuEntry(
			OptionTitle,
			OptionToolTip,
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateSP(SharedThis, &FTedsCompositeHierarchyViewerOption::ToggleIsActive),
				FCanExecuteAction(),
				FIsActionChecked::CreateSP(SharedThis, &FTedsCompositeHierarchyViewerOption::IsActive)
			),
			NAME_None,
			EUserInterfaceActionType::ToggleButton
		);
	}

	FTedsCompositeHierarchyViewerOption::EOptionCategories FTedsCompositeHierarchyViewerOption::GetCategory() const
	{
		return OptionCategory;
	}

	void STedsCompositeHierarchyViewer::Construct(const FArguments& InArgs, TSharedPtr<IHierarchyViewerDataInterface> InHierarchyInterface)
	{
		Options = InArgs._Options;

		TSharedPtr<SVerticalBox> ContentWidget = SNew(SVerticalBox);

		const bool bFilteringEnabled = InArgs._EnableFiltering;
		const bool bSearchingEnabled = InArgs._EnableSearching;
		const bool bSettingsEnabled = InArgs._EnableSettings;

		bShowFilteredOutParentHierarchy = InArgs._ShowFilteredParentHierarchy;
		
		SHierarchyViewer::FArguments HierarchyViewerArgs = InArgs._HierarchyViewerArgs;

		QueryNode = HierarchyViewerArgs._AllNodeProvider;
		CompositeQueryNode = QueryNode;
		
		if(bFilteringEnabled || bSearchingEnabled)
		{
			// Box containing the default search + filter widgets
			TSharedPtr<SHorizontalBox> FilterSearchBar = SNew(SHorizontalBox);

			if (bSearchingEnabled)
			{
				SearchBox = SNew(STedsSearchBox)
					.InSearchableRowNode(CompositeQueryNode)
					.OutSearchNode(&SearchNode);

				if (SearchNode.IsValid())
				{
					// If filtering is enabled, we want to add back the parent rows on that node instead (if enabled) since it is later down the chain
					CompositeQueryNode = SearchNode;
					if (!bFilteringEnabled)
					{
						AddFilteredParentRowsToQueryStack(&CompositeQueryNode, InHierarchyInterface);
					}
				}
			}
			if (bFilteringEnabled)
			{
				FilterBar = SNew(STedsFilterBar)
					.InFilterableRowNode(CompositeQueryNode)
					.OutFilteredNode(&FilterNode)
					.CommonSectionFilters(InArgs._CommonSectionFilters)
					.Filters(InArgs._Filters)
					.UseSectionsForCategories(InArgs._UseSectionsForCategories)
					.OnPostFiltersChanged_Lambda([this]()
					{
						if (HierarchyViewer.IsValid() && FilterNode.IsValid())
						{
							// Forward the updated FilterNode to the Model QueryStack since it holds a static copy to the old filter node
							// Only called when the filters have changed, meaning the query stack has to be recomputed
							CompositeQueryNode = FilterNode;
							AddFilteredParentRowsToQueryStack(&CompositeQueryNode, HierarchyViewer->HierarchyInterface);
							HierarchyViewer->SetQueryStack(CompositeQueryNode);
						}
					});

				if (FilterNode.IsValid())
				{
					// If searching is disabled, there is no reason to add filtered parent rows since on init there should be nothing
					// filtering out rows (filters cannot init enabled yet), when a filter is enabled, OnPostFiltersChanged will take
					// care of adding the FilteredParentNode
					CompositeQueryNode = FilterNode;
					if (!bSearchingEnabled)
					{
						AddFilteredParentRowsToQueryStack(&CompositeQueryNode, InHierarchyInterface);
					}
				}
			}

			// Add Filter Menu Button before Search Box if it was successfully created
			if (FilterBar)
			{
				FilterSearchBar->AddSlot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(2.0f)
				[
					FilterBar->MakeAddFilterButton(FilterBar.ToSharedRef())
				];
			}
			// Add Search Box if it was successfully created
			if (SearchBox)
			{
				FilterSearchBar->AddSlot()
				.FillWidth(1.0f)
				.VAlign(VAlign_Center)
				.Padding(2.0f)
				[
					SearchBox.ToSharedRef()
				];
			}

			// Add Filter menu dropdown button + search bar above the viewer
			// TODO: Make it so that different default styles can be set to show the Filter/Search bar in different ways
			ContentWidget->AddSlot()
			.AutoHeight()
			[
				FilterSearchBar.ToSharedRef()
			];

			// Add Filter shelf
			if (bFilteringEnabled)
			{
				ContentWidget->AddSlot()
				.AutoHeight()
				.Padding(2.0f)
				[
					FilterBar.ToSharedRef()
				];
			}

			if (bSettingsEnabled)
			{
				// View mode combo button
				FilterSearchBar->AddSlot()
				.VAlign(VAlign_Center)
				.AutoWidth()
				[
					SAssignNew(SettingsComboButton, SComboButton)
					.ComboButtonStyle(FAppStyle::Get(), "SimpleComboButtonWithIcon") // Use the toolbar item style for this button
					.OnGetMenuContent(this, &STedsCompositeHierarchyViewer::GetSettingsButtonContent)
					.HasDownArrow(false)
					.ButtonContent()
					[
						SNew(SImage)
						.ColorAndOpacity(FSlateColor::UseForeground())
						.Image(FAppStyle::Get().GetBrush("Icons.Settings"))
					]
				];
			}
		}

		HierarchyViewerArgs.AllNodeProvider(CompositeQueryNode);
		SAssignNew(HierarchyViewer, SHierarchyViewer, MoveTemp(InHierarchyInterface))
			.AllNodeProvider(HierarchyViewerArgs._AllNodeProvider)
			.TableViewerIdentifier(HierarchyViewerArgs._TableViewerIdentifier)
			.Columns(HierarchyViewerArgs._Columns)
			.EmptyRowsMessage(HierarchyViewerArgs._EmptyRowsMessage)
			.CellWidgetPurpose(HierarchyViewerArgs._CellWidgetPurpose)
			.HeaderWidgetPurpose(HierarchyViewerArgs._HeaderWidgetPurpose)
			.GenericMetaData(HierarchyViewerArgs._GenericMetaData)
			.ColumnMetaData(HierarchyViewerArgs._ColumnMetaData)
			.ListSelectionMode(HierarchyViewerArgs._ListSelectionMode)
			.OnSelectionChanged(HierarchyViewerArgs._OnSelectionChanged)
			.PersistUIRowsOnClose(HierarchyViewerArgs._PersistUIRowsOnClose)
			.GetCustomTableViewerRowMapping(HierarchyViewerArgs._GetCustomTableViewerRowMapping)
			.ExpandNewRows(HierarchyViewerArgs._ExpandNewRows)
			.PrimaryColumn(HierarchyViewerArgs._PrimaryColumn)
			.AllowInvisibleItemSelection(HierarchyViewerArgs._AllowInvisibleItemSelection)
			.ShowWires(HierarchyViewerArgs._ShowWires);
		
		ContentWidget->AddSlot()
		[
			HierarchyViewer->AsWidget()
		];
		
		ChildSlot
		[
			ContentWidget.ToSharedRef()
		];
	}

	void STedsCompositeHierarchyViewer::SetQueryStack(TSharedPtr<QueryStack::IRowNode> InRowQueryStack)
	{
		// SetQueryStack is not yet supported in the composite viewer to recreate the search/filter query stack
		// The call is available from the base class ptr, but will only display the given QueryStack with no composite functionality
		ensureMsgf(false, TEXT("SetQueryStack is not supported in the Composite Viewer yet."));
		HierarchyViewer->SetQueryStack(InRowQueryStack);
	}

	void STedsCompositeHierarchyViewer::SetColumns(const TArray<TWeakObjectPtr<const UScriptStruct>>& InColumns)
	{
		HierarchyViewer->SetColumns(InColumns);
	}
	
	void STedsCompositeHierarchyViewer::SetTableViewerIdentifier(const FName& NewIdentifier)
    {
		HierarchyViewer->SetTableViewerIdentifier(NewIdentifier);
    }

	void STedsCompositeHierarchyViewer::AddCustomRowWidget(const TSharedRef<FTedsTableViewerColumn>& InColumn)
	{
		HierarchyViewer->AddCustomRowWidget(InColumn);
	}

	void STedsCompositeHierarchyViewer::ForEachSelectedRow(TFunctionRef<void(RowHandle)> InCallback) const
	{
		HierarchyViewer->ForEachSelectedRow(InCallback);
	}

	RowHandle STedsCompositeHierarchyViewer::GetWidgetRowHandle() const
	{
		return HierarchyViewer->GetWidgetRowHandle();
	}

	void STedsCompositeHierarchyViewer::SetSelection(RowHandle Row, bool bSelected, const ESelectInfo::Type SelectInfo) const
	{
		HierarchyViewer->SetSelection(Row, bSelected, SelectInfo);
	}

	void STedsCompositeHierarchyViewer::ScrollIntoView(RowHandle Row) const
	{
		HierarchyViewer->ScrollIntoView(Row);
	}

	void STedsCompositeHierarchyViewer::ClearSelection() const
	{
		HierarchyViewer->ClearSelection();
	}

	TSharedRef<SWidget> STedsCompositeHierarchyViewer::AsWidget()
	{
		return HierarchyViewer->AsWidget();
	}

	bool STedsCompositeHierarchyViewer::IsSelected(RowHandle InRow) const
	{
		return HierarchyViewer->IsSelected(InRow);
	}

	bool STedsCompositeHierarchyViewer::IsSelectedExclusively(RowHandle InRow) const
	{
		return HierarchyViewer->IsSelectedExclusively(InRow);
	}

	void STedsCompositeHierarchyViewer::AddFilter(const TSharedPtr<FTedsFilter> InFilter) const
	{
		if (FilterBar)
		{
			FilterBar->AddFilter(InFilter);
		}
	}

	void STedsCompositeHierarchyViewer::AddFilters(const TArray<TSharedPtr<FTedsFilter>>& InFilter) const
	{
		if (FilterBar)
		{
			FilterBar->AddFilters(InFilter);
		}
	}

	void STedsCompositeHierarchyViewer::DeleteFilter(const FName& InFilterName) const
	{
		if (FilterBar)
		{
			FilterBar->DeleteFilter(InFilterName);
		}
	}

	void STedsCompositeHierarchyViewer::DeleteAllFilters() const
	{
		if (FilterBar)
		{
			FilterBar->DeleteAllFilters();
		}
	}

	void STedsCompositeHierarchyViewer::SetCommonFilters(const TArray<FName>& InFilters) const
	{
		if (FilterBar)
		{
			FilterBar->SetCommonFilters(InFilters);
		}
	}

	void STedsCompositeHierarchyViewer::ReplaceFilters(const TArray<TSharedPtr<FTedsFilter>>& InFilters, const bool bPersistFilterStates /*= true*/) const
	{
		if (FilterBar)
		{
			FilterBar->ReplaceFilters(InFilters, bPersistFilterStates);
		}
	}

	void STedsCompositeHierarchyViewer::ExpandAll() const
	{
		HierarchyViewer->ExpandAll();
	}

	void STedsCompositeHierarchyViewer::CollapseAll() const
	{
		HierarchyViewer->CollapseAll();
	}

	void STedsCompositeHierarchyViewer::ExpandWithParents(RowHandle Row)
	{
		HierarchyViewer->ExpandWithParents(Row);
	}

	void STedsCompositeHierarchyViewer::SetDropHandler(TUniquePtr<FWidgetDropHandler> InDropHandler)
	{
		HierarchyViewer->SetDropHandler(MoveTemp(InDropHandler));
	}

	void STedsCompositeHierarchyViewer::AddFilteredParentRowsToQueryStack(TSharedPtr<QueryStack::IRowNode>* QueryStack, const TSharedPtr<IHierarchyViewerDataInterface>& HierarchyInterface)
	{
		if (bShowFilteredOutParentHierarchy)
		{
			ICoreProvider* Storage = nullptr;
			if (HierarchyViewer.IsValid())
			{
				if (const TSharedPtr<FTedsTableViewerModel> Model = HierarchyViewer->Model)
				{
					Storage = Model->GetDataStorageInterface();
				}
			}
			else
			{
				Storage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName);
			}

			if(Storage)
			{
				TSharedPtr<QueryStack::IRowNode> HierarchyNode =
					MakeShared<QueryStack::FAddParentHierarchyRowNode>(
						Storage,
						HierarchyInterface,
						*QueryStack);
				// Merge with a repeating approach with the original query node so we can make sure that we aren't adding back parents 
				// that weren't in the original query given to the viewer.
				*QueryStack = MakeShared<QueryStack::FRowMergeNode>(
					MakeConstArrayView({HierarchyNode, QueryNode}),
					QueryStack::FRowMergeNode::EMergeApproach::Repeating);
			}
		}
	}

	TSharedRef<SWidget> STedsCompositeHierarchyViewer::GetSettingsButtonContent()
	{
		TArray<TSharedPtr<FTedsCompositeHierarchyViewerOption>> HierarchyOptions;
		TArray<TSharedPtr<FTedsCompositeHierarchyViewerOption>> ShowOptions;
		HierarchyOptions.Reserve(Options.Num());
		ShowOptions.Reserve(Options.Num());
		
		for (TSharedPtr<FTedsCompositeHierarchyViewerOption>& Option : Options)
		{
			switch (Option->GetCategory())
			{
				case FTedsCompositeHierarchyViewerOption::EOptionCategories::Hierarchy:
					HierarchyOptions.Add(Option);
					break;
				case FTedsCompositeHierarchyViewerOption::EOptionCategories::Show:
					ShowOptions.Add(Option);
					break;
				default:
					break;
			}
		}
		
		FMenuBuilder MenuBuilder(true, nullptr);
		// HIERARCHY
		MenuBuilder.BeginSection("Hierarchy", LOCTEXT("HierarchyHeading", "Hierarchy"));
		// Default Hierarchy Actions
		{
			MenuBuilder.AddMenuEntry(
				LOCTEXT("ExpandAll", "Expand All"),
				LOCTEXT("ExpandAllTooltip", "Expand all items in the Hierarchy."),
				FSlateIcon(),
				FUIAction( FExecuteAction::CreateSP(this, &STedsCompositeHierarchyViewer::ExpandAll) )
				);
			MenuBuilder.AddMenuEntry(
				LOCTEXT("CollapseAll", "Collapse All"),
				LOCTEXT("CollapseAllTooltip", "Collapse all items in the Hierarchy."),
				FSlateIcon(),
				FUIAction( FExecuteAction::CreateSP(this, &STedsCompositeHierarchyViewer::CollapseAll) )
				);
		}
		
		for (const TSharedPtr<FTedsCompositeHierarchyViewerOption>& HierarchyOption : HierarchyOptions)
		{
			HierarchyOption->AddMenu(MenuBuilder);
		}

		MenuBuilder.EndSection();
		
		// SHOW
		MenuBuilder.BeginSection("Show", LOCTEXT("ShowHeading", "Show"));

		for (const TSharedPtr<FTedsCompositeHierarchyViewerOption>& ShowOption : ShowOptions)
		{
			ShowOption->AddMenu(MenuBuilder);
		}

		MenuBuilder.EndSection();

		return MenuBuilder.MakeWidget();
	}

	void STedsCompositeHierarchyViewer::OnToggleShowFilteredParentHierarchy()
	{
		bShowFilteredOutParentHierarchy = !bShowFilteredOutParentHierarchy;

		if (FilterNode.IsValid())
		{
			CompositeQueryNode = FilterNode;
		}
		else if (SearchNode.IsValid())
		{
			CompositeQueryNode = SearchNode;
		}
		else
		{
			return; // If we don't have a filter or search node, there is no point in adding back the parents
		}
		AddFilteredParentRowsToQueryStack(&CompositeQueryNode, HierarchyViewer->HierarchyInterface);
		HierarchyViewer->SetQueryStack(CompositeQueryNode);
	}

	void STedsCompositeHierarchyViewer::OnToggleExpandNewRows()
	{
		HierarchyViewer->bExpandNewRows = !HierarchyViewer->bExpandNewRows;
	}

} // namespace UE::Editor::DataStorage

#undef LOCTEXT_NAMESPACE
