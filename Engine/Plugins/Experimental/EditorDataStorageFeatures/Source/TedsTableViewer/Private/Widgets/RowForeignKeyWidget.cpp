// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/RowForeignKeyWidget.h"

#include "Elements/Columns/TypedElementMiscColumns.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Text/STextBlock.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RowForeignKeyWidget)

#define LOCTEXT_NAMESPACE "RowForeignKeyWidget"

namespace UE::Editor::DataStorage::Ui
{
	class SRowForeignKeyCellWidget : public SCompoundWidget
	{
	public:
		SLATE_BEGIN_ARGS(SRowForeignKeyCellWidget)
			: _DataStorage(nullptr)
			, _Row(InvalidRowHandle)
			{
			}

			SLATE_ATTRIBUTE(const ICoreProvider*, DataStorage)
			SLATE_ATTRIBUTE(RowHandle, Row)
		SLATE_END_ARGS()

		void Construct(const FArguments& InArgs)
		{
			DataStorage = InArgs._DataStorage.Get();
			Row = InArgs._Row.Get();
			Table = DataStorage->FindTable(Row);
			
			ChildSlot
				[
					SAssignNew(TextBlock, STextBlock)
						.ColorAndOpacity(FSlateColor::UseForeground())
						.OnDoubleClicked_Lambda(
							[this](const FGeometry&, const FPointerEvent&) 
							{ 
								SetText();
								return FReply::Handled();
							})
				];

			SetText();
		}

		void SetText()
		{
			FName Domain;
			int32 Counter = 0;
			// Note: Tables are never destroyed and foreign keys can't be unregistered. This means that the list of foreign key domains
			// for a given table is also immutable, so we can rely on the index to remain valid across calls.
			DataStorage->ListTableForeignKeyDomains(Table, true, [this, &Counter, &Domain](const FName& DomainName)
				{
					if (Counter == Index)
					{
						Domain = DomainName;
					}
					Counter++;
				});
			if (Counter > 0 && !Domain.IsNone())
			{
				Index++;
				if (Index >= Counter)
				{
					Index = 0;
				}

				FString Text = Domain.ToString();
				Text.Append(TEXT(": "));
				FForeignKey Key = DataStorage->BuildForeignKey(Domain, Row);
				Text.Append(Key.ToString());
				TextBlock->SetText(FText::FromString(MoveTemp(Text)));
			}
			else
			{
				TextBlock->SetText(FText::GetEmpty());
			}
		}

	private:
		TSharedPtr<STextBlock> TextBlock;
		const ICoreProvider* DataStorage = nullptr;
		RowHandle Row = InvalidRowHandle;
		TableHandle Table = InvalidTableHandle;
		int32 Index = 0;
	};
} // namespace UE::Editor::DataStorage::Ui

void URowForeignKeyWidgetFactory::RegisterWidgetConstructors(UE::Editor::DataStorage::ICoreProvider& DataStorage, UE::Editor::DataStorage::IUiProvider& DataStorageUi) const
{
	DataStorageUi.RegisterWidgetFactory(DataStorageUi.FindPurpose(
		UE::Editor::DataStorage::IUiProvider::FPurposeInfo("General", "Cell", "RowForeignKey").GeneratePurposeID()),
		FRowForeignKeyWidgetConstructor::StaticStruct());
}

void URowForeignKeyWidgetFactory::RegisterWidgetPurposes(UE::Editor::DataStorage::IUiProvider& DataStorageUi) const
{
	DataStorageUi.RegisterWidgetPurpose(
		UE::Editor::DataStorage::IUiProvider::FPurposeInfo("General", "Cell", "RowForeignKey",
			UE::Editor::DataStorage::IUiProvider::EPurposeType::UniqueByName,
			LOCTEXT("GeneralRowForeignKeyPurpose", "Specific purpose to request a widget to display row foreign keys.")));
}

FRowForeignKeyWidgetConstructor::FRowForeignKeyWidgetConstructor()
	: Super(FRowForeignKeyWidgetConstructor::StaticStruct())
{
	
}

TSharedPtr<SWidget> FRowForeignKeyWidgetConstructor::CreateWidget(const UE::Editor::DataStorage::FMetaDataView& Arguments)
{
	return SNew(SBox)
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Center)
		.Padding(8, 0, 0, 0);}

bool FRowForeignKeyWidgetConstructor::FinalizeWidget(UE::Editor::DataStorage::ICoreProvider* DataStorage, 
	UE::Editor::DataStorage::IUiProvider* DataStorageUi, UE::Editor::DataStorage::RowHandle Row, const TSharedPtr<SWidget>& Widget)
{
	using namespace UE::Editor::DataStorage::Ui;

	checkf(Widget->GetType() == SBox::StaticWidgetClass().GetWidgetType(),
		TEXT("Stored widget with FRowForeignKeyWidgetConstructor doesn't match type %s, but was a %s."),
		*(SBox::StaticWidgetClass().GetWidgetType().ToString()),
		*(Widget->GetTypeAsString()));
	
	SBox* BoxWidget = static_cast<SBox*>(Widget.Get());

	UE::Editor::DataStorage::RowHandle TargetRowHandle = UE::Editor::DataStorage::InvalidRowHandle;

	if (const FTypedElementRowReferenceColumn* RowReferenceColumn = DataStorage->GetColumn<FTypedElementRowReferenceColumn>(Row))
	{
		BoxWidget->SetContent(
			SNew(SRowForeignKeyCellWidget)
			.DataStorage(DataStorage)
			.Row(RowReferenceColumn->Row));
	}
	else
	{
		BoxWidget->SetContent(SNullWidget::NullWidget);
	}
	return true;
}

FText FRowForeignKeyWidgetConstructor::CreateWidgetDisplayNameText(UE::Editor::DataStorage::ICoreProvider* DataStorage,
	UE::Editor::DataStorage::RowHandle Row) const
{
	return LOCTEXT("RowForeignKeyColumnName", "Row Foreign Key");
}

#undef LOCTEXT_NAMESPACE //"RowForeignKeyWidget"
