// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/RowHandleWidget.h"

#include "Elements/Columns/TypedElementMiscColumns.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Text/STextBlock.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(RowHandleWidget)

#define LOCTEXT_NAMESPACE "RowHandleWidget"

namespace UE::Editor::DataStorage
{
	//
	// FRowHandleSorter
	//

	int32 FRowHandleSorter::Compare(const ICoreProvider& Storage, RowHandle Left, RowHandle Right) const
	{
		return (Left > Right) - (Left < Right);
	}

	FPrefixInfo FRowHandleSorter::CalculatePrefix(const ICoreProvider& Storage, RowHandle Row, uint32 ByteIndex) const
	{
		return CreateSortPrefix(ByteIndex, Row);
	}

	FColumnSorterInterface::ESortType FRowHandleSorter::GetSortType() const
	{
		return ESortType::FixedSize64;
	}

	FText FRowHandleSorter::GetShortName() const
	{
		return FText::GetEmpty();
	}
}

void URowHandleWidgetFactory::RegisterWidgetConstructors(UE::Editor::DataStorage::ICoreProvider& DataStorage, UE::Editor::DataStorage::IUiProvider& DataStorageUi) const
{
	DataStorageUi.RegisterWidgetFactory(DataStorageUi.FindPurpose(
		UE::Editor::DataStorage::IUiProvider::FPurposeInfo("General", "Cell", "RowHandle").GeneratePurposeID()),
		FRowHandleWidgetConstructor::StaticStruct());
}

void URowHandleWidgetFactory::RegisterWidgetPurposes(UE::Editor::DataStorage::IUiProvider& DataStorageUi) const
{
	DataStorageUi.RegisterWidgetPurpose(
		UE::Editor::DataStorage::IUiProvider::FPurposeInfo("General", "Cell", "RowHandle",
			UE::Editor::DataStorage::IUiProvider::EPurposeType::UniqueByName,
			LOCTEXT("GeneralRowHandlePurpose", "Specific purpose to request a widget to display row handles.")));

	DataStorageUi.RegisterWidgetPurpose(
		UE::Editor::DataStorage::IUiProvider::FPurposeInfo("RowDetails", "Cell", NAME_None,
			UE::Editor::DataStorage::IUiProvider::EPurposeType::UniqueByName,
			LOCTEXT("DetailsRowHandlePurpose", "Specific purpose to request a widget to display the details on a row (e.g SRowDetails)."),
			DataStorageUi.GetDefaultWidgetPurposeID()));
	
	UE::Editor::DataStorage::IUiProvider::FPurposeID GeneralLargePurposeID =
		UE::Editor::DataStorage::IUiProvider::FPurposeInfo("General", "Cell", "Large").GeneratePurposeID();
	
	DataStorageUi.RegisterWidgetPurpose(
		UE::Editor::DataStorage::IUiProvider::FPurposeInfo("RowDetails", "Cell", "Large",
			UE::Editor::DataStorage::IUiProvider::EPurposeType::UniqueByNameAndColumn,
			LOCTEXT("GeneralRowHandlePurpose", "Specific purpose to request a widget to display row handles."),
			GeneralLargePurposeID));
}

FRowHandleWidgetConstructor::FRowHandleWidgetConstructor()
	: Super(FRowHandleWidgetConstructor::StaticStruct())
{
	
}

TSharedPtr<SWidget> FRowHandleWidgetConstructor::CreateWidget(const UE::Editor::DataStorage::FMetaDataView& Arguments)
{
	return SNew(SBox)
		.HAlign(HAlign_Left)
		.VAlign(VAlign_Center)
		.Padding(8, 0, 0, 0);}

bool FRowHandleWidgetConstructor::FinalizeWidget(UE::Editor::DataStorage::ICoreProvider* DataStorage, UE::Editor::DataStorage::IUiProvider* DataStorageUi, UE::Editor::DataStorage::RowHandle Row, const TSharedPtr<SWidget>& Widget)
{

	checkf(Widget->GetType() == SBox::StaticWidgetClass().GetWidgetType(),
		TEXT("Stored widget with FRowHandleWidgetConstructor doesn't match type %s, but was a %s."),
		*(SBox::StaticWidgetClass().GetWidgetType().ToString()),
		*(Widget->GetTypeAsString()));
	
	SBox* BoxWidget = static_cast<SBox*>(Widget.Get());

	UE::Editor::DataStorage::RowHandle TargetRowHandle = UE::Editor::DataStorage::InvalidRowHandle;

	if(const FTypedElementRowReferenceColumn* RowReferenceColumn = DataStorage->GetColumn<FTypedElementRowReferenceColumn>(Row))
	{
		TargetRowHandle = RowReferenceColumn->Row;
	}
	
	FNumberFormattingOptions NumberFormattingOptions;
	NumberFormattingOptions.SetUseGrouping(false);
	const FText Text = FText::AsNumber(TargetRowHandle, &NumberFormattingOptions);

	BoxWidget->SetContent(
			SNew(STextBlock)
				.Text(Text)
				.ColorAndOpacity(FSlateColor::UseForeground())
		);
	return true;
}

FText FRowHandleWidgetConstructor::CreateWidgetDisplayNameText(UE::Editor::DataStorage::ICoreProvider* DataStorage, 
	UE::Editor::DataStorage::RowHandle Row) const
{
	return LOCTEXT("RowHandleColumnName", "Row Handle");
}

TArray<TSharedPtr<const UE::Editor::DataStorage::FColumnSorterInterface>> FRowHandleWidgetConstructor::ConstructColumnSorters(
	UE::Editor::DataStorage::ICoreProvider* DataStorage,
	UE::Editor::DataStorage::IUiProvider* DataStorageUi,
	const UE::Editor::DataStorage::FMetaDataView& Arguments)
{
	using namespace UE::Editor::DataStorage;
	return TArray<TSharedPtr<const FColumnSorterInterface>>(
		{
			MakeShared<FRowHandleSorter>()
		});
}

#undef LOCTEXT_NAMESPACE //"RowHandleWidget"
