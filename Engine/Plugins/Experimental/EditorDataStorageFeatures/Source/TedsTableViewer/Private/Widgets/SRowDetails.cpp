// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SRowDetails.h"

#include "TedsTableViewerUtils.h"
#include "DataStorage/Features.h"
#include "Elements/Columns/TypedElementMiscColumns.h"
#include "Elements/Columns/TypedElementTypeInfoColumns.h"
#include "Elements/Interfaces/TypedElementDataStorageInterface.h"
#include "Elements/Interfaces/TypedElementDataStorageUiInterface.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/SListView.h"

#define LOCTEXT_NAMESPACE "SRowDetails"

namespace UE::Editor::DataStorage
{
	//
	// SRowDetails
	//

	const FName SRowDetails::NameColumn = "Name";
	const FName SRowDetails::DataColumn = "Data";

	void SRowDetails::Construct(const FArguments& InArgs)
	{
		bShowAllDetails = InArgs._ShowAllDetails;
		bShowUnpopulatedColumns = InArgs._ShowUnpopulatedColumns;

		WidgetPurpose = InArgs._WidgetPurposeOverride;
		if(!WidgetPurpose.IsSet())
		{
			WidgetPurpose = IUiProvider::FPurposeInfo("RowDetails", "Cell", "Large").GeneratePurposeID();
		}

		checkf(AreEditorDataStorageFeaturesEnabled(), TEXT("Unable to initialize SRowDetails without the editor data storage interfaces."));

		DataStorage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName);
		DataStorageUi = GetMutableDataStorageFeature<IUiProvider>(UiFeatureName);
		check(DataStorage);
		check(DataStorageUi);

		OnRelatedRowSelectedDelegate = InArgs._OnRelatedRowSelected;
		RequestedTedsColumns = InArgs._Columns;
		NameMetaData = InArgs._NameMetaData;
		DataMetaData = InArgs._DataMetaData;
		ColumnMetaData = InArgs._ColumnMetaData;
		RowPadding = InArgs._RowPadding;

		ChildSlot
		[
			SAssignNew(ListView, SListView<RowDetailsItemPtr>)
				.ListItemsSource(&Items)
				.SelectionMode(InArgs._SelectionMode)
				.OnGenerateRow(this, &SRowDetails::CreateRow)
				.Visibility_Lambda([this]()
					{
						return Items.IsEmpty() ? EVisibility::Hidden : EVisibility::Visible;
					})
				.HeaderRow
				(
					InArgs._HeaderRow
						? InArgs._HeaderRow.ToSharedRef()
						: SNew(SHeaderRow)
						+SHeaderRow::Column(NameColumn)
							.DefaultLabel(FText::FromString(TEXT("Name")))
							.FillWidth(0.3f)
						+SHeaderRow::Column(DataColumn)
							.DefaultLabel(FText::FromString(TEXT("Value")))
							.FillWidth(0.7f)
				)
		];
	}
	
	void SRowDetails::SetRow(RowHandle Row)
	{
		ClearRow();

		if (!DataStorage->IsRowAssigned(Row))
		{
			return;
		}

		// A Map of TEDS Columns -> UI columns so we can add them in the same order they were specified
		TMap<TWeakObjectPtr<const UScriptStruct>, TSharedRef<FRowDetailsItem>> NewColumnMap;

		TArray<TWeakObjectPtr<const UScriptStruct>> Columns = RequestedTedsColumns;
		if (Columns.IsEmpty())
		{
			DataStorage->ListColumns(Row, [&Columns](const UScriptStruct& ColumnType)
			{
				Columns.Emplace(&ColumnType);
				return true;
			});
			// Sorting by display name by default
			Columns.Sort(
				[](const TWeakObjectPtr<const UScriptStruct>& Lhs, const TWeakObjectPtr<const UScriptStruct>& Rhs)
				{
					return Lhs->GetDisplayNameText().CompareTo(Rhs->GetDisplayNameText()) < 0;
				});
		}
		TArray<TWeakObjectPtr<const UScriptStruct>> ColumnsCopy = Columns;

		RowHandle PurposeRow = DataStorageUi->FindPurpose(WidgetPurpose);

		static const FMetaData EmptyMetaData;
		// Using the DataMetaData here because the widget constructor will be used for the Data column
		FComboMetaDataView<FGenericMetaDataView, FColumnsMetaDataView> MetaDataView =
			FComboMetaDataView(FGenericMetaDataView(DataMetaData ? *DataMetaData : EmptyMetaData))
			.Next(FColumnsMetaDataView(ColumnMetaData));
		
		int32 IndexOffset = 0;

		// Create the widget constructors for the columns
		DataStorageUi->CreateWidgetConstructors(PurposeRow, IUiProvider::EMatchApproach::LongestMatch, ColumnsCopy,
			MetaDataView, [this, Row, &NewColumnMap, &IndexOffset](
			IUiProvider::FWidgetConstructorPtr WidgetConstructor, TConstArrayView<TWeakObjectPtr<const UScriptStruct>> MatchedColumns)
			{
				if (!bShowUnpopulatedColumns && !DataStorage->HasColumns(Row, MatchedColumns))
				{
					return true;
				}
					
				TArray<TWeakObjectPtr<const UScriptStruct>> MatchedColumnsCopy(MatchedColumns);
				FName NameId = TableViewerUtils::FindLongestMatchingName(MatchedColumns, IndexOffset);
				TSharedRef<FRowDetailsItem> Item = MakeShared<FRowDetailsItem>(NameId, MoveTemp(MatchedColumnsCopy), WidgetConstructor, Row);
					
				for (const TWeakObjectPtr<const UScriptStruct>& Column : MatchedColumns)
				{
					NewColumnMap.Emplace(Column, Item);
				}
					
				++IndexOffset;
				return true;
			});

			if (bShowAllDetails)
			{
				// ColumnsCopy has been modified in-place by CreateWidgetConstructors above, which removes matched columns.
				// Only unmatched columns remain here.
				for (TWeakObjectPtr<const UScriptStruct> Column : ColumnsCopy)
				{
					DataStorageUi->CreateWidgetConstructors(DataStorageUi->FindPurpose(DataStorageUi->GetDefaultWidgetPurposeID()),
						MetaDataView, [this, Column, Row, &NewColumnMap, &IndexOffset](
							IUiProvider::FWidgetConstructorPtr Constructor, TConstArrayView<TWeakObjectPtr<const UScriptStruct>> MatchedColumns)
						{
							FName NameId = TableViewerUtils::FindLongestMatchingName({Column}, IndexOffset);
							NewColumnMap.Emplace(Column, MakeShared<FRowDetailsItem>(NameId, TArray{Column}, MoveTemp(Constructor), Row));
							++IndexOffset;
							return true;
						});
				}
			}

		// Add the actual UI columns in the order the Teds Columns were specified
		for (const TWeakObjectPtr<const UScriptStruct>& ColumnType : Columns)
		{
			if (const TSharedRef<FRowDetailsItem>* FoundColumn = NewColumnMap.Find(ColumnType))
			{
				// If the column already exists, a widget matched it and a previously encountered column together and was already added
				// so we can safely ignore it here
				if(!HasItem((*FoundColumn)->Name))
				{
					Items.Add(*FoundColumn);
				}
			}
		}

		// Add relation items: for each relation type, list this row's related rows as navigable entries.
		DataStorage->ListRelationTypes([this, Row](RelationTypeHandle Type, const FName& TypeName)
		{
			TArray<RowHandle> Objects;
			DataStorage->GetRelationObjects(Type, Row, Objects);
			for (RowHandle Object : Objects)
			{
				FName ItemName = FName(*FString::Printf(TEXT("%s (Object)"), *TypeName.ToString()));
				Items.Add(MakeShared<FRowDetailsItem>(ItemName, Row, Object));
			}

			TArray<RowHandle> Subjects;
			DataStorage->GetRelationSubjects(Type, Row, Subjects);
			for (RowHandle Subject : Subjects)
			{
				FName ItemName = FName(*FString::Printf(TEXT("%s (Subject)"), *TypeName.ToString()));
				Items.Add(MakeShared<FRowDetailsItem>(ItemName, Row, Subject));
			}
		});

		ListView->RequestListRefresh();
	}

	void SRowDetails::ClearRow()
	{
		for (const RowDetailsItemPtr& Item : Items)
		{
			if (DataStorage->IsRowAvailable(Item->WidgetRow))
			{
				DataStorage->RemoveRow(Item->WidgetRow);
			}
		}
		Items.Reset();
		ListView->RequestListRefresh();
	}
	
	TSharedRef<ITableRow> SRowDetails::CreateRow(RowDetailsItemPtr InItem, const TSharedRef<STableViewBase>& OwnerTable)
	{
		TSharedRef<SRowDetailsRow> Row = SNew(SRowDetailsRow, OwnerTable, SharedThis(this))
			.Item(MoveTemp(InItem))
			.Padding(RowPadding);

		return Row;
	}

	bool SRowDetails::HasItem(const FName& ItemName) const
	{
		return Items.FindByPredicate([ItemName]
			(const RowDetailsItemPtr& InItem)
		{
			return InItem->Name == ItemName;
		}) != nullptr;
	}

	//
	// SRowDetailsRow
	//
	void SRowDetailsRow::Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& OwnerTableView, const TSharedRef<SRowDetails>& InOwnerRowDetails)
	{
		Item = InArgs._Item;
		check(Item);
		OwnerRowDetails = InOwnerRowDetails;

		SMultiColumnTableRow<RowDetailsItemPtr>::Construct(FSuperRowType::FArguments().Padding(InArgs._Padding), OwnerTableView);
	}
	
	TSharedRef<SWidget> SRowDetailsRow::GenerateWidgetForColumn(const FName& ColumnName)
	{
		TSharedPtr<SRowDetails> RowDetails = OwnerRowDetails.Pin();
		if (!RowDetails)
		{
			return SNullWidget::NullWidget;
		}

		// Relation items have their own rendering -- no widget row needed.
		if (Item->IsRelationItem())
		{
			if (ColumnName == SRowDetails::NameColumn)
			{
				return SNew(STextBlock).Text(FText::FromName(Item->Name));
			}
			else if (ColumnName == SRowDetails::DataColumn)
			{
				const RowHandle RelatedRow = Item->RelatedRow;
				return SNew(SButton)
					.Text(FText::FromString(FString::Printf(TEXT("Row %llu"), RelatedRow)))
					.OnClicked_Lambda([RowDetails, RelatedRow]()
					{
						RowDetails->NotifyRelatedRowSelected(RelatedRow);
						return FReply::Handled();
					});
			}
			return SNullWidget::NullWidget;
		}

		ICoreProvider* DataStorage = RowDetails->GetDataStorage();
		if (!DataStorage->IsRowAvailable(Item->WidgetRow))
		{
			Item->WidgetRow = DataStorage->AddRow(DataStorage->FindTable(FName("Editor_WidgetTable")));

			DataStorage->AddColumn<FTypedElementRowReferenceColumn>(Item->WidgetRow, FTypedElementRowReferenceColumn
				{
					.Row = Item->Row
				});

			if (Item->ColumnTypes.Num() &&
				Item->WidgetConstructor->GetAdditionalColumnsList().Contains(FTypedElementScriptStructTypeInfoColumn::StaticStruct()))
			{
				// FTypedElementScriptStructTypeInfoColumn can only store one column type but we can match against multiple columns, for now
				// we just store the first one
				DataStorage->AddColumn(Item->WidgetRow, FTypedElementScriptStructTypeInfoColumn
					{
						.TypeInfo = Item->ColumnTypes[0]
					});
			}
		}

		static const FMetaData EmptyMetaData;
		if (ColumnName == SRowDetails::NameColumn)
		{
			FComboMetaDataView<FGenericMetaDataView, FColumnsMetaDataView> MetaDataView =
				FComboMetaDataView(FGenericMetaDataView(RowDetails->GetNameMetaData() ? *RowDetails->GetNameMetaData() : EmptyMetaData))
				.Next(FColumnsMetaDataView(RowDetails->GetColumnMetaData()));

			STextBlock::FArguments Args = STextBlock::FArguments();
			if (const FString* const* FontStyle = MetaDataView.FindForColumnsOrGeneric(Item->ColumnTypes, "Font").TryGetExact<const FString*>())
			{
				Args._Font = FAppStyle::Get().GetFontStyle(FName(**FontStyle));
			}

			return SArgumentNew(Args, STextBlock)
				.Text(Item->WidgetConstructor->CreateWidgetDisplayNameText(DataStorage, Item->WidgetRow));
		}
		else if (ColumnName == SRowDetails::DataColumn)
		{
			FComboMetaDataView<FGenericMetaDataView, FColumnsMetaDataView> MetaDataView =
				FComboMetaDataView(FGenericMetaDataView(RowDetails->GetDataMetaData() ? *RowDetails->GetDataMetaData() : EmptyMetaData))
				.Next(FColumnsMetaDataView(RowDetails->GetColumnMetaData()));

			const TSharedPtr<SWidget> Widget = RowDetails->GetDataStorageUi()->ConstructWidget(Item->WidgetRow, *(Item->WidgetConstructor), MetaDataView);
			return Widget ? Widget.ToSharedRef() : SNullWidget::NullWidget;
		}
		else
		{
			return SNew(STextBlock)
					.Text(LOCTEXT("InvalidColumnType", "Invalid Column Type"));
		}
	}

	void SRowDetails::NotifyRelatedRowSelected(RowHandle Row)
	{
		OnRelatedRowSelectedDelegate.ExecuteIfBound(Row);
	}

	// FRowDetailsItem
	FRowDetailsItem::FRowDetailsItem(const FName& InItemName, const TArray<TWeakObjectPtr<const UScriptStruct>>& InColumnTypes,
		IUiProvider::FWidgetConstructorPtr InWidgetConstructor, RowHandle InRow)
		: Name(InItemName)
		, ColumnTypes(InColumnTypes)
		, WidgetConstructor(MoveTemp(InWidgetConstructor))
		, Row(InRow)
	{
	}

	FRowDetailsItem::FRowDetailsItem(const FName& InRelationName, RowHandle InRow, RowHandle InRelatedRow)
		: Name(InRelationName)
		, Row(InRow)
		, RelatedRow(InRelatedRow)
	{
	}
	
} // namespace UE::Editor::DataStorage


#undef LOCTEXT_NAMESPACE // "SRowDetails"
