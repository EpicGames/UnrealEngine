// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SRowDetailsNavigator.h"
#include "Widgets/SRowDetails.h"
#include "Widgets/Layout/SWidgetSwitcher.h"
#include "Widgets/Navigation/SBreadcrumbTrail.h"

namespace UE::Editor::DataStorage
{

void SRowDetailsNavigator::Construct(const FArguments& InArgs)
{
	ChildSlot
	[
		SNew(SVerticalBox)
		+SVerticalBox::Slot()
		.AutoHeight()
		[
			SAssignNew(Breadcrumbs, SBreadcrumbTrail<RowHandle>)
				.OnCrumbClicked(this, &SRowDetailsNavigator::OnBreadcrumbClicked)
				.GetCrumbButtonContent_Lambda([](RowHandle Row, const FTextBlockStyle* TextStyle) -> TSharedRef<SWidget>
				{
					return SNew(STextBlock)
						.TextStyle(TextStyle)
						.Text(FText::FromString(FString::Printf(TEXT("Row %llu"), Row)));
				})
		]
		+SVerticalBox::Slot()
		.FillHeight(1.0f)
		[
			SAssignNew(PanelSwitcher, SWidgetSwitcher)
		]
	];
}

void SRowDetailsNavigator::SetRow(RowHandle Row)
{
	TruncateStack(0);

	TSharedRef<SRowDetails> Panel = CreatePanel();
	PanelStack.Add(Panel);
	PanelSwitcher->AddSlot()[Panel];
	PanelSwitcher->SetActiveWidgetIndex(0);

	Panel->SetRow(Row);
	Breadcrumbs->PushCrumb(FText::FromString(FString::Printf(TEXT("Row %llu"), Row)), Row);
}

void SRowDetailsNavigator::ClearRow()
{
	TruncateStack(0);
}

void SRowDetailsNavigator::NavigateTo(RowHandle Row)
{
	TSharedRef<SRowDetails> Panel = CreatePanel();
	PanelStack.Add(Panel);
	PanelSwitcher->AddSlot()[Panel];

	const int32 NewIndex = PanelStack.Num() - 1;
	PanelSwitcher->SetActiveWidgetIndex(NewIndex);

	Panel->SetRow(Row);
	Breadcrumbs->PushCrumb(FText::FromString(FString::Printf(TEXT("Row %llu"), Row)), Row);
}

void SRowDetailsNavigator::OnBreadcrumbClicked(const RowHandle& Row)
{
	// SBreadcrumbTrail already popped all crumbs after the clicked one.
	// Sync the panel stack to match.
	const int32 TargetCount = Breadcrumbs->NumCrumbs();
	TruncateStack(TargetCount);

	if (TargetCount > 0)
	{
		PanelSwitcher->SetActiveWidgetIndex(TargetCount - 1);
	}
}

TSharedRef<SRowDetails> SRowDetailsNavigator::CreatePanel()
{
	return SNew(SRowDetails)
		.OnRelatedRowSelected_Lambda([this](RowHandle RelatedRow)
		{
			NavigateTo(RelatedRow);
		});
}

void SRowDetailsNavigator::TruncateStack(int32 KeepCount)
{
	while (PanelStack.Num() > KeepCount)
	{
		PanelSwitcher->RemoveSlot(PanelStack.Last().ToSharedRef());
		PanelStack.Pop();
	}

	if (KeepCount == 0)
	{
		Breadcrumbs->ClearCrumbs();
	}
}

} // namespace UE::Editor::DataStorage
