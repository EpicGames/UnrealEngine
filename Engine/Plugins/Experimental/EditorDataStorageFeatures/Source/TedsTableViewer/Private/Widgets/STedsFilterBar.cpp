// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/STedsFilterBar.h"

#include "TedsQueryHandleNode.h"
#include "TedsQueryMergeNode.h"
#include "TedsRowFilterNode.h"
#include "TedsRowMergeNode.h"
#include "TedsRowQueryResultsNode.h"

#define LOCTEXT_NAMESPACE "STedsFilterBar"

namespace UE::Editor::DataStorage
{
	void STedsFilterBar::Construct(const FArguments& InArgs)
	{
		using namespace DataStorage::QueryStack;

		OnPostFiltersChanged = InArgs._OnPostFiltersChanged;
		CommonSectionFilters = InArgs._CommonSectionFilters;

		typename SBasicFilterBar<FTedsRowHandle&>::FArguments Args;
		Args._OnFilterChanged = InArgs._OnFilterChanged;
		Args._FilterBarLayout = InArgs._FilterBarLayout;
		Args._FilterPillStyle = InArgs._FilterPillStyle;
		Args._UseSectionsForCategories = InArgs._UseSectionsForCategories;

		TSharedRef<STedsFilterBar> STedsFilterBarRef = SharedThis(this);
		Storage = GetMutableDataStorageFeature<DataStorage::ICoreProvider>(StorageFeatureName);

		OtherFiltersCategory = MakeShared<FFilterCategory>(LOCTEXT("OtherFilters", "Other Filters"), LOCTEXT("OtherFiltersToolTip", "Other Filters"));
		for (const TSharedPtr<FTedsFilter>& Filter : InArgs._Filters)
		{
			if (Filter)
			{
				if (!Filter->GetCategory())
				{
					Filter->SetCategory(OtherFiltersCategory);
				}
				Filter->SetTedsFilterBar(STedsFilterBarRef);
				Args._CustomFilters.Add(Filter.ToSharedRef());
			}
		}
		
		if(ensureMsgf(InArgs._OutFilteredNode && InArgs._InFilterableRowNode, TEXT("In the TEDSFilterBar, a valid Row Node must be passed in to search on and a valid Out Node pointer must be given")))
		{
			InFilterableRowNode = InArgs._InFilterableRowNode;
			OutFilteredNode = InArgs._OutFilteredNode;
			check(OutFilteredNode);
			// Immediately compile the Query so the OutFilteredNode can be set
			RecompileQueries();
		}
		
		SBasicFilterBar<FTedsRowHandle&>::Construct(Args);
	}

	void STedsFilterBar::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
	{
		SBasicFilterBar<FTedsRowHandle&>::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);
		if (bMarkDirty)
		{
			RecompileQueries();
		}
	}

	TSharedPtr<FTedsFilter> STedsFilterBar::AddFilter(const TSharedPtr<FTedsFilter> InFilter)
	{
		if (ensureMsgf(InFilter, TEXT("The given TedsFilter was invalid and cannot be added.")))
		{
			if (!InFilter->GetCategory())
			{
				InFilter->SetCategory(OtherFiltersCategory);
			}
			
			if (const TSharedPtr<FFilterBase<FTedsRowHandle&>> FoundFilter = GetFilter(InFilter->GetName()))
			{
				// If we did find an existing filter with that name, delete the old version so we can overwrite it
				DeleteFilter(InFilter->GetIdentifier());
			}

			InFilter->SetTedsFilterBar(SharedThis(this));

			SBasicFilterBar<FTedsRowHandle&>::AddFilter(InFilter.ToSharedRef());
			return InFilter;
		}
		return nullptr;
	}

	void STedsFilterBar::AddFilters(const TArray<TSharedPtr<FTedsFilter>>& InFilters)
	{
		for (const TSharedPtr<FTedsFilter>& FilterData : InFilters)
		{
			AddFilter(FilterData);
		}
	}

	void STedsFilterBar::DeleteFilter(const FName& InFilterName)
	{
		if (const TSharedPtr<FFilterBase<FTedsRowHandle&>> FilterToRemove = GetFilter(InFilterName.ToString()))
		{
			// Check if the filter is active and remove from any active filter maps
			if (IsFilterActive(FilterToRemove.ToSharedRef()))
			{
				RemoveExternalQuery(InFilterName);
				RemoveExternalQueryFunction(InFilterName);
				RemoveClassQueryFunction(InFilterName);
			}
			RemoveFilter(FilterToRemove.ToSharedRef());
			AllFrontendFilters.Remove(FilterToRemove.ToSharedRef());
		}
	}

	void STedsFilterBar::DeleteAllFilters()
	{
		// Delete all filters and empty all active queries
		ExternalQueries.Empty();
		ExternalQueryFunctions.Empty();
		ClassFilters.Empty();

		RemoveAllFilters();
		AllFilterCategories.Empty();
		AllFrontendFilters.Empty();
	}

	void STedsFilterBar::SetCommonFilters(const TArray<FName>& InFilters)
	{
		CommonSectionFilters = InFilters;
	}

	void STedsFilterBar::ReplaceFilters(const TArray<TSharedPtr<FTedsFilter>>& InFilters, const bool bPersistFilterStates /*= true*/)
	{
		// Find all current filters on the filter bar and which ones are active so that state can be persisted if 
		// filters with the same name are re-added
		TMap<FString, bool> PreviousFilterStates;
		if (bPersistFilterStates)
		{
			for(const TSharedRef<SFilter>& FilterOnBar : Filters)
			{
				PreviousFilterStates.Add(FilterOnBar->GetFilterName(), FilterOnBar->IsEnabled());
			}
		}
		
		DeleteAllFilters();

		// Add the new given filters
		for (const TSharedPtr<FTedsFilter>& Filter : InFilters)
		{
			if (const TSharedPtr<FTedsFilter> NewFilter = AddFilter(Filter))
			{
				if (const bool* FilterState = PreviousFilterStates.Find(NewFilter->GetName()))
				{
					const TSharedRef<SFilter> AddedFilter = AddFilterToBar(NewFilter.ToSharedRef());
					if (*FilterState) // Enabled defaults to false when a filter is created so only perform if we need to enable
					{
						AddedFilter->SetEnabled(true);
						SetFrontendFilterActive(NewFilter.ToSharedRef(), true);
					}
				}
			}
		}
	}
	
	void STedsFilterBar::PopulateAddFilterMenu(UToolMenu* InMenu)
	{
		PopulateCommonFilterSections(InMenu);
		PopulateCustomFilters(InMenu);
	}

	void STedsFilterBar::PopulateCommonFilterSections(UToolMenu* InMenu)
	{
		FToolMenuSection& ActionSection = InMenu->AddSection("FilterBarResetFilters");
		ActionSection.AddMenuEntry(
			"ResetFilters",
			LOCTEXT("FilterListResetFilters", "Reset Filters"),
			LOCTEXT("FilterListResetToolTip", "Resets current pinned filters"),
			FSlateIcon(FAppStyle::Get().GetStyleSetName(), "PropertyWindow.DiffersFromDefault"),
			FUIAction(
				FExecuteAction::CreateSP(this, &STedsFilterBar::OnResetFilters),
				FCanExecuteAction::CreateLambda([this]() { return HasAnyFilters(); })
			)
		);

		if (CommonSectionFilters.IsEmpty())
		{
			ActionSection.AddSeparator("TedsActionsFiltersSeparator");
		}
		else
		{
			FToolMenuSection& CommonFiltersSection = InMenu->AddSection("FilterBarCommonFilters", LOCTEXT("TedsFilterBarCommonFiltersSection", "Common Filters"));
			for(const FName& CommonFilter : CommonSectionFilters)
			{
				if (const TSharedPtr<FFilterBase<FTedsRowHandle&>>& FrontendFilter = GetFilter(CommonFilter.ToString()))
				{
					CommonFiltersSection.AddMenuEntry(
						NAME_None,
						FrontendFilter->GetDisplayName(),
						FrontendFilter->GetToolTipText(),
						FSlateIcon(FAppStyle::GetAppStyleSetName(), FrontendFilter->GetIconName()),
						FUIAction(
						FExecuteAction::CreateSP(this, &STedsFilterBar::FrontendFilterClicked, FrontendFilter.ToSharedRef()),
						FCanExecuteAction(),
						FIsActionChecked::CreateSP(this, &STedsFilterBar::IsFrontendFilterInUse, FrontendFilter.ToSharedRef())),
						EUserInterfaceActionType::ToggleButton
						);
				}
			}
			CommonFiltersSection.AddSeparator("TedsCommonFiltersSeparator");
		}
	}

	void STedsFilterBar::PopulateCustomFilters(UToolMenu* InMenu)
	{
		if (bUseSectionsForCategories)
		{
			// Add all the filters as sections
			for (const TSharedPtr<FFilterCategory>& Category : AllFilterCategories)
			{
				FToolMenuSection& Section = InMenu->AddSection(*Category->Title.ToString(), Category->Title);
				CreateOtherFiltersMenuCategory(Section, Category);
			}
		}
		else
		{
			FToolMenuSection& Section = InMenu->AddSection("TEDSFilterBarMainSection");

			// Add all the filters as submenus
			for (const TSharedPtr<FFilterCategory>& Category : AllFilterCategories)
			{
				Section.AddSubMenu(
					NAME_None,
					Category->Title,
					Category->Tooltip,
					FNewToolMenuDelegate::CreateSP(this, &STedsFilterBar::CreateOtherFiltersMenuCategory, Category),
					FUIAction(
						FExecuteAction::CreateSP(this, &STedsFilterBar::FrontendFilterCategoryClicked, Category),
						FCanExecuteAction(),
						FGetActionCheckState::CreateSP(this, &STedsFilterBar::IsFrontendFilterCategoryChecked, Category)
					),
					EUserInterfaceActionType::ToggleButton
				);
			}
		}
	}

	TSharedRef<SWidget> STedsFilterBar::MakeAddFilterMenu()
	{
		static const FName FilterMenuName = "TedsFilterMenu.FilterMenu";
		if (!UToolMenus::Get()->IsMenuRegistered(FilterMenuName))
		{
			UToolMenu* Menu = UToolMenus::Get()->RegisterMenu(FilterMenuName);
			Menu->bShouldCloseWindowAfterMenuSelection = true;
			Menu->bCloseSelfOnly = true;
            
			Menu->AddDynamicSection(NAME_None, FNewToolMenuDelegate::CreateLambda([](UToolMenu* InMenu)
			{
				if (UFilterBarContext* Context = InMenu->FindContext<UFilterBarContext>())
				{
					Context->PopulateFilterMenu.ExecuteIfBound(InMenu);
				}
			}));
		}

		UFilterBarContext* FilterBarContext = NewObject<UFilterBarContext>();
		FilterBarContext->PopulateFilterMenu = FOnPopulateAddFilterMenu::CreateSP(this, &STedsFilterBar::PopulateAddFilterMenu);
		FToolMenuContext ToolMenuContext(FilterBarContext);
		return UToolMenus::Get()->GenerateWidget(FilterMenuName, ToolMenuContext);
	}

	void STedsFilterBar::RecompileQueries()
	{
		using namespace DataStorage::Queries;
		using namespace DataStorage::QueryStack;

		if (!ExternalQueries.IsEmpty())
		{
			TArray<TSharedPtr<IQueryNode>> QueryNodes;
			for (const TPair<FName, TSharedPtr<IQueryNode>>& ExternalQuery : ExternalQueries)
			{
				QueryNodes.Add(ExternalQuery.Value);
			}
			const TSharedPtr<IQueryNode> CombinedQueryNode = MakeShared<FQueryMergeNode>(*Storage, QueryNodes);
			CombinedRowCollectorNode = MakeShared<FRowQueryResultsNode>(*Storage, CombinedQueryNode, FRowQueryResultsNode::ESyncActions::ForceRefreshOnUpdate);
			const TSharedPtr<QueryStack::IRowNode> TemporaryFilterNodesArray[2] = {CombinedRowCollectorNode, InFilterableRowNode};
			
			CombinedQueryFilterRowNode = MakeShared<FRowMergeNode>(TemporaryFilterNodesArray, FRowMergeNode::EMergeApproach::Repeating);
		}
		else
		{
			CombinedQueryFilterRowNode = InFilterableRowNode;
		}

		FilterNodes.Empty();
		CompositeFilterNode = CombinedQueryFilterRowNode;
		// Combines all active external filters using AND
		for (const TPair<FName, TConstQueryFunction<bool>>& ExternalQueryFunction : ExternalQueryFunctions)
		{
			const TSharedPtr<FRowFilterNode> FilterNode = MakeShared<FRowFilterNode>(Storage, CompositeFilterNode, ExternalQueryFunction.Value);
			FilterNodes.Add(FilterNode);
			CompositeFilterNode = FilterNode;
		}

		ClassFilterNodes.Empty();
		// Combines all active class filters using OR
		if (!ClassFilters.IsEmpty())
		{
			for (const TPair<FName, TConstQueryFunction<bool>>& ClassFilter : ClassFilters)
			{
				ClassFilterNodes.Add(MakeShared<FRowFilterNode>(Storage, InFilterableRowNode, ClassFilter.Value));
			}

			CombinedClassFilterRowNode = MakeShared<FRowMergeNode>(ClassFilterNodes, FRowMergeNode::EMergeApproach::Unique);
			const TSharedPtr<QueryStack::IRowNode> TemporaryCombinedClassFilterNodes[2] = {CombinedClassFilterRowNode, InFilterableRowNode};
			
			*OutFilteredNode = MakeShared<FRowMergeNode>(TemporaryCombinedClassFilterNodes, FRowMergeNode::EMergeApproach::Repeating);
		}
		else
		{
			*OutFilteredNode = MoveTemp(CompositeFilterNode);
		}
		
		OnPostFiltersChanged.ExecuteIfBound();
		bMarkDirty = false;
	}

	void STedsFilterBar::AddExternalQuery(const FName& QueryName, const QueryHandle& InQueryHandle)
	{
		if (ensureMsgf(InQueryHandle != InvalidQueryHandle, TEXT("An Invalid Query Handle cannot be used for a TEDS Filter")))
		{
			ExternalQueries.Emplace(QueryName, MakeShared<DataStorage::QueryStack::FQueryHandleNode>(InQueryHandle));
		}
		bMarkDirty = true;
	}

	void STedsFilterBar::RemoveExternalQuery(const FName& QueryName)
	{
		ExternalQueries.Remove(QueryName);
		bMarkDirty = true;
	}

	void STedsFilterBar::AppendExternalQueries(FQueryDescription& OutQuery)
	{
		for(const TPair<FName, TSharedPtr<QueryStack::IQueryNode>>& ExternalQuery : ExternalQueries)
		{
			DataStorage::Queries::MergeQueries(OutQuery, Storage->GetQueryDescription(ExternalQuery.Value->GetQuery()));
		}
		bMarkDirty = true;
	}

	void STedsFilterBar::AddExternalQueryFunction(const FName& QueryName, const DataStorage::Queries::TConstQueryFunction<bool>& InQueryFunction)
	{
		// Store the external query functions as functions instead of RowNodes since the row node to filter on has not been initialized yet
		ExternalQueryFunctions.Emplace(QueryName, InQueryFunction);
		bMarkDirty = true;
	}

	void STedsFilterBar::RemoveExternalQueryFunction(const FName& QueryName)
	{
		ExternalQueryFunctions.Remove(QueryName);
		bMarkDirty = true;
	}

	void STedsFilterBar::AddClassQueryFunction(const FName& ClassName, const Queries::TConstQueryFunction<bool>& InClassQueryFunction)
	{
		ClassFilters.Emplace(ClassName, InClassQueryFunction);
		bMarkDirty = true;
	}

	void STedsFilterBar::RemoveClassQueryFunction(const FName& ClassName)
	{
		ClassFilters.Remove(ClassName);
		bMarkDirty = true;
	}
	
} // namespace UE::Editor::DataStorage

#undef LOCTEXT_NAMESPACE

