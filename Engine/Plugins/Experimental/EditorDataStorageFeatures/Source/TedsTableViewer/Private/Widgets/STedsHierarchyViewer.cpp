// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/STedsHierarchyViewer.h"

#include "Columns/SlateDelegateColumns.h"
#include "DataStorage/Features.h"
#include "DragAndDrop/Widgets/WidgetDropHandler.h"
#include "Elements/Columns/TypedElementHiearchyColumns.h"
#include "Elements/Columns/TypedElementUIColumns.h"
#include "Elements/Framework/TypedElementDataStorageWidget.h"
#include "QueryStack/TopLevelRowsNode.h"
#include "TedsTableViewerColumn.h"
#include "TedsTableViewerUtils.h"
#include "TedsTableViewerWidgetColumns.h"
#include "Widgets/STedsTableViewerRow.h"
#include "Widgets/STedsTreeView.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/SListView.h"

#define LOCTEXT_NAMESPACE "SHierarchyViewer"

namespace UE::Editor::DataStorage
{
	SHierarchyViewer::SHierarchyViewer()
	{
	}

	SHierarchyViewer::~SHierarchyViewer()
	{
		if(TedsWidget.IsValid() && Model.IsValid())
		{
			if (bPersistUIRowsOnClose)
			{
				TableViewerUtils::DeactivateAllTableViewerUIRows(Model->GetDataStorageInterface(), TedsWidget->GetRowHandle());
			}
			else
			{
				TableViewerUtils::RemoveAllTableViewerUIRows(Model->GetDataStorageInterface(), TedsWidget->GetRowHandle());
			}
		}
	}
	
	void SHierarchyViewer::Construct(
		const FArguments& InArgs,
		TSharedPtr<IHierarchyViewerDataInterface> InHierarchyInterface)
	{
		HierarchyInterface = MoveTemp(InHierarchyInterface);
		OnSelectionChanged = InArgs._OnSelectionChanged;
		bPersistUIRowsOnClose = InArgs._PersistUIRowsOnClose;
		GetCustomTableViewerRowMapping = InArgs._GetCustomTableViewerRowMapping;
		bExpandNewRows = InArgs._ExpandNewRows;
		EmptyRowsMessage = InArgs._EmptyRowsMessage;
		bShowWires = InArgs._ShowWires;
		
		IUiProvider::FPurposeID CellWidgetPurpose = InArgs._CellWidgetPurpose;

		IUiProvider* StorageUi = GetMutableDataStorageFeature<IUiProvider>(UiFeatureName);

		if (!ensureMsgf(StorageUi, TEXT("Cannot use SHierarchyViewer without TEDS UI being initialized")))
		{
			return;
		}

		if (!CellWidgetPurpose.IsSet())
		{
			CellWidgetPurpose = StorageUi->GetGeneralWidgetPurposeID();
		}

		IUiProvider::FPurposeID HeaderWidgetPurpose = InArgs._HeaderWidgetPurpose;

		if (!HeaderWidgetPurpose.IsSet())
		{
			HeaderWidgetPurpose = IUiProvider::FPurposeInfo("General", "Header", NAME_None).GeneratePurposeID();
		}

		Model = MakeShared<FTedsTableViewerModel>(FTedsTableViewerModel::FArguments{
			.RowQueryStack = InArgs._AllNodeProvider,
			.RequestedColumns = InArgs._Columns,
			.GenericMetaData = InArgs._GenericMetaData,
			.ColumnMetaData = InArgs._ColumnMetaData,
			.CellWidgetPurpose = CellWidgetPurpose,
			.HeaderWidgetPurpose = HeaderWidgetPurpose,
			.IsItemVisibleDelegate = FTedsTableViewerModel::FIsItemVisible::CreateSP(this, &SHierarchyViewer::IsItemVisible),
			.PrimaryColumn = InArgs._PrimaryColumn,
			.bShowHiddenRows = InArgs._ShowHiddenRows
		});

		TopLevelRowsNode = MakeShared<QueryStack::FTopLevelRowsNode>(Model->GetDataStorageInterface(), HierarchyInterface, Model->GetRowNode());

		ExecutorInstanceName = FName("TableViewerModel", InstanceCounter++);
		QueryStackExecutor = QueryStack::FExplicitUpdateExecutor(ExecutorInstanceName, TopLevelRowsNode);

		SetTableViewerIdentifier(InArgs._TableViewerIdentifier);
		
		HeaderRowWidget = SNew( SHeaderRow )
							.CanSelectGeneratedColumn(InArgs._CanSelectGeneratedColumn);
		
		TedsWidget = StorageUi->CreateContainerTedsWidget(InvalidRowHandle);
			
		ChildSlot
		[
			TedsWidget->AsWidget()
		];
		
		AddWidgetColumns();

		TreeView = SNew(STedsTreeView, STedsTreeView::FOnGetParent::CreateSP(this, &SHierarchyViewer::GetParentRow), GetWidgetRowHandle())
			.HeaderRow(HeaderRowWidget)
			.TopLevelRowsSource(&TopLevelRows)
			.RowsSource(&Model->GetItems())
			.OnGenerateRow(this, &SHierarchyViewer::MakeTableRowWidget)
			.OnExpansionChanged(this, &SHierarchyViewer::OnExpansionChanged)
			.OnSetExpansionRecursive(this, &SHierarchyViewer::OnSetExpansionRecursive)
			.OnSelectionChanged(this, &SHierarchyViewer::OnListSelectionChanged)
			.SelectionMode(InArgs._ListSelectionMode)
			.AllowInvisibleItemSelection(InArgs._AllowInvisibleItemSelection);

		CreateInternalWidget();
		
		// Add each Teds column from the model to our header row widget
		Model->ForEachColumn([this](const TSharedRef<FTedsTableViewerColumn>& Column)
		{
			HeaderRowWidget->AddColumn(Column->ConstructHeaderRowColumn());
		});

		// Whenever the model changes, refresh the list to update the UI
		Model->GetOnModelChanged().AddRaw(this, &SHierarchyViewer::OnModelChanged);
	}

	void SHierarchyViewer::OnModelChanged()
	{
		TopLevelRows.Empty();

		// Includes a generic way of detecting hierarchy changes. If the model isn't refreshed on hierarchy change the top level rows will still use the old
		// hierarchy, so the explicit update is needed.
		QueryStackExecutor.Update();
		
		FRowHandleArrayView Rows(TopLevelRowsNode->GetRows());

		for(const RowHandle RowHandle : Rows)
		{
			FTedsRowHandle TedsRowHandle{ .RowHandle = RowHandle };
			TopLevelRows.Add(TedsRowHandle);
		}

		if(ICoreProvider* DataStorage = Model->GetDataStorageInterface())
		{
			if (!Rows.IsEmpty())
			{
				DataStorage->AddColumn(GetWidgetRowHandle(), FHierarchyViewerRootColumn{.Root = Rows[0]});
			}
			else
			{
				DataStorage->RemoveColumn<FHierarchyViewerRootColumn>(GetWidgetRowHandle());
			}
		}

		// Check if the model has been initialized yet, if so, run any Post initialization/deferred actions
		if (!bModelInitialized)
		{
			if (const TSharedPtr<QueryStack::IRowNode> RowNode = Model->GetRowNode())
			{
				if (RowNode->GetRevision() != QueryStack::INode::UninitializedRevisionId)
				{
					InitializeAllTableViewerUIRows();
					bModelInitialized = true;
				}

				if (bExpansionStateChangeRequested.IsSet())
				{
					FRowHandleArrayView AllRows = RowNode->GetRows();
					if (AllRows.Num())
					{
						SetExpansionState_Internal(AllRows, bExpansionStateChangeRequested.GetValue());
						bExpansionStateChangeRequested.Reset();
					}
				}
			}
		}
		
		TreeView->RequestListRefresh();
		CreateInternalWidget();
	}

	void SHierarchyViewer::SetExpansionState_Internal(FRowHandleArrayView Rows, bool bExpanded) const
	{
		for (RowHandle Row : Rows)
		{
			FTedsRowHandle TedsRowHandle{ .RowHandle = Row };
			TreeView->SetItemExpansion(TedsRowHandle, bExpanded);
		}
	}

	void SHierarchyViewer::OnDragEnter(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
	{
		if (DropHandler)
		{
			DropHandler->OnDragEnter(MyGeometry, DragDropEvent);
		}
	}

	FReply SHierarchyViewer::OnDragOver(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
	{
		return DropHandler ? DropHandler->OnDragOver(MyGeometry, DragDropEvent) : FReply::Unhandled();
	}

	void SHierarchyViewer::OnDragLeave(const FDragDropEvent& DragDropEvent)
	{
		if (DropHandler)
		{
			DropHandler->OnDragLeave(DragDropEvent);
		}
		
		if (ICoreProvider* DataStorage = Model->GetDataStorageInterface())
		{
			DataStorage->RemoveColumn<FTableViewerDropInfoColumn>(GetWidgetRowHandle());
		}
	}

	FReply SHierarchyViewer::OnDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
	{
		FReply Reply = DropHandler ? DropHandler->OnDrop(MyGeometry, DragDropEvent) : FReply::Unhandled();
		if (ICoreProvider* DataStorage = Model->GetDataStorageInterface())
		{
			DataStorage->RemoveColumn<FTableViewerDropInfoColumn>(GetWidgetRowHandle());
		}
		return Reply;
	}

	void SHierarchyViewer::AddWidgetColumns()
	{
		if(ICoreProvider* DataStorage = Model->GetDataStorageInterface())
		{
			const RowHandle WidgetRowHandle = GetWidgetRowHandle();
		
			if(DataStorage->IsRowAvailable(WidgetRowHandle))
			{
				// FHideRowFromUITag - The table viewer should not show up as a row in a table viewer because that will cause all sorts of recursion issues
				// The others are Columns we are going to bind to attributes on STreeView
				DataStorage->AddColumns<FTableViewerTag, FHideRowFromUITag, FWidgetContextMenuColumn, FWidgetRowScrolledIntoView,
					FWidgetDoubleClickedColumn, FTableViewerSelectionChangedColumn, FTableViewerRenameSelectionColumn>(WidgetRowHandle);
				DataStorage->AddColumn(WidgetRowHandle, FTableViewerReferenceColumn{.TableViewerWeak = SharedThis(this)});
				DataStorage->AddColumn(WidgetRowHandle, FHierarchyViewerDataColumn{.HierarchyData = HierarchyInterface});
			}
		}
	}

	TableViewerItemPtr SHierarchyViewer::GetParentRow(TableViewerItemPtr InItem) const
	{
		if (ICoreProvider* Storage = Model->GetDataStorageInterface())
		{
			// To work around unreliable query stack update timings, in case the widget queries parents before the top level rows node has updated
			// but after the hierarchies themselves have updated since HierarchyInterface directly queries TEDS. 
			if (TopLevelRows.Contains(InItem) || !HierarchyInterface)
			{
				return FTedsRowHandle(InvalidRowHandle);
			}
		
			return FTedsRowHandle(HierarchyInterface->GetParent(*Storage, InItem));
		}

		return FTedsRowHandle(InvalidRowHandle);
	}

	void SHierarchyViewer::CreateInternalWidget()
	{
		TSharedPtr<SWidget> ContentWidget;

		// If there are no rows and the table viewer wants to show a message
		if(Model->GetRowCount() == 0 && EmptyRowsMessage.IsSet())
		{
			ContentWidget = 
				SNew(SBox)
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Center)
					[
						SNew(STextBlock)
							.Text(EmptyRowsMessage)
					];
		}
		else if(Model->GetColumnCount() == 0)
		{
			ContentWidget =
				SNew(SBox)
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Center)
					[
						SNew(STextBlock)
							.Text(LOCTEXT("EmptyTableViewerColumnsText", "No columns found to display."))
					];
		}
		else
		{
			ContentWidget = TreeView.ToSharedRef();
		}

		TedsWidget->SetContent(ContentWidget.ToSharedRef());
	}

	void SHierarchyViewer::RefreshColumnWidgets()
	{
		HeaderRowWidget->ClearColumns();
		Model->ForEachColumn([this](const TSharedRef<FTedsTableViewerColumn>& Column)
		{
			HeaderRowWidget->AddColumn(Column->ConstructHeaderRowColumn());
		});

		CreateInternalWidget();
	}

	void SHierarchyViewer::OnExpansionChanged(TableViewerItemPtr Item, bool bExpanded) const
	{
		if (ICoreProvider* Storage = Model->GetDataStorageInterface())
		{
			FMapKey MappingKey;
			GetTableViewerRowMapping(TableViewerIdentifier, Item.RowHandle, MappingKey);
			if (const RowHandle UIRow = Storage->LookupMappedRow(TableViewerUIRowMappingDomain, MappingKey); Storage->IsRowAvailable(UIRow))
			{
				if (bExpanded)
				{
					Storage->AddColumns<FExpandedInUITag>(UIRow);
				}
				else
				{
					Storage->RemoveColumns<FExpandedInUITag>(UIRow);
				}
			}
		}
	}

	void SHierarchyViewer::OnSetExpansionRecursive(TableViewerItemPtr Item, bool bExpanded) const
	{
		if (const ICoreProvider* Storage = Model->GetDataStorageInterface(); Storage != nullptr && HierarchyInterface.IsValid())
		{
			TArray<RowHandle> ChildRowsToSet;
			HierarchyInterface->WalkDepthFirst(*Storage, Item.RowHandle,
			[&ChildRowsToSet](const ICoreProvider&, RowHandle, const RowHandle Target)
			{
				ChildRowsToSet.Add(Target);
			});
			
			SetExpansionState_Internal(FRowHandleArrayView(ChildRowsToSet, FRowHandleArrayView::EFlags::IsUnique), bExpanded);
		}
	}

	void SHierarchyViewer::OnListSelectionChanged(TableViewerItemPtr Item, ESelectInfo::Type SelectInfo)
	{
		if(OnSelectionChanged.IsBound())
		{
			OnSelectionChanged.Execute(Item, SelectInfo);
		}

		// Some systems can only access the table viewer by the row handle so also fire the selection changed event on the column
		if (ICoreProvider* Storage = Model->GetDataStorageInterface())
		{
			if (FTableViewerSelectionChangedColumn* Column = Storage->GetColumn<FTableViewerSelectionChangedColumn>(GetWidgetRowHandle()))
			{
				if (Column->OnSelectionChanged)
				{
					Column->OnSelectionChanged->Broadcast();
				}
			}
		}
	}

	void SHierarchyViewer::SetQueryStack(TSharedPtr<QueryStack::IRowNode> InRowQueryStack)
	{
		Model->SetQueryStack(InRowQueryStack);
		TopLevelRowsNode = MakeShared<QueryStack::FTopLevelRowsNode>(Model->GetDataStorageInterface(), HierarchyInterface, Model->GetRowNode());
		QueryStackExecutor = QueryStack::FExplicitUpdateExecutor(ExecutorInstanceName, TopLevelRowsNode);
		QueryStackExecutor.Update();
		OnModelChanged();
	}

	void SHierarchyViewer::SetColumns(const TArray<TWeakObjectPtr<const UScriptStruct>>& InColumns)
	{
		Model->SetColumns(InColumns);
		RefreshColumnWidgets();
	}

	void SHierarchyViewer::SetTableViewerIdentifier(const FName& NewIdentifier)
	{
		// Only use this function when it is necessary to get a name after initialization, the rows associated with the
		// previous name will be removed regardless of the bRemoveTableViewerRowsOnClose argument and re-initialized.
		// If the model hasn't been initialized yet, there are no rows to remove so we can skip this
		if (bModelInitialized && TedsWidget.IsValid() && Model.IsValid())
		{
			// Remove while set to the old identifier rather than deactivating rows
			TableViewerUtils::RemoveAllTableViewerUIRows(Model->GetDataStorageInterface(), GetWidgetRowHandle());
		}
 
		TableViewerIdentifier = NewIdentifier.IsNone() ?
			FName("TEDSHierarchyViewerID", InstanceCounter) :
			NewIdentifier;
		
		// If the model hasn't been initialized yet, we can skip initializing the new rows as it will be done later
		if (bModelInitialized)
		{
			// We want to reinitialize the current rows as table viewer row UI rows under the new mapping
			InitializeAllTableViewerUIRows();
		}
	}

	void SHierarchyViewer::AddCustomRowWidget(const TSharedRef<FTedsTableViewerColumn>& InColumn)
	{
		Model->AddCustomRowWidget(InColumn);
		RefreshColumnWidgets();
	}

	void SHierarchyViewer::ForEachSelectedRow(TFunctionRef<void(RowHandle)> InCallback) const
	{
		TArray<TableViewerItemPtr> SelectedRows;
		TreeView->GetSelectedItems(SelectedRows);

		for(TableViewerItemPtr& Row : SelectedRows)
		{
			InCallback(Row);
		}
	}

	RowHandle SHierarchyViewer::GetWidgetRowHandle() const
	{
		return TedsWidget->GetRowHandle();
	}

	void SHierarchyViewer::SetSelection(RowHandle Row, bool bSelected, const ESelectInfo::Type SelectInfo) const
	{
		FTedsRowHandle TedsRowHandle{ .RowHandle = Row };
		TreeView->SetItemSelection(TedsRowHandle, bSelected, SelectInfo);
	}

	void SHierarchyViewer::ScrollIntoView(RowHandle Row) const
	{
		FTedsRowHandle TedsRowHandle{ .RowHandle = Row };
		TreeView->RequestScrollIntoView(TedsRowHandle);
	}

	void SHierarchyViewer::ClearSelection() const
	{
		TreeView->ClearSelection();
	}

	TSharedRef<SWidget> SHierarchyViewer::AsWidget()
	{
		return AsShared();
	}

	bool SHierarchyViewer::IsSelected(RowHandle InRow) const
	{
		FTedsRowHandle TedsRowHandle{ .RowHandle = InRow };
		return TreeView->IsItemSelected(TedsRowHandle);
	}

	bool SHierarchyViewer::IsSelectedExclusively(RowHandle InRow) const
	{
		return IsSelected(InRow) && TreeView->GetNumItemsSelected() == 1;
	}

	void SHierarchyViewer::ExpandAll() const
	{
		// If the query stack hasn't been initialized yet, defer the expansion until next refresh
		if (!bModelInitialized)
		{
			bExpansionStateChangeRequested = true;
		}
		else if (TSharedPtr<QueryStack::IRowNode> RowNode = Model->GetRowNode())
		{
			SetExpansionState_Internal(RowNode->GetRows(), true);
		}
	}

	void SHierarchyViewer::CollapseAll() const
	{
		// If the query stack hasn't been initialized yet, defer the expansion until next refresh
		if (!bModelInitialized)
		{
			bExpansionStateChangeRequested = false;
		}
		else if (TSharedPtr<QueryStack::IRowNode> RowNode = Model->GetRowNode())
		{
			SetExpansionState_Internal(RowNode->GetRows(), false);
		}
	}

	void SHierarchyViewer::ExpandWithParents(RowHandle Row)
	{
		FTedsRowHandle TedsRowHandle{ .RowHandle = Row };
		int32 SanityCheckCount = 0;
		constexpr int32 SANITY_CHECK_LIMIT = 1000;

		while (TedsRowHandle.IsValid() && ensure(SanityCheckCount++ < SANITY_CHECK_LIMIT))
		{
			TreeView->SetItemExpansion(TedsRowHandle, true);
			TedsRowHandle = GetParentRow(TedsRowHandle);
		}
	}

	void SHierarchyViewer::SetDropHandler(TUniquePtr<FWidgetDropHandler> InDropHandler)
	{
		DropHandler = MoveTemp(InDropHandler);
	}

	bool SHierarchyViewer::IsItemVisible(TableViewerItemPtr InItem) const
	{
		return TreeView->IsItemVisible(InItem);
	}
	
	void SHierarchyViewer::GetTableViewerRowMapping(const FName& InTableViewerIdentifier, const RowHandle InRow, FMapKey& OutMapKey) const
	{
		if(GetCustomTableViewerRowMapping.IsBound())
		{
			GetCustomTableViewerRowMapping.Execute(InTableViewerIdentifier, InRow, OutMapKey);
			if (OutMapKey.IsSet())
			{
				return;
			}
		}
		TableViewerUtils::CreateRowMappingKey(InTableViewerIdentifier, InRow, OutMapKey);
	}
	
	RowHandle SHierarchyViewer::AddOrFindTableViewerUIRow(const TableViewerItemPtr InItem) const
	{
		if (ICoreProvider* Storage = Model->GetDataStorageInterface())
		{
			FMapKey MappingKey;
			GetTableViewerRowMapping(TableViewerIdentifier, InItem.RowHandle, MappingKey);
			// If the row already existed but was removed from TreeView's SparseItemInfos (Searching, Filters, etc.), re-expand it
			if (const RowHandle ExistingRow = Storage->LookupMappedRow(TableViewerUIRowMappingDomain, MappingKey); Storage->IsRowAvailable(ExistingRow))
			{
				if(Storage->HasColumns<FExpandedInUITag>(ExistingRow) && !TreeView->IsItemExpanded(InItem))
				{
					TreeView->SetItemExpansion(InItem, true);
				}
				Storage->RemoveColumns<FTedsTableViewerInactiveRowTag>(ExistingRow);
				return ExistingRow;
			}
			else
			{
				const RowHandle AddedRow = Storage->AddRow(Storage->FindTable(TableViewerUtils::GetRowWidgetTableName()));
				
				Storage->SetParentRow(Storage->FindHierarchyByName(TableViewerUtils::GetWidgetHierarchyName()), AddedRow, GetWidgetRowHandle());
				Storage->MapRow(TableViewerUIRowMappingDomain, MappingKey, AddedRow);

				const bool bIsItemExpandedInTree = TreeView->IsItemExpanded(InItem);
				if (bIsItemExpandedInTree != bExpandNewRows)
				{
					// Default to bExpandNewRows if not already
					TreeView->SetItemExpansion(InItem, bExpandNewRows);
				}
				else if (bIsItemExpandedInTree && bExpandNewRows)
				{
					Storage->AddColumns<FExpandedInUITag>(AddedRow);
				}

				return AddedRow;
			}
		}

		return InvalidRowHandle;
	}

	void SHierarchyViewer::InitializeAllTableViewerUIRows() const
	{
		TableViewerUtils::InitializeAllTableViewerUIRows(
			Model->GetDataStorageInterface(),
			Model->GetItems(), 
			TableViewerUtils::FGetTableViewerRowMapping::CreateSP(this, &SHierarchyViewer::GetTableViewerRowMapping), 
			TableViewerIdentifier, 
			GetWidgetRowHandle(),
			true);
	}

	TSharedRef<ITableRow> SHierarchyViewer::MakeTableRowWidget(TableViewerItemPtr InItem, const TSharedRef<STableViewBase>& OwnerTable) const
	{
		const RowHandle TableViewerRowUIRow = AddOrFindTableViewerUIRow(InItem);
		
		return SNew(SHierarchyViewerRow, OwnerTable, Model.ToSharedRef(), SharedThis(this))
				.Padding(FMargin(0))
				.WidgetRowHandle(TableViewerRowUIRow == InvalidRowHandle ? GetWidgetRowHandle() : TableViewerRowUIRow)
				.TableViewerWidgetRowHandle(GetWidgetRowHandle())
				.ShowWires(bShowWires)
				.Item(InItem);
	}
} // namespace UE::Editor::DataStorage

#undef LOCTEXT_NAMESPACE //"SHierarchyViewer"

