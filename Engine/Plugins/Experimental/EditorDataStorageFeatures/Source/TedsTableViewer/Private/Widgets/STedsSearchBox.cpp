// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/STedsSearchBox.h"

#include "DataStorage/Features.h"
#include "Elements/Framework/TypedElementQueryBuilder.h"
#include "Elements/Framework/TypedElementQueryContext.h"

#include "Elements/Columns/TypedElementLabelColumns.h"
#include "Elements/Columns/TypedElementTypeInfoColumns.h"
#include "TedsQueryNode.h"
#include "TedsRowFilterNode.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "STedsSearchBox"

namespace UE::Editor::DataStorage
{
	void STedsSearchBox::Construct(const FArguments& InArgs)
	{
		using namespace UE::Editor::DataStorage::Queries;

		ICoreProvider* Storage = GetMutableDataStorageFeature<ICoreProvider>(UE::Editor::DataStorage::StorageFeatureName);
		if(!ensureMsgf(Storage, TEXT("Cannot create a TEDS search widget before TEDS is initialized")))
		{
			ChildSlot
			[
				SNew(STextBlock)
					.Text(LOCTEXT("TedsSearchErrorText", "No valid editor data storage available."))
			];
			return;
		}

		if(InArgs._OutSearchNode)
		{
			if (InArgs._InSearchableQueryNode)
			{
				TSharedRef<QueryStack::FQuerySearchNode> NewSearchNode = MakeShared<QueryStack::FQuerySearchNode>(
					*Storage, 
					InArgs._InSearchableQueryNode.ToSharedRef(),
					InArgs._InSearchableQueryFlags);
				SearchNode.Emplace<TSharedRef<QueryStack::FQuerySearchNode>>(NewSearchNode);
				*InArgs._OutSearchNode = MoveTemp(NewSearchNode);
			}
			else if (InArgs._InSearchableRowNode)
			{
				TSharedPtr<QueryStack::FRowFilterNode> NewSearchNode = MakeShared<QueryStack::FRowFilterNode>(Storage, InArgs._InSearchableRowNode,
					[Widget = AsWeak()](TConstQueryContext<SingleRowInfo> Context, const FTypedElementLabelColumn& LabelColumn)
					{
						if (TSharedPtr<SWidget> WidgetPtr = Widget.Pin())
						{
							STedsSearchBox& SearchBox = *static_cast<STedsSearchBox*>(WidgetPtr.Get());
							if (!SearchBox.SearchText.IsEmpty())
							{
								TArray<FString> SearchTokens;
								SearchBox.SearchText.ToString().ParseIntoArrayWS(SearchTokens);
								const FString Label = LabelColumn.Label;

								for (const FString& SearchToken : SearchTokens)
								{
									if (Label.IsEmpty() || !Label.Contains(SearchToken, ESearchCase::Type::IgnoreCase))
									{
										return false;
									}
								}
							}
						}
						return true;
					});

				SearchNode.Emplace<TSharedPtr<QueryStack::FRowFilterNode>>(NewSearchNode);
				*InArgs._OutSearchNode = MoveTemp(NewSearchNode);
			}
		}
		
		ChildSlot
		[
			SNew(SSearchBox)
				.OnTextChanged(this, &STedsSearchBox::OnTextChanged)
				.DelayChangeNotificationsWhileTyping(false)
		];
	}

	void STedsSearchBox::OnTextChanged(const FText& InSearchText)
	{
		SearchText = InSearchText;

		struct FVisitor
		{
			void operator()(const TSharedPtr<QueryStack::FRowFilterNode>& Node)
			{
				if (Node)
				{
					Node->ForceRefresh();
				}
			}
			
			void operator()(const TSharedPtr<QueryStack::FQuerySearchNode>& Node)
			{
				if (Node)
				{
					if (This->SearchText.IsEmpty())
					{
						Node->ClearSearch();
					}
					else
					{
						Node->StartSearch(This->SearchText.ToString());
					}
				}
			}

			STedsSearchBox* This;
		};

		Visit(FVisitor{ .This = this }, SearchNode);
	}
	
} // namespace UE::Editor::DataStorage

#undef LOCTEXT_NAMESPACE //"STedsSearchBox"