// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/STedsTableViewer.h"

#include "Columns/SlateDelegateColumns.h"
#include "DataStorage/Features.h"
#include "DragAndDrop/Widgets/WidgetDropHandler.h"
#include "Elements/Columns/TypedElementUIColumns.h"
#include "Elements/Framework/TypedElementAttributeBinding.h"
#include "Elements/Framework/TypedElementDataStorageWidget.h"
#include "TedsTableViewerColumn.h"
#include "TedsTableViewerUtils.h"
#include "TedsTableViewerWidgetColumns.h"
#include "Widgets/STedsTableViewerRow.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/SListView.h"

#define LOCTEXT_NAMESPACE "STedsTableViewer"

namespace UE::Editor::DataStorage
{
	STedsTableViewer::STedsTableViewer()
	{
	}

	STedsTableViewer::~STedsTableViewer()
	{
		if(TedsWidget.IsValid() && Model.IsValid())
		{
			if (bPersistUIRowsOnClose)
			{
				TableViewerUtils::DeactivateAllTableViewerUIRows(Model->GetDataStorageInterface(), TedsWidget->GetRowHandle());
			}
			else
			{
				TableViewerUtils::RemoveAllTableViewerUIRows(Model->GetDataStorageInterface(), TedsWidget->GetRowHandle());
			}
		}
	}
	
	void STedsTableViewer::Construct(const FArguments& InArgs)
	{
		OnSelectionChanged = InArgs._OnSelectionChanged;
		GetCustomTableViewerRowMapping = InArgs._GetCustomTableViewerRowMapping;
		bPersistUIRowsOnClose = InArgs._PersistUIRowsOnClose;
		EmptyRowsMessage = InArgs._EmptyRowsMessage;
		ItemHeight = InArgs._ItemHeight;
		ItemPadding = InArgs._ItemPadding;

		IUiProvider::FPurposeID CellWidgetPurpose = InArgs._CellWidgetPurpose;

		IUiProvider* StorageUi = GetMutableDataStorageFeature<IUiProvider>(UiFeatureName);

		if (!ensureMsgf(StorageUi, TEXT("Cannot use STedsTableViewer without TEDS UI being initialized")))
		{
			return;
		}

		if (!CellWidgetPurpose.IsSet())
		{
			CellWidgetPurpose = StorageUi->GetGeneralWidgetPurposeID();
		}

		IUiProvider::FPurposeID HeaderWidgetPurpose = InArgs._HeaderWidgetPurpose;

		if (!HeaderWidgetPurpose.IsSet())
		{
			HeaderWidgetPurpose = IUiProvider::FPurposeInfo("General", "Header", NAME_None).GeneratePurposeID();
		}
		
		Model = MakeShared<FTedsTableViewerModel>(FTedsTableViewerModel::FArguments{
			.RowQueryStack = InArgs._QueryStack,
			.RequestedColumns = InArgs._Columns,
			.GenericMetaData = InArgs._GenericMetaData,
			.ColumnMetaData = InArgs._ColumnMetaData,
			.CellWidgetPurpose = CellWidgetPurpose,
			.HeaderWidgetPurpose = HeaderWidgetPurpose,
			.IsItemVisibleDelegate = FTedsTableViewerModel::FIsItemVisible::CreateSP(this, &STedsTableViewer::IsItemVisible),
			.PrimaryColumn = InArgs._PrimaryColumn,
			.bShowHiddenRows = InArgs._ShowHiddenRows
		});

		InstanceCounter++;
		SetTableViewerIdentifier(InArgs._TableViewerIdentifier);
		
		HeaderRowWidget = SNew( SHeaderRow )
							.CanSelectGeneratedColumn(InArgs._CanSelectGeneratedColumn);

		TedsWidget = StorageUi->CreateContainerTedsWidget(InvalidRowHandle);
			
		ChildSlot
		[
			TedsWidget->AsWidget()
		];
		
		AddWidgetColumns();

		// Attribute binder to bind widget columns to attributes on the ListView
		FAttributeBinder Binder(TedsWidget->GetRowHandle());

		ListView = SNew(SListView<TableViewerItemPtr>)
			.HeaderRow(HeaderRowWidget)
			.ListItemsSource(&Model->GetItems())
			.OnGenerateRow(this, &STedsTableViewer::MakeTableRowWidget)
			.OnSelectionChanged(this, &STedsTableViewer::OnListSelectionChanged)
			.SelectionMode(InArgs._ListSelectionMode)
			.OnContextMenuOpening(Binder.BindEvent(&FWidgetContextMenuColumn::OnContextMenuOpening))
			.OnItemScrolledIntoView(Binder.BindEvent(&FWidgetRowScrolledIntoView::OnItemScrolledIntoView))
			.OnMouseButtonDoubleClick(Binder.BindEvent(&FWidgetDoubleClickedColumn::OnMouseButtonDoubleClick));

		CreateInternalWidget();
		
		// Add each Teds column from the model to our header row widget
		Model->ForEachColumn([this](const TSharedRef<FTedsTableViewerColumn>& Column)
		{
			HeaderRowWidget->AddColumn(Column->ConstructHeaderRowColumn());
		});

		// Whenever the model changes, refresh the list to update the UI
		Model->GetOnModelChanged().AddRaw(this, &STedsTableViewer::OnModelChanged);
	}

	void STedsTableViewer::OnModelChanged()
	{
		// Check if the model hasn't been initialized yet, if so, run any Post initialization/deferred actions
		if (!bModelInitialized)
		{
			if (const TSharedPtr<QueryStack::IRowNode> RowNode = Model->GetRowNode())
			{
				if (RowNode->GetRevision() != QueryStack::INode::UninitializedRevisionId)
				{
					InitializeAllTableViewerUIRows();
					bModelInitialized = true;
				}
			}
		}
		
		ListView->RequestListRefresh();
		CreateInternalWidget();
	}

	void STedsTableViewer::OnDragEnter(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
	{
		if (DropHandler)
		{
			DropHandler->OnDragEnter(MyGeometry, DragDropEvent);
		}
	}

	FReply STedsTableViewer::OnDragOver(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
	{
		return DropHandler ? DropHandler->OnDragOver(MyGeometry, DragDropEvent) : FReply::Unhandled();
	}

	void STedsTableViewer::OnDragLeave(const FDragDropEvent& DragDropEvent)
	{
		if (DropHandler)
		{
			DropHandler->OnDragLeave(DragDropEvent);
		}
		
		if (ICoreProvider* DataStorage = Model->GetDataStorageInterface())
		{
			DataStorage->RemoveColumn<FTableViewerDropInfoColumn>(GetWidgetRowHandle());
		}
	}

	FReply STedsTableViewer::OnDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
	{
		FReply Reply = DropHandler ? DropHandler->OnDrop(MyGeometry, DragDropEvent) : FReply::Unhandled();
		if (ICoreProvider* DataStorage = Model->GetDataStorageInterface())
		{
			DataStorage->RemoveColumn<FTableViewerDropInfoColumn>(GetWidgetRowHandle());
		}
		return Reply;
	}

	void STedsTableViewer::SetDropHandler(TUniquePtr<FWidgetDropHandler> InDropHandler)
	{
		DropHandler = MoveTemp(InDropHandler);
	}

	void STedsTableViewer::AddWidgetColumns()
	{
		if(ICoreProvider* DataStorage = Model->GetDataStorageInterface())
		{
			const RowHandle WidgetRowHandle = TedsWidget->GetRowHandle();
		
			if(DataStorage->IsRowAvailable(WidgetRowHandle))
			{
				// The table viewer should not show up as a row in a table viewer because that will cause all sorts of recursion issues
				DataStorage->AddColumn(WidgetRowHandle, FHideRowFromUITag::StaticStruct());

				// Columns we are going to bind to attributes on SListView
				DataStorage->AddColumns<FTableViewerTag, FWidgetContextMenuColumn, FWidgetRowScrolledIntoView,
					FWidgetDoubleClickedColumn, FTableViewerSelectionChangedColumn, FTableViewerRenameSelectionColumn>(WidgetRowHandle);
				DataStorage->AddColumn(WidgetRowHandle, FTableViewerReferenceColumn{.TableViewerWeak = SharedThis(this)});
			}
		}
	}

	void STedsTableViewer::CreateInternalWidget()
	{
		TSharedPtr<SWidget> ContentWidget;

		// If there are no rows and the table viewer wants to show a message
		if(Model->GetRowCount() == 0 && EmptyRowsMessage.IsSet())
		{
			ContentWidget = 
				SNew(SBox)
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Center)
					[
						SNew(STextBlock)
							.Text(EmptyRowsMessage)
					];
		}
		else if(Model->GetColumnCount() == 0)
		{
			ContentWidget =
				SNew(SBox)
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Center)
					[
						SNew(STextBlock)
							.Text(LOCTEXT("EmptyTableViewerColumnsText", "No columns found to display."))
					];
		}
		else
		{
			ContentWidget = ListView.ToSharedRef();
		}

		TedsWidget->SetContent(ContentWidget.ToSharedRef());
	}

	void STedsTableViewer::RefreshColumnWidgets()
	{
		HeaderRowWidget->ClearColumns();
		Model->ForEachColumn([this](const TSharedRef<FTedsTableViewerColumn>& Column)
		{
			HeaderRowWidget->AddColumn(Column->ConstructHeaderRowColumn());
		});

		CreateInternalWidget();
	}
	void STedsTableViewer::OnListSelectionChanged(TableViewerItemPtr Item, ESelectInfo::Type SelectInfo)
	{
		if(OnSelectionChanged.IsBound())
		{
			OnSelectionChanged.Execute(Item, SelectInfo);
		}

		// Some systems can only access the table viewer by the row handle so also fire the selection changed event on the column
		if (ICoreProvider* Storage = Model->GetDataStorageInterface())
		{
			if (FTableViewerSelectionChangedColumn* Column = Storage->GetColumn<FTableViewerSelectionChangedColumn>(GetWidgetRowHandle()))
			{
				if (Column->OnSelectionChanged)
				{
					Column->OnSelectionChanged->Broadcast();
				}
			}
		}
	}

	void STedsTableViewer::SetTableViewerIdentifier(const FName& NewIdentifier)
	{
		// Only use this function when it is necessary to get a name after initialization, the rows associated with the
		// previous name will be removed regardless of the bRemoveTableViewerRowsOnClose argument and re-initialized.
		// If the model hasn't been initialized yet, there are no rows to remove so we can skip this
		if (bModelInitialized && TedsWidget.IsValid() && Model.IsValid())
		{
			// Remove while set to the old identifier rather than deactivating rows
			TableViewerUtils::RemoveAllTableViewerUIRows(Model->GetDataStorageInterface(), GetWidgetRowHandle());
		}
 
		TableViewerIdentifier = NewIdentifier.IsNone() ?
			FName("TEDSTableViewerID", InstanceCounter) :
			NewIdentifier;
		
		// If the model hasn't been initialized yet, we can skip initializing the new rows as it will be done later
		if (bModelInitialized)
		{
			// We want to reinitialize the current rows as table viewer row UI rows under the new mapping
			InitializeAllTableViewerUIRows();
		}
	}

	void STedsTableViewer::SetQueryStack(TSharedPtr<QueryStack::IRowNode> InRowQueryStack)
	{
		Model->SetQueryStack(InRowQueryStack);
	}

	void STedsTableViewer::SetColumns(const TArray<TWeakObjectPtr<const UScriptStruct>>& InColumns)
	{
		Model->SetColumns(InColumns);
		RefreshColumnWidgets();
	}

	void STedsTableViewer::AddCustomRowWidget(const TSharedRef<FTedsTableViewerColumn>& InColumn)
	{
		Model->AddCustomRowWidget(InColumn);
		RefreshColumnWidgets();
	}

	void STedsTableViewer::ForEachSelectedRow(TFunctionRef<void(RowHandle)> InCallback) const
	{
		TArray<TableViewerItemPtr> SelectedRows;
		ListView->GetSelectedItems(SelectedRows);

		for(TableViewerItemPtr& Row : SelectedRows)
		{
			InCallback(Row);
		}
	}

	RowHandle STedsTableViewer::GetWidgetRowHandle() const
	{
		return TedsWidget->GetRowHandle();
	}

	void STedsTableViewer::SetSelection(RowHandle Row, bool bSelected, const ESelectInfo::Type SelectInfo) const
	{
		FTedsRowHandle TedsRowHandle{ .RowHandle = Row };
		ListView->SetItemSelection(TedsRowHandle, bSelected, SelectInfo);
	}

	void STedsTableViewer::ScrollIntoView(RowHandle Row) const
	{
		FTedsRowHandle TedsRowHandle{ .RowHandle = Row };
		ListView->RequestScrollIntoView(TedsRowHandle);
	}

	void STedsTableViewer::ClearSelection() const
	{
		ListView->ClearSelection();
	}

	TSharedRef<SWidget> STedsTableViewer::AsWidget()
	{
		return AsShared();
	}

	bool STedsTableViewer::IsSelected(RowHandle InRow) const
	{
		FTedsRowHandle TedsRowHandle{ .RowHandle = InRow };
		return ListView->IsItemSelected(TedsRowHandle);
	}

	bool STedsTableViewer::IsSelectedExclusively(RowHandle InRow) const
	{
		return IsSelected(InRow) && ListView->GetNumItemsSelected() == 1;
	}

	bool STedsTableViewer::IsItemVisible(TableViewerItemPtr InItem) const
	{
		return ListView->IsItemVisible(InItem);
	}
	
	void STedsTableViewer::GetTableViewerRowMapping(const FName& InTableViewerIdentifier, const RowHandle InRow, FMapKey& OutMapKey) const
	{
		if(GetCustomTableViewerRowMapping.IsBound())
		{
			GetCustomTableViewerRowMapping.Execute(InTableViewerIdentifier, InRow, OutMapKey);
			if (OutMapKey.IsSet())
			{
				return;
			}
		}
		TableViewerUtils::CreateRowMappingKey(InTableViewerIdentifier, InRow, OutMapKey);
	}

	RowHandle STedsTableViewer::AddOrFindTableViewerUIRow(const TableViewerItemPtr InItem) const
	{
		if (ICoreProvider* Storage = Model->GetDataStorageInterface())
		{
			FMapKey MappingKey;
			GetTableViewerRowMapping(TableViewerIdentifier, InItem.RowHandle, MappingKey);
			// If the row already existed but was removed from TreeView's SparseItemInfos (Searching, Filters, etc.), re-expand it
			if (const RowHandle ExistingRow = Storage->LookupMappedRow(TableViewerUIRowMappingDomain, MappingKey); Storage->IsRowAvailable(ExistingRow))
			{
				Storage->RemoveColumns<FTedsTableViewerInactiveRowTag>(ExistingRow);
				return ExistingRow;
			}
			else
			{
				const RowHandle AddedRow = Storage->AddRow(Storage->FindTable(TableViewerUtils::GetRowWidgetTableName()));
				
				Storage->SetParentRow(Storage->FindHierarchyByName(TableViewerUtils::GetWidgetHierarchyName()), AddedRow, GetWidgetRowHandle());
				Storage->MapRow(TableViewerUIRowMappingDomain, MappingKey, AddedRow);
				
				return AddedRow;
			}
		}

		return InvalidRowHandle;
	}

	void STedsTableViewer::InitializeAllTableViewerUIRows() const
	{
		TableViewerUtils::InitializeAllTableViewerUIRows(
			Model->GetDataStorageInterface(),
			Model->GetItems(), 
			TableViewerUtils::FGetTableViewerRowMapping::CreateSP(this, &STedsTableViewer::GetTableViewerRowMapping), 
			TableViewerIdentifier, 
			GetWidgetRowHandle());
	}

	TSharedRef<ITableRow> STedsTableViewer::MakeTableRowWidget(TableViewerItemPtr InItem, const TSharedRef<STableViewBase>& OwnerTable) const
	{
		const RowHandle TableViewerRowUIRow = AddOrFindTableViewerUIRow(InItem);
		
		return SNew(STedsTableViewerRow, OwnerTable, Model.ToSharedRef(), SharedThis(this))
				.ItemHeight(ItemHeight)
				.Padding(ItemPadding)
				.WidgetRowHandle(TableViewerRowUIRow)
				.TableViewerWidgetRowHandle(GetWidgetRowHandle())
				.Item(InItem);
	}
} // namespace UE::Editor::DataStorage

#undef LOCTEXT_NAMESPACE //"STedsTableViewer"

