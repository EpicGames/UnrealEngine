// Copyright Epic Games, Inc. All Rights Reserved.


#include "Widgets/STedsTableViewerRow.h"

#include "Columns/SlateDelegateColumns.h"
#include "Editor.h"
#include "Elements/Columns/TypedElementHiearchyColumns.h"
#include "DragAndDrop/TedsDragDropOp.h"
#include "TedsTableViewerColumn.h"
#include "TedsTableViewerUtils.h"
#include "TedsTableViewerWidgetColumns.h"
#include "Widgets/STedsTableViewer.h"

#define LOCTEXT_NAMESPACE "TedsTableViewerRow"

namespace UE::Editor::DataStorage
{
	void STedsTableViewerRow::Construct(const FArguments& InArgs, TSharedRef<STableViewBase> OwnerTableView, TSharedRef<FTedsTableViewerModel> InTableViewerModel, TWeakPtr<const ITableViewer> InTableViewer)
	{
		Item = InArgs._Item;
		TableViewerModel = MoveTemp(InTableViewerModel);
		WeakTableViewer = MoveTemp(InTableViewer);
		WidgetRowHandle = InArgs._WidgetRowHandle;
		TableViewerWidgetRowHandle = InArgs._TableViewerWidgetRowHandle;
		ItemHeight = InArgs._ItemHeight;
		bShowWires = InArgs._ShowWires;

		const auto Args = FSuperRowType::FArguments()
		.Padding(InArgs._Padding)
		.ShowWires(bShowWires)
		.OnDragDetected(this, &STedsTableViewerRow::HandleOnDragDetected)
		.OnDragEnter(this, &STedsTableViewerRow::HandleOnDragEnter)
		.OnDragLeave(this, &STedsTableViewerRow::HandleOnDragLeave)
		.Style(&FAppStyle::Get().GetWidgetStyle<FTableRowStyle>("SceneOutliner.TableViewRow"));

		SMultiColumnTableRow<TableViewerItemPtr>::Construct(Args, MoveTemp(OwnerTableView));
	}

	TSharedRef<SWidget> STedsTableViewerRow::GenerateWidgetForColumn(const FName& ColumnName)
	{
		if(TSharedPtr<FTedsTableViewerColumn> Column = TableViewerModel->GetColumn(ColumnName))
		{
			// Lambda so we can access member function (capturing this is safe since this lambda is run immediately and discarded)
			auto WidgetRowSetupDelegate = [this] (ICoreProvider& InStorage, const RowHandle& InUIRowHandle)
			{
				SetupWidgetColumns(InStorage, InUIRowHandle);
			};

			if (TSharedPtr<SWidget> RowWidget = Column->ConstructRowWidget(Item, WidgetRowSetupDelegate))
			{
				return SNew(SBox)
						.HeightOverride(this, &STedsTableViewerRow::GetCurrentItemHeight)
						.MinDesiredHeight(20.f)
						.VAlign(VAlign_Center)
						[
							RowWidget.ToSharedRef()
						];
			}
		}

		return SNullWidget::NullWidget;
	}

	void STedsTableViewerRow::SetupWidgetColumns(ICoreProvider& InStorage, const RowHandle& InUIRowHandle)
	{
		const FHierarchyHandle WidgetHierarchy = InStorage.FindHierarchyByName(TableViewerUtils::GetWidgetHierarchyName());
		InStorage.SetParentRow(WidgetHierarchy, InUIRowHandle, WidgetRowHandle);

		// Currently only this column is needed (not FExternalWidgetExclusiveSelectionColumn) since the double-click action will automatically
		// deselect all and select only the desired row. 
		InStorage.AddColumn(InUIRowHandle, FExternalWidgetSelectionColumn { .IsSelected = MakeShared<FIsSelected>(FIsSelected::CreateSP(this, &STedsTableViewerRow::IsSelected))});
	}

	FOptionalSize STedsTableViewerRow::GetCurrentItemHeight() const
	{
		return ItemHeight.IsSet() ? ItemHeight.Get() : FOptionalSize();
	}

	FReply STedsTableViewerRow::HandleOnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
	{
		if (MouseEvent.IsMouseButtonDown(EKeys::LeftMouseButton))
		{
			if (TSharedPtr<const ITableViewer> Viewer = WeakTableViewer.Pin())
			{
				// Add a column noting that a drop event is currently in progress
				if (ICoreProvider* Storage = TableViewerModel->GetDataStorageInterface())
				{
					Storage->AddColumn(TableViewerWidgetRowHandle, FTableViewerDropInfoColumn{.DropTarget = InvalidRowHandle});
				}

				FRowHandleArray Rows;
				Viewer->ForEachSelectedRow([&Rows](RowHandle Row)
				{
					Rows.Add(Row);
				});
			
				TSharedRef<Widgets::FTedsDragDropOp> Operation = Widgets::FTedsDragDropOp::New(MoveTemp(Rows));
				return FReply::Handled().BeginDragDrop(Operation);
			}
			
		}

		return FReply::Unhandled();
	}

	void STedsTableViewerRow::HandleOnDragEnter(const FDragDropEvent& DragDropEvent)
	{
		// If the drag enters this widget, the row handle corresponding to it is the drop target
		// The actual drop logic is handled in the table viewer widget itself
		if (ICoreProvider* Storage = TableViewerModel->GetDataStorageInterface())
		{
			Storage->AddColumn(TableViewerWidgetRowHandle, FTableViewerDropInfoColumn{.DropTarget = Item});
		}
	}

	void STedsTableViewerRow::HandleOnDragLeave(const FDragDropEvent& DragDropEvent)
	{
		// If the drag exits this widget, remove the corresponding row handle as the drop target
		// The actual drop logic is handled in the table viewer widget itself
		if (ICoreProvider* Storage = TableViewerModel->GetDataStorageInterface())
		{
			if (FTableViewerDropInfoColumn* DropInfoColumn = Storage->GetColumn<FTableViewerDropInfoColumn>(TableViewerWidgetRowHandle))
			{
				DropInfoColumn->DropTarget = InvalidRowHandle;
			}
		}
	}

	TSharedRef<SWidget> SHierarchyViewerRow::GenerateWidgetForColumn(const FName& ColumnName)
	{
		TSharedRef<SWidget> ActualWidget = STedsTableViewerRow::GenerateWidgetForColumn(ColumnName);

		FName PrimaryColumName = TableViewerModel->GetPrimaryColumnName();
		
		bool bShowExpanderArrow =
			PrimaryColumName.IsNone()
			? TableViewerModel->GetColumnIndex(ColumnName) == 0 // If there was no primary column specified, the expander shows up on the leftmost column
			: PrimaryColumName == ColumnName; // Otherwise it shows up on the primary column only
			
		if (bShowExpanderArrow)
		{
			return SNew(SBox)
				.MinDesiredHeight(20.0f)
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.AutoWidth()
					.Padding(6.f, 0.f, 0.f, 0.f)
					[
						SNew(SExpanderArrow, SharedThis(this))
						.IndentAmount(12)
						.ShouldDrawWires(bShowWires)
					]
					+ SHorizontalBox::Slot()
					.FillWidth(1.0f)
					[
						ActualWidget
					]
				];
		}

		return ActualWidget;
	}
} // namespace UE::Editor::DataStorage

#undef LOCTEXT_NAMESPACE
