// Copyright Epic Games, Inc. All Rights Reserved.


#include "Widgets/STedsTileViewer.h"

#include "Brushes/SlateColorBrush.h"
#include "Columns/SlateDelegateColumns.h"
#include "DataStorage/Features.h"
#include "Elements/Columns/TypedElementUIColumns.h"
#include "Elements/Framework/TypedElementAttributeBinding.h"
#include "Elements/Framework/TypedElementDataStorageWidget.h"
#include "TedsTableViewerColumn.h"
#include "TedsTableViewerUtils.h"
#include "TedsTableViewerWidgetColumns.h"
#include "Elements/Columns/TypedElementHiearchyColumns.h"
#include "Styling/StyleColors.h"
#include "Widgets/STedsTableViewerRow.h"

namespace UE::Editor::DataStorage
{
	void STedsTileViewer::Construct(const FArguments& InArgs)
	{
		OnSelectionChanged = InArgs._OnSelectionChanged;
		WidgetPurpose = InArgs._WidgetPurpose;
		Columns = InArgs._Columns;
		TableRowStyle = InArgs._TileStyle;
		TilePadding = InArgs._TilePadding;

		// Use the default purpose if the user didn't specify any
		// TODO: Have a better "default" widget for tiles instead of the default cell widget purpose
		if (!WidgetPurpose.IsSet())
		{
			if (IUiProvider* StorageUi = GetMutableDataStorageFeature<IUiProvider>(UiFeatureName))
			{
				WidgetPurpose = StorageUi->GetGeneralWidgetPurposeID();
			}
			
		}
		
		// While we might have columns for the widget constructor, we don't want to display any UI columns since that doesn't make sense
		// so we give FTedsTableViewerModel empty dummy columns and purposes
		IUiProvider::FPurposeID DummyPurpose;
		TArray<TWeakObjectPtr<const UScriptStruct>> DummyColumns;
		TArray<TSharedPtr<FColumnMetaData>> DummyColumnMetaData;
		
		Model = MakeShared<FTedsTableViewerModel>(FTedsTableViewerModel::FArguments{
			.RowQueryStack = InArgs._QueryStack,
			.RequestedColumns = DummyColumns,
			.ColumnMetaData = DummyColumnMetaData,
			.CellWidgetPurpose = DummyPurpose,
			.HeaderWidgetPurpose = DummyPurpose,
			.IsItemVisibleDelegate = FTedsTableViewerModel::FIsItemVisible::CreateSP(this, &STedsTileViewer::IsItemVisible),
			.bShowHiddenRows = InArgs._ShowHiddenRows
		});

		IUiProvider* StorageUi = Model->GetDataStorageUiProvider();
		
		if (ensure(StorageUi))
		{
			TedsWidget = StorageUi->CreateContainerTedsWidget(InvalidRowHandle);
			
			ChildSlot
			[
				TedsWidget->AsWidget()
			];
		}

		AddWidgetColumns();
		
		// Attribute binder to bind widget columns to attributes on the ListView
		FAttributeBinder Binder(TedsWidget->GetRowHandle());

		TileView = SNew(STileView<TableViewerItemPtr>)
			.ItemAlignment(InArgs._ItemAlignment)
			.ListItemsSource(&Model->GetItems())
			.OnGenerateTile(this, &STedsTileViewer::MakeTileWidget)
			.OnSelectionChanged(this, &STedsTileViewer::OnListSelectionChanged)
			.ItemWidth(InArgs._ItemWidth)
			.ItemHeight(InArgs._ItemHeight)
			.SelectionMode(InArgs._SelectionMode)
			.OnContextMenuOpening(Binder.BindEvent(&FWidgetContextMenuColumn::OnContextMenuOpening))
			.OnItemScrolledIntoView(Binder.BindEvent(&FWidgetRowScrolledIntoView::OnItemScrolledIntoView))
			.OnMouseButtonDoubleClick(Binder.BindEvent(&FWidgetDoubleClickedColumn::OnMouseButtonDoubleClick));

		TedsWidget->SetContent(TileView.ToSharedRef());
		
		// Whenever the model changes, refresh the list to update the UI
		Model->GetOnModelChanged().AddLambda([this]()
		{
			TileView->RequestListRefresh();
		});

		CreateTileWidgetConstructor();
	}

	void STedsTileViewer::CreateTileWidgetConstructor()
	{
		// Empty the Model's columns
		Model->SetColumns(TArray<TWeakObjectPtr<const UScriptStruct>>());

		// We're going to use a custom column to represent our tile so we can use FTedsTableViewerModel with STileView which doesn't have the concept
		// of columns
		TSharedPtr<FTedsTableViewerColumn> Column;
		
		auto AssignWidgetToColumn = [&Column](IUiProvider::FWidgetConstructorPtr WidgetConstructor, TConstArrayView<TWeakObjectPtr<const UScriptStruct>> InMatchedColumns)
		{
			TArray<TWeakObjectPtr<const UScriptStruct>> MatchedColumnsCopy(InMatchedColumns);
			Column = MakeShared<FTedsTableViewerColumn>(TEXT("TileView"), WidgetConstructor, MatchedColumnsCopy);
			return false;
		};
	
		if (IUiProvider* StorageUi = GetMutableDataStorageFeature<IUiProvider>(UiFeatureName))
		{
			RowHandle WidgetPurposeRow = StorageUi->FindPurpose(WidgetPurpose);

			if (Columns.IsEmpty())
			{
				StorageUi->CreateWidgetConstructors(WidgetPurposeRow, FMetaDataView(), AssignWidgetToColumn);
			}
			else
			{
				StorageUi->CreateWidgetConstructors(WidgetPurposeRow, IUiProvider::EMatchApproach::ExactMatch, Columns,
					FMetaDataView(), AssignWidgetToColumn);
			}
		}
		
		if (Column)
		{
			Model->AddCustomRowWidget(Column.ToSharedRef());
		}
	}

	void STedsTileViewer::ForEachSelectedRow(TFunctionRef<void(RowHandle)> InCallback) const
	{
		TArray<TableViewerItemPtr> SelectedRows;
		TileView->GetSelectedItems(SelectedRows);

		for(TableViewerItemPtr& Row : SelectedRows)
		{
			InCallback(Row);
		}
	}

	RowHandle STedsTileViewer::GetWidgetRowHandle() const
	{
		return TedsWidget->GetRowHandle();
	}

	void STedsTileViewer::SetSelection(RowHandle Row, bool bSelected, const ESelectInfo::Type SelectInfo) const
	{
		FTedsRowHandle TedsRowHandle{ .RowHandle = Row };
		TileView->SetItemSelection(TedsRowHandle, bSelected, SelectInfo);

	}

	void STedsTileViewer::ScrollIntoView(RowHandle Row) const
	{
		FTedsRowHandle TedsRowHandle{ .RowHandle = Row };
		TileView->RequestScrollIntoView(TedsRowHandle);
	}

	void STedsTileViewer::ClearSelection() const
	{
		TileView->ClearSelection();
	}

	TSharedRef<SWidget> STedsTileViewer::AsWidget()
	{
		return AsShared();
	}

	bool STedsTileViewer::IsSelected(RowHandle InRow) const
	{
		FTedsRowHandle TedsRowHandle{ .RowHandle = InRow };
		return TileView->IsItemSelected(TedsRowHandle);
	}

	bool STedsTileViewer::IsSelectedExclusively(RowHandle InRow) const
	{
		return IsSelected(InRow) && TileView->GetNumItemsSelected() == 1;
	}

	void STedsTileViewer::SetTableViewerIdentifier(const FName& NewIdentifier)
	{
		// Do nothing, a name identifier is not needed for the Tile Viewer at the moment
	}
	
	void STedsTileViewer::SetQueryStack(TSharedPtr<QueryStack::IRowNode> InRowQueryStack)
	{
		Model->SetQueryStack(InRowQueryStack);
	}

	void STedsTileViewer::SetColumns(const TArray<TWeakObjectPtr<const UScriptStruct>>& InColumns)
	{
		// We aren't using columns in the traditional way, so all we have to do is update our columns list and re-create the tile constructor
		Columns = InColumns;
		CreateTileWidgetConstructor();
	}

	void STedsTileViewer::AddCustomRowWidget(const TSharedRef<FTedsTableViewerColumn>& InColumn)
	{
		// The Tile Viewer doesn't have traditional columns, so adding a custom row widget is not implemented yet (where should the widget be placed on the tile?)
		ensureMsgf(false, TEXT("Adding custom columns is currently not supported in the Tile Viewer!"));
	}

	TSharedRef<ITableRow> STedsTileViewer::MakeTileWidget(TableViewerItemPtr InItem,
	                                                      const TSharedRef<STableViewBase>& OwnerTable) const
	{
		TSharedPtr<SWidget> RowWidget = SNullWidget::NullWidget;
		
		if (TSharedPtr<FTedsTableViewerColumn> Column = Model->GetColumn("TileView"))
		{
			RowWidget = Column->ConstructRowWidget(InItem,
				[&](ICoreProvider& DataStorage, const RowHandle& WidgetRow)
				{
					FHierarchyHandle WidgetHierarchy = DataStorage.FindHierarchyByName(TableViewerUtils::GetWidgetHierarchyName());
					DataStorage.SetParentRow(WidgetHierarchy, WidgetRow, GetWidgetRowHandle());
					DataStorage.AddColumn(WidgetRow, FExternalWidgetSelectionColumn{ .IsSelected = MakeShared<FIsSelected>(FIsSelected::CreateSP(this, &STedsTileViewer::IsSelected, InItem.RowHandle)) });
					DataStorage.AddColumn(WidgetRow, FExternalWidgetExclusiveSelectionColumn{ .IsSelectedExclusively = MakeShared<FIsSelected>(FIsSelected::CreateSP(this, &STedsTileViewer::IsSelectedExclusively, InItem.RowHandle)) });
				});
		}

		return SNew(STableRow<TableViewerItemPtr>, OwnerTable)
			.Padding(TilePadding)
			.Style(TableRowStyle)
			.Cursor(bAllowDragging ? EMouseCursor::GrabHand : EMouseCursor::Default)
			[
				RowWidget.ToSharedRef()
			];
	}

	bool STedsTileViewer::IsItemVisible(TableViewerItemPtr InItem) const
	{
		return TileView->IsItemVisible(InItem);
	}

	void STedsTileViewer::OnListSelectionChanged(TableViewerItemPtr Item, ESelectInfo::Type SelectInfo)
	{
		if(OnSelectionChanged.IsBound())
		{
			OnSelectionChanged.Execute(Item);
		}

		// Some systems can only access the table viewer by the row handle so also fire the selection changed event on the column
		if (ICoreProvider* Storage = Model->GetDataStorageInterface())
		{
			if (FTableViewerSelectionChangedColumn* Column = Storage->GetColumn<FTableViewerSelectionChangedColumn>(GetWidgetRowHandle()))
			{
				if (Column->OnSelectionChanged)
				{
					Column->OnSelectionChanged->Broadcast();
				}
			}
		}
	}

	void STedsTileViewer::AddWidgetColumns()
	{
		if(ICoreProvider* DataStorage = Model->GetDataStorageInterface())
		{
			const RowHandle WidgetRowHandle = TedsWidget->GetRowHandle();

			if (DataStorage->IsRowAvailable(WidgetRowHandle))
			{
				// The table viewer should not show up as a row in a table viewer because that will cause all sorts of recursion issues
				DataStorage->AddColumn(WidgetRowHandle, FHideRowFromUITag::StaticStruct());

				// Columns we are going to bind to attributes on SListView
				DataStorage->AddColumns<FTableViewerTag, FWidgetContextMenuColumn, FWidgetRowScrolledIntoView,
					FWidgetDoubleClickedColumn, FTableViewerSelectionChangedColumn, FTableViewerRenameSelectionColumn>(WidgetRowHandle);
				DataStorage->AddColumn(WidgetRowHandle, FTableViewerReferenceColumn{.TableViewerWeak = SharedThis(this)});

			}
		}
	}
}


