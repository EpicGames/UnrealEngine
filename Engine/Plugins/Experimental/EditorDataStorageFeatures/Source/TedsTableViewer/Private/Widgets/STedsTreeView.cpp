// Copyright Epic Games, Inc. All Rights Reserved

#include "STedsTreeView.h"

#include "Columns/SlateDelegateColumns.h"
#include "Elements/Framework/TypedElementAttributeBinding.h"

namespace UE::Editor::DataStorage
{
	void STedsTreeView::Construct( const FArguments& InArgs, const FOnGetParent& InOnGetParent, RowHandle InWidgetRow )
	{
		checkf(InOnGetParent.IsBound(), TEXT("Must provide FOnGetParent delegate."));
		checkf(InArgs._RowsSource, TEXT("Must provide a source for the rows to be displayed."));

		OnGetParent = InOnGetParent;
		RowsSource = InArgs._RowsSource;
		WidgetRow = InWidgetRow;
		
		// Attribute binder to bind widget columns to attributes on the ListView
		FAttributeBinder Binder(WidgetRow);
			
		STreeView::Construct(STreeView<TableViewerItemPtr>::FArguments()
			.HeaderRow(InArgs._HeaderRow)
			.TreeItemsSource( InArgs._TopLevelRowsSource)
			.OnGenerateRow(InArgs._OnGenerateRow)
			.OnExpansionChanged(InArgs._OnExpansionChanged)
			.OnSetExpansionRecursive(InArgs._OnSetExpansionRecursive)
			.OnSelectionChanged(InArgs._OnSelectionChanged)
			.SelectionMode(InArgs._SelectionMode)
			.OnContextMenuOpening(Binder.BindEvent(&FWidgetContextMenuColumn::OnContextMenuOpening))
			.OnItemScrolledIntoView(Binder.BindEvent(&FWidgetRowScrolledIntoView::OnItemScrolledIntoView))
			.OnMouseButtonDoubleClick(Binder.BindEvent(&FWidgetDoubleClickedColumn::OnMouseButtonDoubleClick))
			.OnGetChildren(this, &STedsTreeView::GetChildren_Internal)
			.AllowInvisibleItemSelection(InArgs._AllowInvisibleItemSelection));
	}

	void STedsTreeView::Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime )
	{
		if (bDirty)
		{
			UpdateTreeMap();
			// The list refresh will invalidate the layout in STableViewBase::RequestLayoutRefresh()
			// and we want to defer that until after we update the tree map.
			// We are still marking the base class for refresh here prior to it's next tick which we call below.
			STreeView<TableViewerItemPtr>::RequestListRefresh();
			bDirty = false;
		}

		STreeView::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);

	}
		
	void STedsTreeView::RequestListRefresh()
	{
		bDirty = true;
	}
	
	void STedsTreeView::UpdateTreeMap()
	{
		/*
		 * Currently, whenever a tree refresh is requested we re-calulate the hierarchy of all items and store it, and simply re-use the existing
		 * top down logic in STreeView to provide the actual hierarchy to the widget. In the future this can be optimized to override the whole logic
		 * contained in STreeView's Tick() function and use a completely bottom up approach.
		 */
		TreeMap.Empty();

		const TArrayView<const TableViewerItemPtr> ItemsSourceRef(*RowsSource);

		// For each item, grab the parent and add the item to the parent item's array of children
		for (TableViewerItemPtr Item : ItemsSourceRef)
		{
			if (TableViewerItemPtr Parent = OnGetParent.Execute(Item))
			{
				if (TArray<TableViewerItemPtr>* FoundChildren = TreeMap.Find(Parent))
				{
					FoundChildren->Add(Item);
				}
				else
				{
					TreeMap.Emplace(Parent, {Item});
				}
			}
		}
	}

	void STedsTreeView::GetChildren_Internal(TableViewerItemPtr InParent, TArray<TableViewerItemPtr>& OutChildren)
	{
		// Simply look up the child in our internally cached array for the row
		if (TArray<TableViewerItemPtr>* FoundArray = TreeMap.Find(InParent))
		{
			OutChildren = *FoundArray; 
		}
	}
}
