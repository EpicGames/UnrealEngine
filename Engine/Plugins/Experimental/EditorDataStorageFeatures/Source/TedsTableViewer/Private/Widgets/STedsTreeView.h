// Copyright Epic Games, Inc. All Rights Reserved
 
#pragma once

#include "Widgets/Views/STreeView.h"
#include "TedsTableViewerModel.h"

namespace UE::Editor::DataStorage
{
	/**
	 * A tree view widget specialized for TableViewerItemPtr to support constructing the widget using a bottom up hierarchy (FOnGetParent)
	 */
	class STedsTreeView : public STreeView<TableViewerItemPtr>
	{
	public:

		/** Delegate that gets the parent row handle for a given row */
		DECLARE_DELEGATE_RetVal_OneParam(TableViewerItemPtr /** OutParent */, FOnGetParent, TableViewerItemPtr /** InChild */)

		SLATE_BEGIN_ARGS(STedsTreeView) {}

			/** All the rows that can be displayed by the widget */
			SLATE_ARGUMENT( const TArray<TableViewerItemPtr>*, RowsSource )

			/** Only the top level rows being displayed by the widget */
			SLATE_ARGUMENT( const TArray<TableViewerItemPtr>*, TopLevelRowsSource )

			/** Delegate to generate the actual row widget */
			SLATE_EVENT( FOnGenerateRow, OnGenerateRow )

			/** Delegate fired on selection change */
			SLATE_EVENT( FOnSelectionChanged, OnSelectionChanged )
			
			/** Delegate fired on expansion change */
			SLATE_EVENT( FOnExpansionChanged, OnExpansionChanged )

			/** Delegate fired on a request to change expansion on an item and it's children */
			SLATE_EVENT( FOnSetExpansionRecursive, OnSetExpansionRecursive )

			/** Delegate that determines the selection mode */
			SLATE_ATTRIBUTE( ESelectionMode::Type, SelectionMode )

			/** The header row widget to use */
			SLATE_ARGUMENT( TSharedPtr<SHeaderRow>, HeaderRow )

			/** Allows collapsed or temporarily hidden items to still be selected */
			SLATE_ARGUMENT(bool, AllowInvisibleItemSelection);
			
		SLATE_END_ARGS()
		
		void Construct( const FArguments& InArgs, const FOnGetParent& InOnGetParent, RowHandle InWidgetRow );

		virtual void Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime ) override;
		virtual void RequestListRefresh() override;

	protected:

		// Update the internal tree map used to contain the hierarchy
		void UpdateTreeMap();

		// For a given row, get all the children
		void GetChildren_Internal(TableViewerItemPtr InParent, TArray<TableViewerItemPtr>& OutChildren);

		protected:
		
		FOnGetParent OnGetParent;
		
		RowHandle WidgetRow = InvalidRowHandle;

		// All rows that can be shown in this tree, including the full hieararchy regardless of whether it is currently expanded
		const TArray<TableViewerItemPtr>* RowsSource = nullptr;

		// The internal tree map used to contain the hierarchy
		TMap<TableViewerItemPtr, TArray<TableViewerItemPtr>> TreeMap;

		// Whether the TreeMap needs to be re-calculated
		bool bDirty = false;
	};
}


