// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/TableViewerWidgetDropHandler.h"

#include "DragAndDrop/DropOperationInput.h"
#include "ITedsTableViewer.h"
#include "TedsTableViewerWidgetColumns.h"

namespace UE::Editor::DataStorage::Widgets
{
	FTableViewerWidgetDropHandler::FTableViewerWidgetDropHandler(TWeakPtr<ITableViewer> InTableViewer, TWeakObjectPtr<UTypedElementSelectionSet> InSelectionSet)
		: FWidgetDropHandler(MoveTemp(InSelectionSet))
		, TableViewer(MoveTemp(InTableViewer))
	{
	}

	FTableViewerWidgetDropHandler::~FTableViewerWidgetDropHandler()
	{
	}

	FWidgetDropHandler::EUpdateParameters FTableViewerWidgetDropHandler::UpdateParameters(ICoreProvider& Storage, const FGeometry& Geometry,
		const FDragDropEvent& InputEvent)
	{
		RowHandle TargetRow = GetDropTarget(Storage);

		bool bTargetRowChanged = LastTargetRow != TargetRow;
		LastTargetRow = TargetRow;
		if (bTargetRowChanged)
		{
			return EUpdateParameters::ResetOperations;
		}
		return FWidgetDropHandler::UpdateParameters(Storage, Geometry, InputEvent);
	}

	void FTableViewerWidgetDropHandler::SetRowsFromParameters(ICoreProvider& Storage, FRowHandleArrayView Rows) const
	{
		for (RowHandle Row : Rows)
		{
			Storage.AddColumn(Row, Operations::FDropTargetColumn{ .Value = LastTargetRow });
		}
		FWidgetDropHandler::SetRowsFromParameters(Storage, Rows);
	}

	RowHandle FTableViewerWidgetDropHandler::GetDropTarget(ICoreProvider& Storage) const
	{
		if (TSharedPtr<ITableViewer> TableViewerPtr = TableViewer.Pin())
		{
			RowHandle TableViewerRow = TableViewerPtr->GetWidgetRowHandle();

			// If we are dragging onto a specific row, that's the target
			if (FTableViewerDropInfoColumn* DropInfoColumn = Storage.GetColumn<FTableViewerDropInfoColumn>(TableViewerRow))
			{
				if (Storage.IsRowAvailable(DropInfoColumn->DropTarget))
				{
					return DropInfoColumn->DropTarget;
				}
			}
			// Otherwise if we have a valid "root" row, that is our drop target
			if (FHierarchyViewerRootColumn* RootColumn = Storage.GetColumn<FHierarchyViewerRootColumn>(TableViewerRow))
			{
				return RootColumn->Root;
			}
		}

		return InvalidRowHandle;
	}

	void FTableViewerWidgetDropHandler::Stop(ICoreProvider& Storage, UDropOperationSystem& DropSystem)
	{
		LastTargetRow = InvalidRowHandle;
		
		FWidgetDropHandler::Stop(Storage, DropSystem);
	}
}
