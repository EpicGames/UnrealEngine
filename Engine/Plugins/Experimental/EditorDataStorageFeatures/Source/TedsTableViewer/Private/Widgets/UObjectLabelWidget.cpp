// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/UObjectLabelWidget.h"

#include "Elements/Columns/TypedElementCompatibilityColumns.h"
#include "Elements/Columns/TypedElementLabelColumns.h"
#include "Elements/Framework/TypedElementAttributeBinding.h"
#include "TedsTableViewerUtils.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/SBoxPanel.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(UObjectLabelWidget)


#define LOCTEXT_NAMESPACE "FUObjectLabelWidgetConstructor"

void UUObjectLabelWidgetFactory::RegisterWidgetConstructors(UE::Editor::DataStorage::ICoreProvider& DataStorage,
	UE::Editor::DataStorage::IUiProvider& DataStorageUi) const
{
	using namespace UE::Editor::DataStorage::Queries;
	DataStorageUi.RegisterWidgetFactory<FUObjectLabelWidgetConstructor>(
		DataStorageUi.FindPurpose(IUiProvider::FPurposeInfo("General", "RowLabel", NAME_None).GeneratePurposeID()),
		TColumn<FTypedElementLabelColumn>() && TColumn<FTypedElementUObjectColumn>());
}

FUObjectLabelWidgetConstructor::FUObjectLabelWidgetConstructor()
	: FSimpleWidgetConstructor(StaticStruct())
{
}

FUObjectLabelWidgetConstructor::FUObjectLabelWidgetConstructor(const UScriptStruct* TypeInfo)
	: FSimpleWidgetConstructor(TypeInfo)
{
}

TSharedPtr<SWidget> FUObjectLabelWidgetConstructor::CreateWidget(UE::Editor::DataStorage::ICoreProvider* DataStorage,
	UE::Editor::DataStorage::IUiProvider* DataStorageUi, UE::Editor::DataStorage::RowHandle TargetRow,
	UE::Editor::DataStorage::RowHandle WidgetRow, const UE::Editor::DataStorage::FMetaDataView& Arguments)
{
	using namespace UE::Editor::DataStorage;
	if(DataStorage->IsRowAvailable(TargetRow))
	{
		FAttributeBinder Binder(TargetRow, DataStorage);

		// Once TEDS UI has widget combining functionality, the binder can be used to create the type info widget and label widget and combine them
		return SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
				.AutoWidth()
			[
				SNew(SImage)
					.Image(TableViewerUtils::GetIconForRow(DataStorage, TargetRow))
					.ColorAndOpacity(FSlateColor::UseForeground())
			]
			+SHorizontalBox::Slot()
				.AutoWidth()
			[
				SNew(SSpacer)
					.Size(FVector2D(5.0f, 0.0f))
			]
			+SHorizontalBox::Slot()
				.FillWidth(1.0f)
			[
				SNew(STextBlock)
					.Text(Binder.BindText(&FTypedElementLabelColumn::Label))
					.ToolTipText(Binder.BindText(&FTypedElementLabelColumn::Label))
			];
	}
	else
	{
		return SNew(STextBlock)
			.Text(LOCTEXT("MissingRowReferenceColumn", "Unable to retrieve row reference."));
	}
	
	
}

#undef LOCTEXT_NAMESPACE // "FUObjectLabelWidgetConstructor"
