// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataStorage/Handles.h"
#include "Elements/Interfaces/TypedElementDataStorageInterface.h"

#define UE_API TEDSTABLEVIEWER_API

namespace UE::Editor::DataStorage
{
	class SHierarchyViewer;
	
	// Data interface for the hierarchy viewer to extract hierarchy information from a row
	class IHierarchyViewerDataInterface
	{
	public:

		// Delegate that is fired when the parent of a row on this hierarchy changes
		DECLARE_MULTICAST_DELEGATE_OneParam(FOnParentChanged, RowHandle);
		
		virtual ~IHierarchyViewerDataInterface() = default;
		virtual RowHandle GetParent(const ICoreProvider& Storage, RowHandle InRow) const = 0;
		virtual void WalkDepthFirst(const ICoreProvider& Storage, RowHandle InRow, ICoreProvider::FHierarchyIterationCallback VisitFn,
			ICoreProvider::ETraversalOrder TraversalOrder = ICoreProvider::ETraversalOrder::PreOrder) const = 0;
		
		FOnParentChanged& OnHierarchyChangedEvent()
		{
			return ParentChangedEvent;
		}

	protected:
		FOnParentChanged ParentChangedEvent;
	};

	// Data interface for the hierarchy viewer that operates on a single FHierarchyHandle 
	class FHierarchyViewerData : public IHierarchyViewerDataInterface
	{
	public:
		UE_API explicit FHierarchyViewerData(ICoreProvider& Storage, FHierarchyHandle InHierarchyHandle);
		UE_API virtual ~FHierarchyViewerData() override;
		UE_API virtual RowHandle GetParent(const ICoreProvider& Storage, RowHandle InRow) const override;
		UE_API virtual void WalkDepthFirst(const ICoreProvider& Storage, RowHandle InRow, ICoreProvider::FHierarchyIterationCallback VisitFn,
			ICoreProvider::ETraversalOrder TraversalOrder = ICoreProvider::ETraversalOrder::PreOrder) const override;
		UE_API FHierarchyHandle GetHierarchy() const;

	protected:
		void RegisterQueries(ICoreProvider& Storage);
		void UnRegisterQueries(ICoreProvider& Storage);
		
	protected:
		FHierarchyHandle HierarchyHandle;
		QueryHandle ParentAddedQuery = InvalidQueryHandle;
		QueryHandle ParentRemovedQuery = InvalidQueryHandle;
	};

	// Accepts an ordered list of hierarchies to combine into one singular view.
	class FHierarchyViewerMultiData : public IHierarchyViewerDataInterface
	{
	public:
		UE_API FHierarchyViewerMultiData(ICoreProvider& Storage, TArray<FHierarchyHandle> OrderedHierarchyHandles);
		// Returns the first parent it finds among the list of hierarchies
		UE_API virtual RowHandle GetParent(const ICoreProvider& Storage, RowHandle InRow) const override;
		// Does a DFS of each hierarchy one-by-one. If a child is encountered multiple times in different hierarchies, it is only reported in the first instance
		UE_API virtual void WalkDepthFirst(const ICoreProvider& Storage, RowHandle InRow, ICoreProvider::FHierarchyIterationCallback VisitFn,
			ICoreProvider::ETraversalOrder TraversalOrder = ICoreProvider::ETraversalOrder::PreOrder) const override;

		// Iterate over each contained hierarchy data. The callback can return false to terminate iteration early
		UE_API void ForEachHierarchyData(TFunctionRef<bool(const TSharedRef<FHierarchyViewerData>&)> VisitFn) const;

	protected:
		TArray<TSharedPtr<FHierarchyViewerData>> Hierarchies;
	};

	// Data interface for a hierarchical relation registered directly via RegisterRelationType (HierarchyMode != Disabled).
	// These relations are not registered in FTedsHierarchyRegistrar and have no FHierarchyHandle, but still support
	// parent/child traversal via the RelationTypeHandle API on ICoreProvider.
	class FRelationTypeHierarchyViewerData : public IHierarchyViewerDataInterface
	{
	public:
		UE_API explicit FRelationTypeHierarchyViewerData(RelationTypeHandle InRelationType);
		UE_API virtual RowHandle GetParent(const ICoreProvider& Storage, RowHandle InRow) const override;
		UE_API virtual void WalkDepthFirst(const ICoreProvider& Storage, RowHandle InRow, ICoreProvider::FHierarchyIterationCallback VisitFn,
			ICoreProvider::ETraversalOrder TraversalOrder = ICoreProvider::ETraversalOrder::PreOrder) const override;

	private:
		RelationTypeHandle RelationType;
	};
}

#undef UE_API