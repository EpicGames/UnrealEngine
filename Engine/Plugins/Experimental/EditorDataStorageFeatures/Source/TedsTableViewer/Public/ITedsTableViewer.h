// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "DataStorage/Handles.h"
#include "TedsQueryStackInterfaces.h"
#include "Widgets/SWidget.h"

namespace ESelectInfo { enum Type : int; }

namespace UE::Editor::DataStorage
{
	class FTedsTableViewerColumn;

	// Interface for all types of TEDS table viewers
    class ITableViewer
    {
    public:

    	// Execute the given callback for each row handle selected in the table viewer
        virtual void ForEachSelectedRow(TFunctionRef<void(RowHandle)> InCallback) const = 0;

    	// Get the row handle for the widget row that contains the contents of the table viewer itself
        virtual RowHandle GetWidgetRowHandle() const = 0;

    	// Set the currently selected row in the table viewer
        virtual void SetSelection(RowHandle Row, bool bSelected, const ESelectInfo::Type SelectInfo) const = 0;

    	// Scroll the input row into view in the table viewer
        virtual void ScrollIntoView(RowHandle Row) const = 0;

    	// Deselect all items in the table viewer
        virtual void ClearSelection() const = 0;

    	// Get the table viewer as an SWidget
    	virtual TSharedRef<SWidget> AsWidget() = 0;

    	// Get if a specific row is selected
    	virtual bool IsSelected(RowHandle InRow) const = 0;

    	// Get if a specific row is selected exclusively
    	virtual bool IsSelectedExclusively(RowHandle InRow) const = 0;

    	// Add a custom per-row widget to the table viewer (that doesn't necessarily map to a TEDS column)
    	// This usually means a new column for column-based views
    	virtual void AddCustomRowWidget(const TSharedRef<FTedsTableViewerColumn>& InRowWidgetFactory) = 0;

    	// Clear the current QueryStack being displayed, set it to the given node, and recreate the sorting nodes
    	virtual void SetQueryStack(TSharedPtr<QueryStack::IRowNode> InRowQueryStack) = 0;

    	// Clear the current list of columns being displayed and set it to the given list
    	virtual void SetColumns(const TArray<TWeakObjectPtr<const UScriptStruct>>& InColumns) = 0;
    	
    	// Sets the Table Identifier (Can be used in case data is dynamic and created post widget init)
        virtual void SetTableViewerIdentifier(const FName& NewIdentifier) = 0;
   
    	virtual ~ITableViewer() = default;
    };
}