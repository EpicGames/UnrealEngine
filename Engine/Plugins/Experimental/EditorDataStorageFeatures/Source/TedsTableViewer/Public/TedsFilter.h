// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "TedsFilter.inl"
#include "DataStorage/Handles.h"

#define UE_API TEDSTABLEVIEWER_API

namespace UE::Editor::DataStorage
{
	class STedsFilterBar;
		
	class FTedsFilter : public FTedsFilterBase<FTedsRowHandle&>
	{
		friend class STedsFilterBar;
		
	public:
		// Inherit all constructors from the base class
		using FTedsFilterBase::FTedsFilterBase;
		
		/** Notification that the filter became active or inactive */
		virtual UE_API void ActiveStateChanged(bool bActive) override;
		
		/** Returns whether the specified Item passes the Filter's restrictions */
		virtual UE_API bool PassesFilter(FTedsRowHandle& InItem ) const override;

	protected:
		/** Sets a TedsFilterBar to be used by this Filter */
		UE_API void SetTedsFilterBar(const TSharedPtr<STedsFilterBar>& InTedsFilterBar);
		
		TWeakPtr<STedsFilterBar> TedsFilterBar;
	};
} // namespace UE::Editor::DataStorage

#undef UE_API
