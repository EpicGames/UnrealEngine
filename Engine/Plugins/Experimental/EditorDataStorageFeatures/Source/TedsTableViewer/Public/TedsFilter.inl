// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <type_traits>

#include "DataStorage/Features.h"
#include "DataStorage/Handles.h"
#include "DataStorage/Queries/Description.h"
#include "Elements/Columns/TypedElementTypeInfoColumns.h"
#include "Elements/Framework/TypedElementQueryContext.h"
#include "Elements/Framework/TypedElementQueryBuilder.h"
#include "Filters/FilterBase.h"
#include "Styling/SlateIconFinder.h"

#define LOCTEXT_NAMESPACE "TEDSFilter"

namespace UE::Editor::DataStorage
{
	// Helper function to check if the given QueryHandle is valid to be used in a filter (is not an observer).
	inline bool CheckValidFilterQueryHandle(const QueryHandle& InQueryHandle)
	{
		// Check if the given Query Handle is valid for a filter (Not an Observer Query Handle)
		DataStorage::ICoreProvider* Storage = GetMutableDataStorageFeature<DataStorage::ICoreProvider>(StorageFeatureName);
		if (ensureMsgf(Storage, TEXT("TEDS must be initialized before TEDS Filters")))
		{
			const EQueryCallbackType QueryHandleCallbackType = Storage->GetQueryDescription(InQueryHandle).Callback.Type;
			return ensureMsgf(QueryHandleCallbackType == EQueryCallbackType::None || QueryHandleCallbackType == EQueryCallbackType::Processor,
				TEXT("TEDS Filters cannot accept Observer Query Handles."));
		}
		return false;
	}

	template<typename FilterType>
	class FTedsFilterBase : public FFilterBase<FilterType>
	{
	public:
		/**
		 * Constructor for a TEDS Filter
		 * @param InFilterName FName Identifier of the Filter
		 * @param InFilterDisplayName Display Name to show on the Add Filter Menu and toggle button
		 * @param InFilterToolTip Tooltip to describe the function of the Filter
		 * @param InFilterIconName Name of an icon in the App Style Set to display in the Add Filter Menu
		 * @param InCategory Group to place the filter in the Add Filter Menu
		 * @param InFilterQuery QueryHandle of a registered query to filter for (must be the same syntax as any other Query used in the viewer)
		 * @param bActiveByDefault Whether this filter should be active by default and can be implemented per-inherited filter type
		 */
		FTedsFilterBase(const FName& InFilterName, const FText& InFilterDisplayName, const FText& InFilterToolTip, const FName& InFilterIconName, 
			const TSharedPtr<FFilterCategory>& InCategory, const QueryHandle& InFilterQuery, const bool bActiveByDefault = false);

		/**
		 * Constructor for a TEDS Filter
		 * @param InFilterName FName Identifier of the Filter
		 * @param InFilterDisplayName Display Name to show on the Add Filter Menu and toggle button
		 * @param InFilterQuery QueryHandle of a registered query to filter for (must be the same syntax as any other Query used in the viewer)
		 * @param bActiveByDefault Whether this filter should be active by default and can be implemented per-inherited filter type
		 */
		FTedsFilterBase(const FName& InFilterName, const FText& InFilterDisplayName, const QueryHandle& InFilterQuery, const bool bActiveByDefault = false);

		/**
		 * Constructor for a TEDS Filter
		 * @param InFilterName FName Identifier of the Filter
		 * @param InFilterDisplayName Display Name to show on the Add Filter Menu and toggle button
		 * @param InFilterToolTip Tooltip to describe the function of the Filter
		 * @param InFilterIconName Name of an icon in the App Style Set to display in the Add Filter Menu
		 * @param InCategory Group to place the filter in the Add Filter Menu
		 * @param InFilterQuery QueryFunction to filter for, Batching should be used for improved performance if possible
		 * @param bActiveByDefault Whether this filter should be active by default and can be implemented per-inherited filter type
		 */
		FTedsFilterBase(const FName& InFilterName, const FText& InFilterDisplayName, const FText& InFilterToolTip, const FName& InFilterIconName,
			const TSharedPtr<FFilterCategory>& InCategory, const Queries::TConstQueryFunction<bool>& InFilterQuery, const bool bActiveByDefault = false);

		/**
		 * Constructor for a TEDS Filter
		 * @param InFilterName FName Identifier of the Filter
		 * @param InFilterDisplayName Display Name to show on the Add Filter Menu and toggle button
		 * @param InFilterQuery QueryFunction to filter for, Batching should be used for improved performance if possible
		 * @param bActiveByDefault Whether this filter should be active by default and can be implemented per-inherited filter type
		 */
		FTedsFilterBase(const FName& InFilterName, const FText& InFilterDisplayName, const Queries::TConstQueryFunction<bool>& InFilterQuery,
			const bool bActiveByDefault = false);
		
		/**
		 * Constructor for a TEDS Filter
		 * Require that the template type cannot be one of the pre-defined FilterQuery types so it will pass to the defined
		 * constructor instead of this one.
		 * @param InFilterName FName Identifier of the Filter
		 * @param InFilterDisplayName Display Name to show on the Add Filter Menu and toggle button
		 * @param InFilterToolTip Tooltip to describe the function of the Filter
		 * @param InFilterIconName Name of an icon in the App Style Set to display in the Add Filter Menu
		 * @param InCategory Group to place the filter in the Add Filter Menu
		 * @param InFilterQuery Function callback that checks the value of a column(s) and returns a bool of whether that row passes the filter
		 * @param bActiveByDefault Whether this filter should be active by default and can be implemented per-inherited filter type
		 */
		template< typename FilterFunction UE_REQUIRES(
			!std::is_same_v<std::remove_cvref_t<FilterFunction>, QueryHandle> &&
			!std::is_same_v<std::remove_cvref_t<FilterFunction>, Queries::TConstQueryFunction<bool>>) >
		FTedsFilterBase(const FName& InFilterName, const FText& InFilterDisplayName, const FText& InFilterToolTip, const FName& InFilterIconName,
			const TSharedPtr<FFilterCategory>& InCategory, FilterFunction&& InFilterQuery, const bool bActiveByDefault = false)
			: FTedsFilterBase(
				InFilterName, 
				InFilterDisplayName, 
				InFilterToolTip, 
				InFilterIconName, 
				InCategory,
				Queries::BuildConstQueryFunction<bool>(Forward<FilterFunction>(InFilterQuery)),
				bActiveByDefault)
		{}

		/**
		 * Constructor for a TEDS Filter
		 * Require that the template type cannot be one of the pre-defined FilterQuery types so it will pass to the defined
		 * constructor instead of this one.
		 * @param InFilterName FName Identifier of the Filter
		 * @param InFilterDisplayName Display Name to show on the Add Filter Menu and toggle button
		 * @param InFilterQuery Function callback that checks the value of a column(s) and returns a bool of whether that row passes the filter
		 * @param bActiveByDefault Whether this filter should be active by default and can be implemented per-inherited filter type
		 */
		template< typename FilterFunction UE_REQUIRES(
			!std::is_same_v<std::remove_cvref_t<FilterFunction>, QueryHandle> &&
			!std::is_same_v<std::remove_cvref_t<FilterFunction>, Queries::TConstQueryFunction<bool>>) >
		FTedsFilterBase(const FName& InFilterName, const FText& InFilterDisplayName, FilterFunction&& InFilterQuery, const bool bActiveByDefault = false)
			: FTedsFilterBase(
				InFilterName, 
				InFilterDisplayName, 
				FText::FromString(InFilterName.ToString()), 
				FName(), 
				nullptr,
				Queries::BuildConstQueryFunction<bool>(Forward<FilterFunction>(InFilterQuery)),
				bActiveByDefault)
		{}

		/**
		 * Constructor for a TEDS Class Filter
		 * @param InClass UClass to filter for on the TypeInfo Column
		 * @param InCategory Group to place the filter in the Add Filter Menu
		 * @param bActiveByDefault Whether this filter should be active by default and can be implemented per-inherited filter type
		 */
		FTedsFilterBase(const UClass* InClass, const TSharedPtr<FFilterCategory>& InCategory = nullptr, const bool bActiveByDefault = false);

		virtual ~FTedsFilterBase() override = default;
		
		void SetCategory(const TSharedPtr<FFilterCategory>& InCategory);
        
		bool IsClassFilter() const { return bIsClassFilter; }

		bool IsActiveByDefault() const { return bActiveByDefault; }
		
		FName GetIdentifier() const { return FilterName; }

		/** Returns the system name for this filter */
		virtual FString GetName() const override;

		/** Returns the human-readable name for this filter */
		virtual FText GetDisplayName() const override;

		/** Returns the tooltip for this filter, shown in the filters menu */
		virtual FText GetToolTipText() const override;

		/** Returns the color this filter button will be when displayed as a button */
		virtual FLinearColor GetColor() const override;

		/** Returns the name of the icon to use in menu entries */
		virtual FName GetIconName() const override;

		/** If true, the filter will be active in the FilterBar when it is inactive in the UI (i.e the filter pill is grayed out) */
		virtual bool IsInverseFilter() const override;

		/** Called when the right-click context menu is being built for this filter */
		virtual void ModifyContextMenu(FMenuBuilder& MenuBuilder) override;
		
		/** Can be overriden for custom FilterBar subclasses to save settings, currently not implemented in any generic Filter Bar */
		virtual void SaveSettings(const FString& IniFilename, const FString& IniSection, const FString& SettingsString) const override;

		/** Can be overridden for custom FilterBar subclasses to load settings, currently not implemented in any generic Filter Bar */
		virtual void LoadSettings(const FString& IniFilename, const FString& IniSection, const FString& SettingsString) override;
		
	protected:
		FName FilterName;
		FText FilterDisplayName;
		FText FilterToolTip;
		FName FilterIconName;

		const bool bIsClassFilter;
		const bool bActiveByDefault;

		TVariant<QueryHandle, Queries::TConstQueryFunction<bool>> FilterQuery;
	};

	template<typename FilterType>
	FTedsFilterBase<FilterType>::FTedsFilterBase(const FName& InFilterName, const FText& InFilterDisplayName, const FText& InFilterToolTip, const FName& InFilterIconName, 
		const TSharedPtr<FFilterCategory>& InCategory, const QueryHandle& InFilterQuery, const bool bActiveByDefault)
		: FFilterBase<FilterType>(InCategory)
		, FilterName(InFilterName)
		, FilterDisplayName(InFilterDisplayName)
		, FilterToolTip(InFilterToolTip)
		, FilterIconName(InFilterIconName)
		, bIsClassFilter(false)
		, bActiveByDefault(bActiveByDefault)
	{
		FilterQuery.Set<QueryHandle>(CheckValidFilterQueryHandle(InFilterQuery) ? InFilterQuery : InvalidQueryHandle);
	}

	template<typename FilterType>
	FTedsFilterBase<FilterType>::FTedsFilterBase(const FName& InFilterName, const FText& InFilterDisplayName, const QueryHandle& InFilterQuery, const bool bActiveByDefault)
		: FTedsFilterBase(InFilterName, InFilterDisplayName, FText::FromString(InFilterName.ToString()), FName(), 
			nullptr, InFilterQuery, bActiveByDefault)
	{}

	template<typename FilterType>
	FTedsFilterBase<FilterType>::FTedsFilterBase(const FName& InFilterName, const FText& InFilterDisplayName, const FText& InFilterToolTip, 
		const FName& InFilterIconName, const TSharedPtr<FFilterCategory>& InCategory, const Queries::TConstQueryFunction<bool>& InFilterQuery,
		const bool bActiveByDefault)
		: FFilterBase<FilterType>(InCategory)
		, FilterName(InFilterName)
		, FilterDisplayName(InFilterDisplayName)
		, FilterToolTip(InFilterToolTip)
		, FilterIconName(InFilterIconName)
		, bIsClassFilter(false)
		, bActiveByDefault(bActiveByDefault)
	{
		FilterQuery.Set<Queries::TConstQueryFunction<bool>>(InFilterQuery);
	}

	template<typename FilterType>
	FTedsFilterBase<FilterType>::FTedsFilterBase(const FName& InFilterName, const FText& InFilterDisplayName, const Queries::TConstQueryFunction<bool>& InFilterQuery,
		const bool bActiveByDefault)
		: FTedsFilterBase(InFilterName, InFilterDisplayName, FText::FromString(InFilterName.ToString()), FName(), 
			nullptr, InFilterQuery, bActiveByDefault)
	{}

	template<typename FilterType>
	FTedsFilterBase<FilterType>::FTedsFilterBase(const UClass* InClass, const TSharedPtr<FFilterCategory>& InCategory, const bool bActiveByDefault)
		: FFilterBase<FilterType>(InCategory)
		, FilterName(InClass->GetFName())
		, FilterDisplayName(InClass->GetDisplayNameText())
		, FilterToolTip(FText::Format(LOCTEXT("FilterClassTooltip", "Filter by {0}"), InClass->GetDisplayNameText()))
		, FilterIconName(FSlateIconFinder::FindIconForClass(InClass).GetStyleName())
		, bIsClassFilter(true)
		, bActiveByDefault(bActiveByDefault)
	{
		using namespace UE::Editor::DataStorage::Queries;
		FilterQuery.Set<TConstQueryFunction<bool>>(
			Queries::BuildConstQueryFunction<bool>([InClass](TConstQueryContext<RowBatchInfo> Context, TResult<bool>& Result, TConstBatch<FTypedElementClassTypeInfoColumn> TypeInfoColumns)
			{
				Context.ForEachRow([&Result, InClass](RowHandle Row, const FTypedElementClassTypeInfoColumn& TypeInfoColumn)
				{
					Result.Add(Row, TypeInfoColumn.TypeInfo->IsChildOf(InClass));
				}, TypeInfoColumns);
			}));
	}

	template<typename FilterType>
	void FTedsFilterBase<FilterType>::SetCategory(const TSharedPtr<FFilterCategory>& InCategory)
	{
		FFilterBase<FilterType>::FilterCategory = InCategory;
	}
	
	template<typename FilterType>
	FString FTedsFilterBase<FilterType>::GetName() const
	{
		return FilterName.ToString();
	}
	
	template<typename FilterType>
	FText FTedsFilterBase<FilterType>::GetDisplayName() const
	{
		return FilterDisplayName;
	}

	template<typename FilterType>
	FText FTedsFilterBase<FilterType>::GetToolTipText() const
	{
		return FilterToolTip;
	}

	template<typename FilterType>
	FLinearColor FTedsFilterBase<FilterType>::GetColor() const
	{
		return FLinearColor();	
	}

	template<typename FilterType>
	FName FTedsFilterBase<FilterType>::GetIconName() const
	{
		return FilterIconName;
	}

	template<typename FilterType>
	bool FTedsFilterBase<FilterType>::IsInverseFilter() const
	{
		return false;
	}

	template<typename FilterType>
	void FTedsFilterBase<FilterType>::ModifyContextMenu(FMenuBuilder& MenuBuilder)
	{}

	template<typename FilterType>
	void FTedsFilterBase<FilterType>::SaveSettings(const FString& IniFilename, const FString& IniSection, const FString& SettingsString) const
	{}

	template<typename FilterType>
	void FTedsFilterBase<FilterType>::LoadSettings(const FString& IniFilename, const FString& IniSection, const FString& SettingsString)
	{}
}// namespace UE::Editor::DataStorage

#undef LOCTEXT_NAMESPACE
