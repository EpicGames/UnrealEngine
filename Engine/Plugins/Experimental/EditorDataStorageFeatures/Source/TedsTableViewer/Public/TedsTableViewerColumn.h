// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataStorage/Handles.h"
#include "Elements/Framework/TypedElementMetaData.h"
#include "Elements/Framework/TypedElementQueryBuilder.h"
#include "Elements/Interfaces/TypedElementDataStorageUiInterface.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"
#include "Widgets/Views/SHeaderRow.h"

struct FTypedElementWidgetConstructor;
class SWidget;

namespace UE::Editor::DataStorage
{
	class ICompatibilityProvider;
	class IUiProvider;
	class ICoreProvider;

	/*
	 * Class representing a column in the UI of the table viewer. Can be constructed using a NameID and a WidgetConstructor to create the actual
	 * widgets for rows (optionally supplying a header widget constructor and widget metadata to use)
	 */
	class FTedsTableViewerColumn
	{
	public:
		// Delegate to check if a row is currently visible in the owning table viewer's UI
		DECLARE_DELEGATE_RetVal_OneParam(bool, FIsRowVisible, const RowHandle);

		TEDSTABLEVIEWER_API FTedsTableViewerColumn(
			const FName& ColumnName, // The unique ID of this column
			const IUiProvider::FWidgetConstructorPtr& InCellWidgetConstructor, // The widget constructor to use for this column
			const TArray<TWeakObjectPtr<const UScriptStruct>>& InMatchedColumns = {}, // An optional list of matched Teds columns 
			const IUiProvider::FWidgetConstructorPtr& InHeaderWidgetConstructor = nullptr, // Optional constructor to use for the header widget
			const FMetaData& MetaData = FMetaData(), // Optional generic metadata to use when constructing widgets
			const TArray<TSharedPtr<FColumnMetaData>>& ColumnMetaData = {});  // Optional metadata specific to the matched columns

		TEDSTABLEVIEWER_API ~FTedsTableViewerColumn();
		
		TEDSTABLEVIEWER_API	TSharedPtr<SWidget> ConstructRowWidget(RowHandle InRowHandle, 
			TFunction<void(ICoreProvider&, const RowHandle&)> WidgetRowSetupDelegate = nullptr) const;
		
		TEDSTABLEVIEWER_API SHeaderRow::FColumn::FArguments ConstructHeaderRowColumn();

		TEDSTABLEVIEWER_API void Tick();
		
		TEDSTABLEVIEWER_API void SetIsRowVisibleDelegate(FIsRowVisible InIsRowVisibleDelegate);


		TEDSTABLEVIEWER_API FName GetColumnName() const;

		TEDSTABLEVIEWER_API TConstArrayView<TWeakObjectPtr<const UScriptStruct>> GetMatchedColumns() const;

		TEDSTABLEVIEWER_API ICoreProvider* GetStorage() const;

		// Delegate triggered when a column is sorted.
		DECLARE_DELEGATE_FourParams(FOnSort, FName ColumnName, TSharedPtr<const FColumnSorterInterface> ColumnSorter,
			EColumnSortMode::Type Direction, EColumnSortPriority::Type Priority);
		// Delegate triggered used to determine is sorting is in progress.
		DECLARE_DELEGATE_RetVal(bool, FIsSorting);
		
		struct FSortHandler
		{
			FOnSort OnSortHandler;
			FIsSorting IsSortingHandler;
		};

		TEDSTABLEVIEWER_API void OnGetSearchStringCallback(const RowHandle& InRowHandle, TArray< FString >& OutSearchStrings);
		TEDSTABLEVIEWER_API void SetInitialSortMode(EColumnSortMode::Type InSortMode);
		TEDSTABLEVIEWER_API void SetSortDelegates(FSortHandler InSortDelegates);
		TEDSTABLEVIEWER_API bool HasSortInfo() const;
		TEDSTABLEVIEWER_API void ClearSort();
		TEDSTABLEVIEWER_API void OnSortCallback(EColumnSortPriority::Type InSortPriority, const FName& InColumnId, EColumnSortMode::Type InSortDirection);
		TEDSTABLEVIEWER_API FComboMetaDataView<FGenericMetaDataView, FColumnsMetaDataView> GetMetaData() const;

	protected:

		void RegisterQueries();
		void UnRegisterQueries();
		bool IsRowVisible(const RowHandle InRowHandle) const;
		void UpdateWidgets();
		
	private:

		EColumnSortMode::Type GetSortMode() const;
		EColumnSortPriority::Type GetSortPriority() const;
		bool IsSorting() const;
		EVisibility GetSortTextVisibility() const;
		FText GetSortText() const;

		// The ID of the column
		FName ColumnName;

		// Widget Constructors
		IUiProvider::FWidgetConstructorPtr CellWidgetConstructor;
		IUiProvider::FWidgetConstructorPtr HeaderWidgetConstructor;

		// Teds Columns this widget constructor matched with
		TArray<TWeakObjectPtr<const UScriptStruct>> MatchedColumns;

		// The matched columns stored as a query condition for quick access
		Queries::FConditions MatchedColumnConditions;
		
		// The Metadata used to create widgets
		const FMetaData WidgetMetaData;
		const TArray<TSharedPtr<FColumnMetaData>> ColumnMetaData;

		// TEDS Constructs
		ICoreProvider* Storage;
		IUiProvider* StorageUi;
		ICompatibilityProvider* StorageCompatibility;

		// Queries used to virtualize widgets when a column is added to/remove from a row
		TArray<QueryHandle> InternalObserverQueries;
		QueryHandle WidgetQuery;
		TMap<RowHandle, bool> RowsToUpdate;

		TArray<TSharedPtr<const FColumnSearcherInterface>> ColumnSearchers;

		// Sorting information
		TArray<TSharedPtr<const FColumnSorterInterface>> ColumnSorters;
		FSortHandler SortDelegates;
		FText SorterShortName;
		uint32 ColumnSorterIndex = 0;
		EColumnSortMode::Type InitialSortMode = EColumnSortMode::Ascending;
		EColumnSortPriority::Type SortPriority = EColumnSortPriority::None;
		bool bIsSorted = false;
		
		// Delegate to check if a row is visible in the owning table viewer
		FIsRowVisible IsRowVisibleDelegate;
	};
} // namespace UE::Editor::DataStorage
