// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Ticker.h"
#include "Delegates/Delegate.h"
#include "DataStorage/Handles.h"
#include "Elements/Interfaces/TypedElementDataStorageUiInterface.h"
#include "TedsQueryStackExecutor.h"
#include "TedsQueryStackInterfaces.h"
#include "Templates/SharedPointer.h"
#include "TypedElementUITypes.h"
#include "UObject/NameTypes.h"
#include "Widgets/Views/SHeaderRow.h"

namespace UE::Editor::DataStorage
{
	namespace QueryStack
	{
		class FRowOrderInversionNode;
		class FRowSortNode;
		class IRowNode;
	}
	class FTedsTableViewerColumn;
	class ICompatibilityProvider;
	class IUiProvider;
	class ICoreProvider;
	
	// Typedef for an item in the table viewer
	using TableViewerItemPtr = FTedsRowHandle;

	// Model class for the TEDS Table Viewer that can be plugged into any widget that is a UI representation of data in TEDS
	// @see STedsTableViewer
	class FTedsTableViewerModel
	{
	public:

		DECLARE_DELEGATE_RetVal_OneParam(bool, FIsItemVisible, TableViewerItemPtr);
		DECLARE_MULTICAST_DELEGATE(FOnModelChanged);

		struct FArguments
		{
			TSharedPtr<QueryStack::IRowNode> RowQueryStack;
			TArray<TWeakObjectPtr<const UScriptStruct>> RequestedColumns;
			TSharedPtr<FMetaData> GenericMetaData;
			TArray<TSharedPtr<FColumnMetaData>> ColumnMetaData;
			IUiProvider::FPurposeID CellWidgetPurpose; 
			IUiProvider::FPurposeID HeaderWidgetPurpose;
			FIsItemVisible IsItemVisibleDelegate;
			TWeakObjectPtr<const UScriptStruct> PrimaryColumn;
			bool bShowHiddenRows = false;
		};

		TEDSTABLEVIEWER_API FTedsTableViewerModel(const FArguments& Args);

		~FTedsTableViewerModel();

		// Get the items this table viewer is viewing
		TEDSTABLEVIEWER_API const TArray<TableViewerItemPtr>& GetItems() const;

		// Get the number of rows currently being observed
		TEDSTABLEVIEWER_API uint64 GetRowCount() const;
		
		// Get the number of columns being displayed
		TEDSTABLEVIEWER_API uint64 GetColumnCount() const;
		
		// Get a specific column that the table viewer is displaying by name
		TEDSTABLEVIEWER_API TSharedPtr<FTedsTableViewerColumn> GetColumn(const FName& ColumnName) const;

		// Get the index of a column by name (INDEX_NONE if column doesn't exist)
		TEDSTABLEVIEWER_API int32 GetColumnIndex(const FName& ColumnName) const;

		// Execute a delegate for each column in the model
		TEDSTABLEVIEWER_API void ForEachColumn(const TFunctionRef<void(const TSharedRef<FTedsTableViewerColumn>&)>& Delegate) const;

		// Delegate when the item list changes
		TEDSTABLEVIEWER_API FOnModelChanged& GetOnModelChanged();

		// Clear the current QueryStack being displayed, set it to the given node, and recreate the sorting nodes 
		TEDSTABLEVIEWER_API void SetQueryStack(const TSharedPtr<QueryStack::IRowNode>& InRowQueryStack, bool bUpdateExecutor = true);
		
		// Clear the current list of columns being displayed and set it to the given list
		TEDSTABLEVIEWER_API void SetColumns(const TArray<TWeakObjectPtr<const UScriptStruct>>& InColumns);

		// Add a custom row widget to display in the table viewer, that doesn't necessarily map to a Teds column
		TEDSTABLEVIEWER_API void AddCustomRowWidget(const TSharedRef<FTedsTableViewerColumn>& InColumn);

		TEDSTABLEVIEWER_API ICoreProvider* GetDataStorageInterface() const;

		TEDSTABLEVIEWER_API IUiProvider* GetDataStorageUiProvider() const;

		// Get the bottom-most row node used in the query stack for this model
		TEDSTABLEVIEWER_API TSharedPtr<QueryStack::IRowNode> GetRowNode() const;

		// The FName of the primary UI column (matches the column name in the underlying slate widgets)
		TEDSTABLEVIEWER_API FName GetPrimaryColumnName() const;
		
	protected:

		// Generate the actual columns to display in the UI using TEDS UI
		void GenerateColumns();

		// Check if the given row is currently visible in the UI
		bool IsRowVisible(RowHandle InRowHandle) const;

		void OnSort(FName ColumnName, TSharedPtr<const FColumnSorterInterface> ColumnSorter, EColumnSortMode::Type Direction, 
			EColumnSortPriority::Type Priority);
		bool IsSorting() const;
		
		bool Tick(float DeltaTime);

		void Refresh();

		void ValidateRequestedColumns();

	private:

		// The row query stack used to supply the rows to display
		TSharedPtr<QueryStack::IRowNode> RowQueryStack;

		// The executor used to update the RowQueryStack.
		QueryStack::FExplicitUpdateExecutor QueryStackExecutor;

		// Row node used to sort the rows in the query stack.
		TSharedPtr<QueryStack::FRowSortNode> RowPrimarySortingNode;
		// Row node used as a backup to sort the rows in the query stack. Users can hold down shift to setup a secondary sort. As a result
		// any duplicates from the the primary sorting node will be sorted by the secondary sort.
		TSharedPtr<QueryStack::FRowSortNode> RowSecondarySortingNode;

		// Inverts the rows from the query stack when needed by the primary sort.
		TSharedPtr<QueryStack::FRowOrderInversionNode> RowPrimaryInversionNode;
		// Inverts the rows from the query stack when needed by the primary sort.
		TSharedPtr<QueryStack::FRowOrderInversionNode> RowSecondaryInversionNode;

		// The name of the active primary sort column or none if not set.
		FName PrimarySortColumnName;
		// The name of the active secondary sort column or none if not set.
		FName SecondarySortColumnName;
		
		// The cached list of rows we are currently displaying
		TArray<TableViewerItemPtr> Items;

		// List of columns the table viewer is currently displaying
		TArray<TSharedRef<FTedsTableViewerColumn>> ColumnsView;

		// The initial TEDS columns the widget was requested to display
		TArray<TWeakObjectPtr<const UScriptStruct>> RequestedTedsColumns;

		// Generic Metadata for all widgets
		TSharedPtr<FMetaData> GenericMetaData;

		// Metadata for RequestedTedsColumns
		TArray<TSharedPtr<FColumnMetaData>> ColumnMetaData;

		// The widget purposes used to create widgets in this table viewer
		IUiProvider::FPurposeID CellWidgetPurpose;
		IUiProvider::FPurposeID HeaderWidgetPurpose;

		// Cached revision ID for the query stack used to check when the table viewer needs a refresh
		uint32 CachedRowQueryStackRevision = QueryStack::INode::UninitializedRevisionId;

		// Delegate supplied by the widget to check if an item is visible in the UI currently
		FIsItemVisible IsItemVisible;

		FTSTicker::FDelegateHandle TickerHandle;

		// Delegate executed when the row list changes
		FOnModelChanged OnModelChanged;
		
		// Teds Constructs
		ICoreProvider* Storage = nullptr;
		IUiProvider* StorageUi = nullptr;

		// The TEDS column that was requested as primary for this viewer
		TWeakObjectPtr<const UScriptStruct> PrimaryColumn;

		// If true, rows tagged with FHideRowFromUITag will still be displayed
		bool bShowHiddenRows = false;

		// TEDS -> UI columns is not a 1:1 mapping, so we cache the name of the UI column so widgets can query it easily
		FName PrimaryUIColumnName;

		// Store the Instance count and the created name for use by the executor (so it can avoid recreation)
		inline static int InstanceCounter = 0;
		FName ExecutorInstanceName;
	};
} // namespace UE::Editor::DataStorage
