// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataStorage/Handles.h"
#include "Containers/ArrayView.h"
#include "Elements/Interfaces/TypedElementDataStorageFactory.h"
#include "TedsTableViewerModel.h"
#include "UObject/Class.h"

#include "TedsTableViewerUtils.generated.h"

namespace UE::Editor::DataStorage
{
	class FMetaDataView;
	class ICoreProvider;
	class IUiProvider;
}

struct FTypedElementWidgetConstructor;
struct FSlateBrush;

// Util library for functions shared by the Teds Table Viewer and the Teds Outliner
namespace UE::Editor::DataStorage::TableViewerUtils
{
	// Delegate fired to get a custom row mapping for the table viewer
	DECLARE_DELEGATE_ThreeParams(FGetTableViewerRowMapping, const FName&, const RowHandle, FMapKey&);
	
	TEDSTABLEVIEWER_API FName GetWidgetTableName();

	// Get the name of the widget hierarchy for table viewer widgets
	// This hierarchy currently contains the relationship between the row containing the table viewer itself and individual cell widgets in that table viewer
	TEDSTABLEVIEWER_API FName GetWidgetHierarchyName();

	TEDSTABLEVIEWER_API FName GetRowWidgetTableName();
	
	// Find the longest matching common column name given a list of columns
	TEDSTABLEVIEWER_API FName FindLongestMatchingName(TConstArrayView<TWeakObjectPtr<const UScriptStruct>> ColumnTypes, int32 DefaultNameIndex);

	// Create a header widget constructor for the given columns
	TEDSTABLEVIEWER_API IUiProvider::FWidgetConstructorPtr CreateHeaderWidgetConstructor(IUiProvider& StorageUi, 
		const FMetaDataView& InMetaData, TConstArrayView<TWeakObjectPtr<const UScriptStruct>> ColumnTypes, RowHandle PurposeRow);

	// Create a copy of the provided column types array after discarding invalid entries
	TEDSTABLEVIEWER_API TArray<TWeakObjectPtr<const UScriptStruct>> CreateVerifiedColumnTypeArray(TConstArrayView<TWeakObjectPtr<const UScriptStruct>> ColumnTypes);
	
	// Scans the row for columns and tries to find the best matching icon.
	TEDSTABLEVIEWER_API const FSlateBrush* GetIconForRow(ICoreProvider* DataStorage, RowHandle Row);

	// Creates the default Row Map Key for the Table Row UI Rows
	TEDSTABLEVIEWER_API void CreateRowMappingKey(const FName& InHierarchyIdentifier, const TableViewerItemPtr InItem, FMapKey& OutMapKey);
	
	// Creates the default Row Map Key for the Table Row UI Rows from the RowHandle directly
	TEDSTABLEVIEWER_API void CreateRowMappingKey(const FName& InHierarchyIdentifier, const RowHandle InItem, FMapKey& OutMapKey);

	// Walks up the UI Widget Hierarchy until finding the row with the FTableViewerRowTag
	TEDSTABLEVIEWER_API RowHandle GetTableViewerRowUiRow(const ICoreProvider* DataStorage, const RowHandle ChildWidgetRow);

	// Walks up the UI Widget Hierarchy until finding the row with the FTableViewerTag
	TEDSTABLEVIEWER_API RowHandle GetTableViewerUiRow(const ICoreProvider* DataStorage, const RowHandle ChildWidgetRow);

	// Finds all the UI Rows for the provided TableViewer and removes the rows
	TEDSTABLEVIEWER_API void RemoveAllTableViewerUIRows(ICoreProvider* DataStorage, const RowHandle TableViewerRowHandle);

	// Finds all the UI Rows for the provided TableViewer and deactivates them (adds the FTedsTableViewerInactiveRowTag)
	TEDSTABLEVIEWER_API void DeactivateAllTableViewerUIRows(ICoreProvider* DataStorage, const RowHandle TableViewerRowHandle);

	// Initializes all rows on a TableViewer for a QueryStack
	TEDSTABLEVIEWER_API void InitializeAllTableViewerUIRows(ICoreProvider* DataStorage, const TArray<TableViewerItemPtr>& Items,
		const FGetTableViewerRowMapping& GetTableViewerRowMapping, const FName& TableViewerIdentifier, 
		const RowHandle TableViewerRowHandle, const bool bAddHierarchyTags = false);
} // namespace UE::Editor::DataStorage::TableViewerUtils

UCLASS()
class UTedsTableViewerFactory : public UEditorDataStorageFactory
{
	GENERATED_BODY()

public:
	~UTedsTableViewerFactory() override = default;

	void RegisterQueries(UE::Editor::DataStorage::ICoreProvider& DataStorage) override;
	void RegisterTables(UE::Editor::DataStorage::ICoreProvider& DataStorage) override;
	void RegisterHierarchies(UE::Editor::DataStorage::ICoreProvider& DataStorage) override;
};

