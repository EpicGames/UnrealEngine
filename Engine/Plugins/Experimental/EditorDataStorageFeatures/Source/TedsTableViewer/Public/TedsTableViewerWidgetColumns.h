// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HierarchyViewerIntefaces.h"
#include "DataStorage/CommonTypes.h"
#include "Framework/SlateDelegates.h"
#include "ITedsTableViewer.h"
#include "Templates/SharedPointer.h"

#include "TedsTableViewerWidgetColumns.generated.h"

namespace UE::Editor::DataStorage
{
	inline static const FName TableViewerUIRowMappingDomain = TEXT("TableViewerUIRow");
}

/**
* Column added to a widget row when an external widget manages exclusive selection for the widget referenced by the row,
 * such as an owning SListView or STreeView
 */
USTRUCT(meta = (DisplayName = "Widget with externally managed exclusive selection"))
struct FExternalWidgetExclusiveSelectionColumn : public FEditorDataStorageColumn
{
	GENERATED_BODY()
	/** Delegate to execute to check the status of if the widget is the only selected between multiple widgets
	 * Only needs to be hooked up if an external widget is managing selection on multiple widgets, such
	 * as a SListView or STreeView.
	 *
	 * E.g usage: Enables renaming by clicking the label twice if the widget is editable.
	 * @see SInlineEditableTextBlock::IsSelected
	 */
	TSharedPtr<FIsSelected> IsSelectedExclusively;
};

/**
 * Tag on a UI row representing a table viewer (or hierarchy viewer) 
 */
USTRUCT(meta = (DisplayName = "Table Viewer Widget Tag"))
struct FTedsTableViewerTag : public FEditorDataStorageTag
{
	GENERATED_BODY()
};

/**
 * Column on table viewer row that contains a reference to the ITableViewer
 */
USTRUCT(meta = (DisplayName = "Table Viewer reference"))
struct FTedsTableViewerReferenceColumn : public FEditorDataStorageColumn
{
	GENERATED_BODY()
	
	TWeakPtr<UE::Editor::DataStorage::ITableViewer> TableViewerWeak;
};

/**
 * Column on the table viewer row with the delegate that executes when selection is changed
 */
USTRUCT(meta = (DisplayName = "On Table Viewer selection changed"))
struct FTedsTableViewerSelectionChangedColumn : public FEditorDataStorageColumn
{
	GENERATED_BODY()
	
	TSharedPtr<FSimpleMulticastDelegate> OnSelectionChanged;
};

/**
 * Column on the table viewer row with the delegate that executes to rename the selected rows
 * TEDS UI TODO: Can we use a generic column for actions/events instead of one hardcoded for rename?
 */
USTRUCT(meta = (DisplayName = "Table Viewer rename event"))
struct FTedsTableViewerRenameSelectionColumn : public FEditorDataStorageColumn
{
	GENERATED_BODY()
	
	TSharedPtr<FSimpleMulticastDelegate> OnRenameSelection;
};

/**
 * Column on the table viewer row that contains information about an ongoing drag and drop event
 */
USTRUCT(meta = (DisplayName = "Table Viewer drop info"))
struct FTedsTableViewerDropInfoColumn : public FEditorDataStorageColumn
{
	GENERATED_BODY()

	UE::Editor::DataStorage::RowHandle DropTarget;
};

/**
 * Column on widget rows that contains a weak ptr to the hierarchy data used by the hierarchy viewer to display rows (if present).
 * This is useful if your widget is meant to be displayed in a hierarchy view and needs access to data on the visual parent of the data row
 * or do a ForEachChild.
 * NOTE: This contains hierarchy data about the data/target row and not the widget row. Widget hierarchies can be accessed directly by using
 * TableViewerUtils::GetWidgetHierarchyName() to get the widget row of the parent widget.
 */
USTRUCT(meta = (DisplayName = "Hierarchy viewer access interface"))
struct FTedsHierarchyViewerDataColumn : public FEditorDataStorageColumn
{
	GENERATED_BODY()
	
	TWeakPtr<UE::Editor::DataStorage::IHierarchyViewerDataInterface> HierarchyData;
};

/**
 * Column on the hierarchy viewer widget row that contains the root of the tree. If there are multiple top level items, the
 * root is the first one (taking sorting into account)
 */
USTRUCT(meta = (DisplayName = "Hierarchy Viewer root item"))
struct FTedsHierarchyViewerRootColumn : public FEditorDataStorageColumn
{
	GENERATED_BODY()

	UE::Editor::DataStorage::RowHandle Root;
};

/**
 * Tag on a UI row representing a row in a table viewer (or hierarchy viewer) 
 */
USTRUCT(meta = (DisplayName = "Table Viewer Row Widget Tag"))
struct FTedsTableViewerRowTag : public FEditorDataStorageTag
{
	GENERATED_BODY()
};


/**
 * Tag on to indicate that this UI row is currently kept around but is not currently being used as a widget
 */
USTRUCT(meta = (DisplayName = "Inactive UI"))
struct FTedsTableViewerInactiveRowTag : public FEditorDataStorageTag
{
	GENERATED_BODY()
};

namespace UE::Editor::DataStorage
{
	using FTableViewerTag = FTedsTableViewerTag;
	using FTableViewerReferenceColumn = FTedsTableViewerReferenceColumn;
	using FTableViewerDropInfoColumn = FTedsTableViewerDropInfoColumn;
	using FTableViewerSelectionChangedColumn = FTedsTableViewerSelectionChangedColumn;
	using FTableViewerRenameSelectionColumn = FTedsTableViewerRenameSelectionColumn;
	using FHierarchyViewerDataColumn = FTedsHierarchyViewerDataColumn;
	using FHierarchyViewerRootColumn = FTedsHierarchyViewerRootColumn;
	using FTableViewerRowTag = FTedsTableViewerRowTag;
}