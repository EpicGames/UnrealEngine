// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataStorage/Handles.h"
#include "Framework/Views/TableViewTypeTraits.h"



/* Template declaration to describe how a row handle behaves as a type for slate widgets like SListView, STreeView etc
 * This allows you to use Row Handles with slate widgets that work on pointers by using the wrapper struct e.g
 * SListView<FTedsRowHandle>
 */
template <>
struct TListTypeTraits<FTedsRowHandle>
{
	typedef FTedsRowHandle NullableType;

	using MapKeyFuncs = TDefaultMapHashableKeyFuncs<FTedsRowHandle, TSharedRef<ITableRow>, false>;
	using MapKeyFuncsSparse = TDefaultMapHashableKeyFuncs<FTedsRowHandle, FSparseItemInfo, false>;
	using SetKeyFuncs = DefaultKeyFuncs<FTedsRowHandle>;

	template<typename U>
	static void AddReferencedObjects(FReferenceCollector&,
		TArray<FTedsRowHandle>&,
		TSet<FTedsRowHandle>&,
		TMap<const U*, FTedsRowHandle>&)
	{
	}

	static bool IsPtrValid(const FTedsRowHandle& InPtr)
	{
		return InPtr != UE::Editor::DataStorage::InvalidRowHandle;
	}

	static void ResetPtr(FTedsRowHandle& InPtr)
	{
		InPtr = UE::Editor::DataStorage::InvalidRowHandle;
	}

	static FTedsRowHandle MakeNullPtr()
	{
		static FTedsRowHandle InvalidRowHandle{ .RowHandle = UE::Editor::DataStorage::InvalidRowHandle };
		return InvalidRowHandle;
	}

	static FTedsRowHandle NullableItemTypeConvertToItemType(const FTedsRowHandle& InPtr)
	{
		return InPtr;
	}

	static FString DebugDump(FTedsRowHandle InPtr)
	{
		return FString::Printf(TEXT("%llu"), InPtr.RowHandle);
	}

	class SerializerType {};
};

// Template declaration to enable using row handles inside of slate widgets like SListView
template <>
struct TIsValidListItem<FTedsRowHandle>
{
	enum
	{
		Value = true
	};
};

/* Template declaration to describe how a table handle behaves as a type for slate widgets like SListView, STreeView etc
 * This allows you to use Table Handles with slate widgets that would otherwise only work on pointers by using the wrapper 
 * struct e.g STreeView<FTedsTableHandle>
 */
template <>
struct TListTypeTraits<FTedsTableHandle>
{
	typedef FTedsTableHandle NullableType;

	using MapKeyFuncs = TDefaultMapHashableKeyFuncs<FTedsTableHandle, TSharedRef<ITableRow>, false>;
	using MapKeyFuncsSparse = TDefaultMapHashableKeyFuncs<FTedsTableHandle, FSparseItemInfo, false>;
	using SetKeyFuncs = DefaultKeyFuncs<FTedsTableHandle>;

	template<typename U>
	static void AddReferencedObjects(FReferenceCollector&,
		TArray<FTedsTableHandle>&,
		TSet<FTedsTableHandle>&,
		TMap<const U*, FTedsTableHandle>&)
	{
	}

	static bool IsPtrValid(const FTedsTableHandle& InPtr)
	{
		return InPtr != UE::Editor::DataStorage::InvalidTableHandle;
	}

	static void ResetPtr(FTedsTableHandle& InPtr)
	{
		InPtr = UE::Editor::DataStorage::InvalidTableHandle;
	}

	static FTedsTableHandle MakeNullPtr()
	{
		static FTedsTableHandle InvalidTableHandle{ .TableHandle = UE::Editor::DataStorage::InvalidTableHandle };
		return InvalidTableHandle;
	}

	static FTedsTableHandle NullableItemTypeConvertToItemType(const FTedsTableHandle& InPtr)
	{
		return InPtr;
	}

	static FString DebugDump(FTedsTableHandle InPtr)
	{
		return FString::Printf(TEXT("%llu"), InPtr.TableHandle);
	}

	class SerializerType {};
};

// Template declaration to enable using table handles inside of slate widgets like STreeView
template <>
struct TIsValidListItem<FTedsTableHandle>
{
	enum
	{
		Value = true
	};
};