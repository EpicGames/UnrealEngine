// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "DataStorage/Handles.h"
#include "TedsFilter.h"
#include "Widgets/STedsHierarchyViewer.h"

namespace UE::Editor::DataStorage
{
	class FTedsTableViewerModel;
	class IRowNode;
	class STedsFilterBar;
	class STedsSearchBox;
}
class SComboButton;

namespace UE::Editor::DataStorage
{
	class FTedsTableViewerColumn;

	/**
	* A class corresponding to one boolean option (e.g. "Show Components"). An array of these objects can be passed into the 
	* STedsCompositeHierarchyViewer and it will auto-populate a menu containing toggle buttons for each object.
	*/
	class FTedsCompositeHierarchyViewerOption : public TSharedFromThis<FTedsCompositeHierarchyViewerOption>
	{
	public:
		enum class EOptionCategories
		{
			Hierarchy,
			Show
		};

		TEDSTABLEVIEWER_API FTedsCompositeHierarchyViewerOption(const FText& InSettingTitle, const FText& InSettingTitleToolTip, EOptionCategories InOptionCategory, bool bInActive);

		TEDSTABLEVIEWER_API bool IsActive() const;

		TEDSTABLEVIEWER_API void ToggleIsActive();

		TEDSTABLEVIEWER_API void AddMenu(FMenuBuilder& InMenuBuilder);

		TEDSTABLEVIEWER_API EOptionCategories GetCategory() const;

		DECLARE_EVENT_OneParam(FTedsCompositeHierarchyViewerOption, FOnToggle, bool);
		FOnToggle& OnToggle() { return OnToggleEvent; }

	private:

		FText OptionTitle;
		FText OptionToolTip;
		EOptionCategories OptionCategory;
		bool bActive;

		FOnToggle OnToggleEvent;
	};


	/*
	 * A hierarchy viewer widget can be used to show a visual representation of data in TEDS. This composite widget adds features in a 'default' layout
	 * such as searching and filtering. The rows to display can be specified using a RowQueryStack, and the columns to display are directly input 
	 * into the widget
	 * Example usage:
	 *
	 *	SNew(STedsCompositeHierarchyViewer, HierarchyData) // Filtering and Searching enabled by default
     *		.HierarchyViewerArgs(SHierarchyViewer::FArguments()
     *			.AllNodeProvider(FilterNode)
     *			.Columns({ FTypedElementLabelColumn::StaticStruct(), FTypedElementClassTypeInfoColumn::StaticStruct() })
     *			.CellWidgetPurpose(PurposeId)
	 *		.Filters(MyCustomFilterArray);
	 */
	class STedsCompositeHierarchyViewer :  public SCompoundWidget, public ITableViewer
	{
	public:

		SLATE_BEGIN_ARGS(STedsCompositeHierarchyViewer)
			: _HierarchyViewerArgs()
			, _EnableSearching(true)
			, _EnableFiltering(true)
			, _EnableSettings(true)
			, _UseSectionsForCategories(false)
			, _ShowFilteredParentHierarchy(true)
		{
		}

		// Arguments used to create the Hierarchy Viewer
		SLATE_ARGUMENT(SHierarchyViewer::FArguments, HierarchyViewerArgs)
			
		// If we want to enable the search bar on the table
		SLATE_ARGUMENT(bool, EnableSearching)

		// If we want to enable the filter bar on the table
		SLATE_ARGUMENT(bool, EnableFiltering)

		// If we want to enable the settings cog on the table (auto-populates with a few default hierarchy actions)
		SLATE_ARGUMENT(bool, EnableSettings)
		
		/** Array of filter names to also add to the 'common filters' section (section at the top of the menu) */
		SLATE_ARGUMENT(TArray<FName>, CommonSectionFilters)

		// Array of filters to add to the filter bar
		SLATE_ARGUMENT(TArray<TSharedPtr<FTedsFilter>>, Filters)

		// Whether to use submenus or sections for categories in the filter menu
		SLATE_ARGUMENT(bool, UseSectionsForCategories)
		
		// Array of boolean option objects used to populate the settings menu
		SLATE_ARGUMENT(TArray<TSharedPtr<FTedsCompositeHierarchyViewerOption>>, Options)

		// Whether to show filtered out parents of a child that matches
		SLATE_ARGUMENT(bool, ShowFilteredParentHierarchy)

		SLATE_END_ARGS()
		
	protected:
		// Clear the current QueryStack being displayed, set it to the given node, and recreate the sorting nodes 
		TEDSTABLEVIEWER_API virtual void SetQueryStack(TSharedPtr<QueryStack::IRowNode> InRowQueryStack) override;

	public:
		
		TEDSTABLEVIEWER_API void Construct(const FArguments& InArgs, TSharedPtr<IHierarchyViewerDataInterface> InHierarchyInterface);

		// Clear the current list of columns being displayed and set it to the given list
		TEDSTABLEVIEWER_API virtual void SetColumns(const TArray<TWeakObjectPtr<const UScriptStruct>>& InColumns) override;
		
		// Sets the TableViewer/Hierarchy Identifier (Can be used in case data is dynamic and created post widget init)
		TEDSTABLEVIEWER_API void SetTableViewerIdentifier(const FName& NewIdentifier);

		// Add a custom per-row widget to the table viewer (that doesn't necessarily map to a TEDS column)
		// This means a new column for the table viewer
		TEDSTABLEVIEWER_API virtual void AddCustomRowWidget(const TSharedRef<FTedsTableViewerColumn>& InColumn) override;

		// Execute the given callback for each row that is selected in the table viewer
		TEDSTABLEVIEWER_API virtual void ForEachSelectedRow(TFunctionRef<void(RowHandle)> InCallback) const override;

		// Get the row handle for the widget row the table viewer's contents are stored in
		TEDSTABLEVIEWER_API virtual RowHandle GetWidgetRowHandle() const override;
		
		// Select the given row in the table viewer
		TEDSTABLEVIEWER_API virtual void SetSelection(RowHandle Row, bool bSelected, const ESelectInfo::Type SelectInfo) const override;

		// Scroll the given row into view in the table viewer
		TEDSTABLEVIEWER_API virtual void ScrollIntoView(RowHandle Row) const override;

		TEDSTABLEVIEWER_API virtual void ClearSelection() const override;
		
		TEDSTABLEVIEWER_API virtual TSharedRef<SWidget> AsWidget() override;

		TEDSTABLEVIEWER_API virtual bool IsSelected(RowHandle InRow) const override;

		TEDSTABLEVIEWER_API virtual bool IsSelectedExclusively(RowHandle InRow) const override;

		TEDSTABLEVIEWER_API void AddFilter(const TSharedPtr<FTedsFilter> InFilter) const;

		TEDSTABLEVIEWER_API void AddFilters(const TArray<TSharedPtr<FTedsFilter>>& InFilters) const;

		TEDSTABLEVIEWER_API void DeleteFilter(const FName& InFilterName) const;

		TEDSTABLEVIEWER_API void DeleteAllFilters() const;
		
		TEDSTABLEVIEWER_API void SetCommonFilters(const TArray<FName>& InFilters) const;

		TEDSTABLEVIEWER_API void ReplaceFilters(const TArray<TSharedPtr<FTedsFilter>>& InFilters, const bool bPersistFilterStates = true) const;
		
		// Recursively Expand all rows currently present in the viewer
		TEDSTABLEVIEWER_API void ExpandAll() const;

		// Recursively Collapse all rows currently present in the viewer
		TEDSTABLEVIEWER_API void CollapseAll() const;

		// Expand this row, and all its parents
		TEDSTABLEVIEWER_API void ExpandWithParents(RowHandle Row);

		// Menu actions
		TEDSTABLEVIEWER_API void OnToggleShowFilteredParentHierarchy();
		TEDSTABLEVIEWER_API void OnToggleExpandNewRows();
		
		// Set a handler that this viewer will forward its drop operations to.
		TEDSTABLEVIEWER_API void SetDropHandler(TUniquePtr<FWidgetDropHandler> InDropHandler);
		
	private:
		void AddFilteredParentRowsToQueryStack(TSharedPtr<QueryStack::IRowNode>* QueryStack, const TSharedPtr<IHierarchyViewerDataInterface>& HierarchyInterface);

		// Build a menu of Toggle Buttons from the ShowOptions objects
		TSharedRef<SWidget> GetSettingsButtonContent();
		
		// Option objects used to construct the settings menu
		TArray<TSharedPtr<FTedsCompositeHierarchyViewerOption>> Options;
		
		// Whether to add back the filtered out parent hierarchy (removed from searching or filtering)
		bool bShowFilteredOutParentHierarchy = true;

		// Original Row Node given as the AllNodeProvider of the HierarchyViewer
		TSharedPtr<QueryStack::IRowNode> QueryNode;

		// Final Composite Row Node given to the Hierarchy Viewer
		TSharedPtr<QueryStack::IRowNode> CompositeQueryNode;

		// Row Node passed to the SearchBox that receives the searched result
		TSharedPtr<QueryStack::IRowNode> SearchNode;

		// Row Node passed to the FilterBar that receives the filtered result
		TSharedPtr<QueryStack::IRowNode> FilterNode;
		
		// The search box widget
		TSharedPtr<STedsSearchBox> SearchBox;

		// The filter bar widget (uses MakeAddFilterButton to create the menu dropdown)
		TSharedPtr<STedsFilterBar> FilterBar;

		// Button used to bring up the settings menu
		TSharedPtr<SComboButton> SettingsComboButton;

		// The actual HierarchyViewer widget that displays the rows
		TSharedPtr<SHierarchyViewer> HierarchyViewer;
	};
}