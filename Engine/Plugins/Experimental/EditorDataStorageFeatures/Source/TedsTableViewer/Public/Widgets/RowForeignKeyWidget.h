// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataStorage/Queries/Conditions.h"
#include "Elements/Interfaces/TypedElementDataStorageFactory.h"
#include "Elements/Interfaces/TypedElementDataStorageUiInterface.h"

#include "RowForeignKeyWidget.generated.h"

#define UE_API TEDSTABLEVIEWER_API

namespace UE::Editor::DataStorage
{
	class ICoreProvider;
} // namespace UE::Editor::DataStorage

class UScriptStruct;

namespace UE::Editor::DataStorage::Ui
{

} // namespace UE::Editor::DataStorage::Ui

UCLASS(MinimalAPI)
class URowForeignKeyWidgetFactory : public UEditorDataStorageFactory
{
	GENERATED_BODY()

public:
	virtual ~URowForeignKeyWidgetFactory() override = default;

	UE_API virtual void RegisterWidgetConstructors(UE::Editor::DataStorage::ICoreProvider& DataStorage,
		UE::Editor::DataStorage::IUiProvider& DataStorageUi) const override;

	UE_API void RegisterWidgetPurposes(UE::Editor::DataStorage::IUiProvider& DataStorageUi) const override;
};

// A custom widget to display the row ForeignKey of a row as text
USTRUCT()
struct FRowForeignKeyWidgetConstructor : public FTypedElementWidgetConstructor
{
	GENERATED_BODY()

public:
	UE_API FRowForeignKeyWidgetConstructor();
	virtual ~FRowForeignKeyWidgetConstructor() override = default;

protected:
	UE_API virtual TSharedPtr<SWidget> CreateWidget(const UE::Editor::DataStorage::FMetaDataView& Arguments) override;
	UE_API virtual bool FinalizeWidget(
		UE::Editor::DataStorage::ICoreProvider* DataStorage,
		UE::Editor::DataStorage::IUiProvider* DataStorageUi,
		UE::Editor::DataStorage::RowHandle Row,
		const TSharedPtr<SWidget>& Widget) override;
		
	UE_API virtual FText CreateWidgetDisplayNameText(UE::Editor::DataStorage::ICoreProvider* DataStorage, 
    		UE::Editor::DataStorage::RowHandle Row = UE::Editor::DataStorage::InvalidRowHandle) const override;
};

#undef UE_API
