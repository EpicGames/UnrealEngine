// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataStorage/Queries/Conditions.h"
#include "Elements/Interfaces/TypedElementDataStorageFactory.h"
#include "Elements/Interfaces/TypedElementDataStorageUiInterface.h"

#include "RowHandleWidget.generated.h"

#define UE_API TEDSTABLEVIEWER_API

namespace UE::Editor::DataStorage
{
	class ICoreProvider;
} // namespace UE::Editor::DataStorage

class UScriptStruct;

namespace UE::Editor::DataStorage
{
	class FRowHandleSorter final : public FColumnSorterInterface
	{
	public:
		virtual int32 Compare(const ICoreProvider& Storage, RowHandle Left, RowHandle Right) const override;
		virtual FPrefixInfo CalculatePrefix(const ICoreProvider& Storage, RowHandle Row, uint32 ByteIndex) const override;
		virtual ESortType GetSortType() const override;
		virtual FText GetShortName() const override;
	};
}

UCLASS(MinimalAPI)
class URowHandleWidgetFactory : public UEditorDataStorageFactory
{
	GENERATED_BODY()

public:
	virtual ~URowHandleWidgetFactory() override = default;

	UE_API virtual void RegisterWidgetConstructors(UE::Editor::DataStorage::ICoreProvider& DataStorage,
		UE::Editor::DataStorage::IUiProvider& DataStorageUi) const override;

	UE_API void RegisterWidgetPurposes(UE::Editor::DataStorage::IUiProvider& DataStorageUi) const override;
};

// A custom widget to display the row handle of a row as text
USTRUCT()
struct FRowHandleWidgetConstructor : public FTypedElementWidgetConstructor
{
	GENERATED_BODY()

public:
	UE_API FRowHandleWidgetConstructor();
	virtual ~FRowHandleWidgetConstructor() override = default;

protected:
	UE_API virtual TSharedPtr<SWidget> CreateWidget(const UE::Editor::DataStorage::FMetaDataView& Arguments) override;
	UE_API virtual bool FinalizeWidget(
		UE::Editor::DataStorage::ICoreProvider* DataStorage,
		UE::Editor::DataStorage::IUiProvider* DataStorageUi,
		UE::Editor::DataStorage::RowHandle Row,
		const TSharedPtr<SWidget>& Widget) override;
		
	UE_API virtual FText CreateWidgetDisplayNameText(UE::Editor::DataStorage::ICoreProvider* DataStorage, 
    		UE::Editor::DataStorage::RowHandle Row = UE::Editor::DataStorage::InvalidRowHandle) const override;

	UE_API virtual TArray<TSharedPtr<const UE::Editor::DataStorage::FColumnSorterInterface>> ConstructColumnSorters(
		UE::Editor::DataStorage::ICoreProvider* DataStorage,
		UE::Editor::DataStorage::IUiProvider* DataStorageUi,
		const UE::Editor::DataStorage::FMetaDataView& Arguments) override;
};

#undef UE_API
