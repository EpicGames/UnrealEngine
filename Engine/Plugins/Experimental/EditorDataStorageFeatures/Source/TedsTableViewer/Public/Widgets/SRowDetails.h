// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "DataStorage/Handles.h"
#include "Elements/Interfaces/TypedElementDataStorageUiInterface.h"
#include "Templates/SharedPointer.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/SListView.h"

#define UE_API TEDSTABLEVIEWER_API

struct FTypedElementWidgetConstructor;

namespace UE::Editor::DataStorage
{
	class ICoreProvider;
	class IUiProvider;

	DECLARE_DELEGATE_OneParam(FOnRelatedRowSelected, RowHandle);

	// A row in the SRowDetails widget that represents either a column or a relation on the TEDS row we are viewing
	struct FRowDetailsItem
	{
		// The logical name of this item (composited from the matched columns or grabbed from the widget constructor)
		FName Name;

		// The columns this row is displaying data for (empty for relation items)
		TArray<TWeakObjectPtr<const UScriptStruct>> ColumnTypes;

		// Widget for the column (null for relation items)
		IUiProvider::FWidgetConstructorPtr WidgetConstructor;

		RowHandle Row = InvalidRowHandle;
		RowHandle WidgetRow = InvalidRowHandle;

		// For relation items: the related row to navigate to (InvalidRowHandle for column items)
		RowHandle RelatedRow = InvalidRowHandle;

		bool IsRelationItem() const { return RelatedRow != InvalidRowHandle; }

		// Constructor for column items
		FRowDetailsItem(const FName& InItemName, const TArray<TWeakObjectPtr<const UScriptStruct>>& InColumnTypes, IUiProvider::FWidgetConstructorPtr InWidgetConstructor,
			RowHandle InRow);

		// Constructor for relation items
		FRowDetailsItem(const FName& InRelationName, RowHandle InRow, RowHandle InRelatedRow);
	};
	
	using RowDetailsItemPtr = TSharedPtr<FRowDetailsItem>;

	// A widget to display all the columns/tags on a given row
	class SRowDetails : public SCompoundWidget
	{
	public:
		UE_API static const FName NameColumn;
		UE_API static const FName DataColumn;

		~SRowDetails() override = default;
		
		SLATE_BEGIN_ARGS(SRowDetails)
			: _ShowUnpopulatedColumns(false)
			, _ShowAllDetails(true)
			, _SelectionMode(ESelectionMode::Multi)
		{}

			// List of Columns to show in order. Leave blank to automatically find all matching columns.
			SLATE_ARGUMENT(TArray<TWeakObjectPtr<const UScriptStruct>>, Columns)
			// If true, requested columns with no data on this row are shown as empty entries. Has no effect when Columns is empty.
			SLATE_ARGUMENT(bool, ShowUnpopulatedColumns)

			// Whether or not to show columns that don't have a dedicated widget to represent them
			SLATE_ARGUMENT(bool, ShowAllDetails)

			// Override for the default widget purposes used to create widgets for the columns
			SLATE_ARGUMENT(IUiProvider::FPurposeID, WidgetPurposeOverride)

			SLATE_ATTRIBUTE(ESelectionMode::Type, SelectionMode)
			// Custom header row for this SRowDetails. Leave null to use a default Name/Value header.
			// If provided, it must contain columns named SRowDetails::NameColumn and SRowDetails::DataColumn.
			SLATE_ARGUMENT(TSharedPtr<SHeaderRow>, HeaderRow)

			// MetaData for the Name Widget shared by all columns
			SLATE_ARGUMENT(TSharedPtr<FMetaData>, NameMetaData)
			// MetaData for the Data Widget shared by all columns
			SLATE_ARGUMENT(TSharedPtr<FMetaData>, DataMetaData)
			// MetaData for all columns shared by the Name and the Data widgets
			SLATE_ARGUMENT(TArray<TSharedPtr<FColumnMetaData>>, ColumnMetaData)
			// The padding inside each Row
			SLATE_ATTRIBUTE(FMargin, RowPadding)

			// Fired when the user clicks a related row in the relation section
			SLATE_EVENT(FOnRelatedRowSelected, OnRelatedRowSelected)
		SLATE_END_ARGS()

		UE_API void Construct(const FArguments& InArgs);

		// Set the row to view
		UE_API void SetRow(RowHandle Row);

		// Clear the row to view
		UE_API void ClearRow();

		// Called by SRowDetailsRow to fire the OnRelatedRowSelected delegate
		UE_API void NotifyRelatedRowSelected(RowHandle Row);

		ICoreProvider* GetDataStorage() const { return DataStorage; }
		IUiProvider* GetDataStorageUi() const { return DataStorageUi; }
		const TSharedPtr<FMetaData>& GetNameMetaData() const { return NameMetaData; }
		const TSharedPtr<FMetaData>& GetDataMetaData() const { return DataMetaData; }
		const TArray<TSharedPtr<FColumnMetaData>>& GetColumnMetaData() const { return ColumnMetaData; }
	private:
		
		UE_API TSharedRef<ITableRow> CreateRow(RowDetailsItemPtr InItem, const TSharedRef<STableViewBase>& OwnerTable);
		bool HasItem(const FName& ItemName) const;

		TArray<TWeakObjectPtr<const UScriptStruct>> RequestedTedsColumns;
		TSharedPtr<SListView<RowDetailsItemPtr>> ListView;
		TArray<RowDetailsItemPtr> Items;

		ICoreProvider* DataStorage = nullptr; 
		IUiProvider* DataStorageUi = nullptr;

		IUiProvider::FPurposeID WidgetPurpose;

		TSharedPtr<FMetaData> NameMetaData;
		TSharedPtr<FMetaData> DataMetaData;
		TArray<TSharedPtr<FColumnMetaData>> ColumnMetaData;
		TAttribute<FMargin> RowPadding;
		bool bShowAllDetails = true;
		bool bShowUnpopulatedColumns = false;

		FOnRelatedRowSelected OnRelatedRowSelectedDelegate;
	};

	class SRowDetailsRow : public SMultiColumnTableRow<RowDetailsItemPtr>
	{
	public:
		
		SLATE_BEGIN_ARGS(SRowDetailsRow) {}
		
			SLATE_ARGUMENT(RowDetailsItemPtr, Item)
			// The padding inside each Row
			SLATE_ATTRIBUTE(FMargin, Padding)
		SLATE_END_ARGS()
		
		void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& OwnerTableView, const TSharedRef<SRowDetails>& InOwnerRowDetails);

		virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override;

	private:
		RowDetailsItemPtr Item;
		TWeakPtr<SRowDetails> OwnerRowDetails;
	};
} // namespace UE::Editor::DataStorage

#undef UE_API
