// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "DataStorage/Handles.h"
#include "Widgets/SCompoundWidget.h"

#define UE_API TEDSTABLEVIEWER_API

class SWidgetSwitcher;

template <typename ItemType>
class SBreadcrumbTrail;

namespace UE::Editor::DataStorage
{
	class SRowDetails;

	// Composite widget: breadcrumb trail + a stack of SRowDetails panels.
	// Navigating to a related row pushes a new panel; clicking a breadcrumb pops back.
	class SRowDetailsNavigator : public SCompoundWidget
	{
	public:

		SLATE_BEGIN_ARGS(SRowDetailsNavigator) {}
		SLATE_END_ARGS()

		UE_API void Construct(const FArguments& InArgs);

		// Reset the stack and show details for this row.
		UE_API void SetRow(RowHandle Row);

		// Clear everything.
		UE_API void ClearRow();

	private:
		// Push a new panel for the given row.
		void NavigateTo(RowHandle Row);

		// Called when a breadcrumb is clicked; pops panels beyond that point.
		void OnBreadcrumbClicked(const RowHandle& Row);

		// Create a new SRowDetails wired to NavigateTo.
		TSharedRef<SRowDetails> CreatePanel();

		// Remove all panels from index onward.
		void TruncateStack(int32 KeepCount);

		TSharedPtr<SBreadcrumbTrail<RowHandle>> Breadcrumbs;
		TSharedPtr<SWidgetSwitcher> PanelSwitcher;
		TArray<TSharedPtr<SRowDetails>> PanelStack;
	};

} // namespace UE::Editor::DataStorage

#undef UE_API
