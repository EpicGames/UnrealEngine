// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Filters/SBasicFilterBar.h"
#include "TedsFilter.h"
#include "TedsQueryStackInterfaces.h"
#include "Elements/Framework/TypedElementQueryBuilder.h"

#define UE_API TEDSTABLEVIEWER_API

namespace UE::Editor::DataStorage
{
	namespace QueryStack
	{
		class FRowQueryResultsNode;
		class FRowFilterNode;
	}

	/**
	* A Basic TEDS Filter Bar widget, which can be used to filter items of type FTedsRowHandle given a list of custom filters
	* 
	* NOTE: The filter functions create copies, so you want to use a reference or pointer as the template type when possible
	* Sample Usage:
	*		SAssignNew(MyFilterBar, STedsFilterBar)
	*		.InFilterableRowNode(QueryStack)
	*		.OutFilteredNode(&MyOutNode)
	*		.Filters(MyCustomFilters) // An array of filters available to this FilterBar
	*
	* Use MakeAddFilterButton() to make the button that summons the dropdown showing all the filters
	*/
	class STedsFilterBar : public SBasicFilterBar<FTedsRowHandle&>
	{
		using FOnFilterChanged = typename SBasicFilterBar<FTedsRowHandle&>::FOnFilterChanged;
		DECLARE_DELEGATE(FOnPostFiltersChanged);
		
	public:
		SLATE_BEGIN_ARGS(STedsFilterBar)
			: _FilterBarIdentifier(NAME_None)
			, _FilterBarLayout(EFilterBarLayout::Horizontal)
			, _FilterPillStyle(EFilterPillStyle::Basic)
			, _UseSectionsForCategories(false)
			{}

			/** A unique identifier for this filter bar needed to enable saving settings in the config file */
			SLATE_ARGUMENT(FName, FilterBarIdentifier)

			/** The node whose rows we'll filter on */
			SLATE_ARGUMENT(TSharedPtr<QueryStack::IRowNode>, InFilterableRowNode)

			/** The filter results node that will contain the filtered rows */
			SLATE_ARGUMENT(TSharedPtr<QueryStack::IRowNode>*, OutFilteredNode)

			/** Array of filters to add to the filter bar */
			SLATE_ARGUMENT(TArray<TSharedPtr<FTedsFilter>>, Filters)
			
			/** Array of filter names to also add to the 'common filters' section (section at the top of the menu) */
            SLATE_ARGUMENT(TArray<FName>, CommonSectionFilters)
			
			/** Delegate for when filters have changed */
			SLATE_EVENT(FOnFilterChanged, OnFilterChanged)

			/** Delegate for after the filters have changed and the query stack has been updated */
			SLATE_EVENT(FOnPostFiltersChanged, OnPostFiltersChanged)
			
			/** The layout that determines how the filters are laid out */
			SLATE_ARGUMENT(EFilterBarLayout, FilterBarLayout)
			
			/** Determines how each individual filter pill looks like */
			SLATE_ARGUMENT(EFilterPillStyle, FilterPillStyle)
		
			/** Whether to use submenus or sections for categories in the filter menu */
			SLATE_ARGUMENT(bool, UseSectionsForCategories)
			
		SLATE_END_ARGS()

		/** Constructs this widget with InArgs */
		UE_API void Construct(const FArguments& InArgs);

		/** Override tick so that filters are only recompiled once if multiple were added at once */
		UE_API virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

		/** Add a new Filter to the menu post-initialization, will overwrite any filters with the same name */
		UE_API TSharedPtr<FTedsFilter> AddFilter(const TSharedPtr<FTedsFilter> InFilter);
		/** Add an array of new Filters to the menu post-initialization, will overwrite any filters with the same name */
		UE_API void AddFilters(const TArray<TSharedPtr<FTedsFilter>>& InFilters);
		/** Remove a Filter from the menu and remove any place it may be referenced */
		UE_API void DeleteFilter(const FName& InFilterName);
		/** Remove all Filters from the menu and all references */
		UE_API void DeleteAllFilters();
		/** Set a new array of Common filters to appear in that section */
		UE_API void SetCommonFilters(const TArray<FName>& InFilters);
		
		/** 
		 * Remove all filters and replace them with a new array of filters with an option to persist their state (is pinned & is active) for new
		 * filters with the same FName identifier
		 */
		UE_API void ReplaceFilters(const TArray<TSharedPtr<FTedsFilter>>& InFilters, const bool bPersistFilterStates = true);
		
		// Add an external query description to the FilterBar
		UE_API void AddExternalQuery(const FName& QueryName, const QueryHandle& InQueryHandle);
		UE_API void RemoveExternalQuery(const FName& QueryName);

		// Append all external queries into the given query description
		UE_API void AppendExternalQueries(DataStorage::FQueryDescription& OutQuery);

		// Add an external query function to the FilterBar
		UE_API void AddExternalQueryFunction(const FName& QueryName, const Queries::TConstQueryFunction<bool>& InQueryFunction);
		UE_API void RemoveExternalQueryFunction(const FName& QueryName);

		// Add an external class query function to the FilterBar (uses OR) 
		UE_API void AddClassQueryFunction(const FName& ClassName, const Queries::TConstQueryFunction<bool>& InClassQueryFunction);
		UE_API void RemoveClassQueryFunction(const FName& ClassName);

	protected:
		UE_API void RecompileQueries();
		
		UE_API virtual void PopulateAddFilterMenu(UToolMenu* InMenu);
		UE_API virtual void PopulateCommonFilterSections(UToolMenu* InMenu);
		UE_API virtual void PopulateCustomFilters(UToolMenu* InMenu);
	
	private:
		UE_API virtual TSharedRef<SWidget> MakeAddFilterMenu() override;

		ICoreProvider* Storage = nullptr;

		// Array of Filter FNames used to populate the 'common filters' section that filters can have a duplicate entry
		// placed at the top of the add filter menu
		TArray<FName> CommonSectionFilters;

		// The row node passed in as the data to filter on
		TSharedPtr<QueryStack::IRowNode> InFilterableRowNode;

		// Pointer to the Node that is filtered and updated by the FilterBar
		TSharedPtr<QueryStack::IRowNode>* OutFilteredNode = nullptr;
		
		// The Node responsible for compositing the final node to be given to the OutFilteredNode
		TSharedPtr<QueryStack::IRowNode> CompositeFilterNode;

		// External query descriptions that are currently active (e.g Filters)
		TMap<FName, TSharedPtr<QueryStack::IQueryNode>> ExternalQueries;
	
		// External query functions that are currently active (e.g Filters)
		TMap<FName, Queries::TConstQueryFunction<bool>> ExternalQueryFunctions;

		// External query functions used to filter by class, these are the only filters that use OR instead of AND
		TMap<FName, Queries::TConstQueryFunction<bool>> ClassFilters;

		// Array to store all filter nodes created from the active query functions
		TArray<TSharedPtr<QueryStack::IRowNode>> FilterNodes;

		// Array to store all filter nodes created from the active class filters
		TArray<TSharedPtr<QueryStack::IRowNode>> ClassFilterNodes;
		
		// The query stack node responsible for collecting all rows that match the composite query on FullRefresh()
		TSharedPtr<QueryStack::IRowNode> CombinedRowCollectorNode;

		// The query stack node responsible for combining query filters - utilizes the repeated merge approach (AND)
		TSharedPtr<QueryStack::IRowNode> CombinedQueryFilterRowNode;

		// The query stack node responsible for combining class filters - utilizes the unique merge approach (OR)
		TSharedPtr<QueryStack::IRowNode> CombinedClassFilterRowNode;
		
		// Delegate for additional functionality to perform after the filters have been updated, performed at the end of the RecompileQueries function
		FOnPostFiltersChanged OnPostFiltersChanged;

		// Default Category if the category ptr on the Filter Data is a nullptr
		TSharedPtr<FFilterCategory> OtherFiltersCategory;

		// Responsible for seeing if a filter was added or removed so multiple recompiles aren't performed every frame
		bool bMarkDirty = false;
	};
}

#undef UE_API
