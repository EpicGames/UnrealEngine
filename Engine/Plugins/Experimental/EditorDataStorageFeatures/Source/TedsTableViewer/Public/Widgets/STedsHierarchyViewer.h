// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "HierarchyViewerIntefaces.h"
#include "ITedsTableViewer.h"
#include "DataStorage/Handles.h"
#include "TedsRowFilterNode.h"
#include "TedsTableViewerModel.h"
#include "TedsTableViewerUtils.h"
#include "TedsQueryStackInterfaces.h"
#include "TedsQueryStackExecutor.h"
#include "Templates/Function.h"
#include "Elements/Columns/TypedElementHiearchyColumns.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/STableRow.h"

class ITableRow;
class STableViewBase;
class SHeaderRow;

namespace UE::Editor
{
	class FWidgetDropHandler;
	
	namespace DataStorage
	{
		class ITedsWidget;
		class STedsTreeView;

		namespace Widgets
		{
			class FTableViewerWidgetDropHandler;
		}
	}
}

namespace UE::Editor::DataStorage
{
	class FTedsTableViewerColumn;

	/*
	 * A table viewer widget can be used to show a visual representation of data in TEDS.
	 * The rows to display can be specified using a RowQueryStack, and the columns to display are directly input into the widget
	 * Example usage:
	 * 
	 *	SNew(SHierarchyViewer)
     *		.QueryStack(MakeShared<UE::Editor::DataStorage::FQueryStackNode_RowView>(&Rows))
	 *		.Columns({FTypedElementLabelColumn::StaticStruct(), FTypedElementClassTypeInfoColumn::StaticStruct());
	 */
	class SHierarchyViewer : public SCompoundWidget, public ITableViewer
	{
	friend class STedsCompositeHierarchyViewer;
		
	public:
		
		enum class EExpansionState : uint8
		{
			Collapsed,
			Expanded,
			Invalid,
		};

		// Delegate fired when the selection in the table viewer changes
		DECLARE_DELEGATE_TwoParams(FOnSelectionChanged, RowHandle, ESelectInfo::Type)

		SLATE_BEGIN_ARGS(SHierarchyViewer)
			: _CanSelectGeneratedColumn(true)
			, _ListSelectionMode(ESelectionMode::Type::Single)
			, _PersistUIRowsOnClose(false)
			, _ExpandNewRows(true)
			, _AllowInvisibleItemSelection(false)
			, _ShowWires(false)
			, _ShowHiddenRows(false)
		{
			
		}
		// Unique Name of this TableViewer (Hierarchy)
		SLATE_ARGUMENT(FName, TableViewerIdentifier)

		// Query Stack that will supply the rows to be displayed
		SLATE_ARGUMENT(TSharedPtr<QueryStack::IRowNode>, AllNodeProvider)

		// The Columns that this table viewer will display
		SLATE_ARGUMENT(TArray<TWeakObjectPtr<const UScriptStruct>>, Columns)

		// Generic Metadata about all widgets table viewer is displaying
		SLATE_ARGUMENT(TSharedPtr<FMetaData>, GenericMetaData)

		// Metadata about the columns the table viewer is displaying
		SLATE_ARGUMENT(TArray<TSharedPtr<FColumnMetaData>>, ColumnMetaData)

		// The widget purpose to use to create the cell widgets
		SLATE_ARGUMENT(IUiProvider::FPurposeID, CellWidgetPurpose)
			
		// The widget purpose to use to create the header widgets
		SLATE_ARGUMENT(IUiProvider::FPurposeID, HeaderWidgetPurpose)
		
		// Whether the user can select the columns generated by right-clicking on the header menu.
		SLATE_ARGUMENT(bool, CanSelectGeneratedColumn)

		// Delegate called when the selection changes
		SLATE_EVENT(FOnSelectionChanged, OnSelectionChanged)

		// The selection mode for the table viewer (single/multi etc)
		SLATE_ARGUMENT(ESelectionMode::Type, ListSelectionMode)

		// Persists all TableViewer UI rows from this viewer past destruction
		SLATE_ARGUMENT(bool, PersistUIRowsOnClose);

		// Delegate called to define a custom row mapping for the Table Viewer Row Widget Rows
		// Used in cases where you need the UI Rows to persist across RowHandle changes
		SLATE_EVENT(TableViewerUtils::FGetTableViewerRowMapping, GetCustomTableViewerRowMapping)

		// Whether new rows will be expanded or collapsed when coming into view, rows will expand on initialization (if not persisted) then follow this set bool
		SLATE_ARGUMENT(bool, ExpandNewRows)

		// The message to show in place of the table viewer when there are no rows provided by the current query stack
		// Empty = simply show the column headers instead of a message
		SLATE_ATTRIBUTE(FText, EmptyRowsMessage)

		// The primary column in the hierarchy viewer is the one where the expansion row shows up
		SLATE_ARGUMENT(TWeakObjectPtr<const UScriptStruct>, PrimaryColumn)

		// Allows collapsed or temporarily hidden items to still be selected
		SLATE_ARGUMENT(bool, AllowInvisibleItemSelection);
		
		// Whether to show hierarchy lines visually
        SLATE_ARGUMENT(bool, ShowWires);

		// If true, rows tagged with FHideRowFromUITag will still be displayed
		SLATE_ARGUMENT(bool, ShowHiddenRows)

		SLATE_END_ARGS()

	public:
		TEDSTABLEVIEWER_API SHierarchyViewer();
		TEDSTABLEVIEWER_API virtual ~SHierarchyViewer() override;
		
		TEDSTABLEVIEWER_API void Construct(
			const FArguments& InArgs,
			TSharedPtr<IHierarchyViewerDataInterface> InHierarchyInterface);

		// Clear the current QueryStack being displayed, set it to the given node, and recreate the sorting nodes 
		TEDSTABLEVIEWER_API virtual void SetQueryStack(TSharedPtr<QueryStack::IRowNode> InRowQueryStack) override;
		
		// Clear the current list of columns being displayed and set it to the given list
		TEDSTABLEVIEWER_API virtual void SetColumns(const TArray<TWeakObjectPtr<const UScriptStruct>>& InColumns) override;

		// Sets the Table Identifier (Can be used in case data is dynamic and created post widget init)
		TEDSTABLEVIEWER_API virtual void SetTableViewerIdentifier(const FName& NewIdentifier) override;

    	// Add a custom per-row widget to the table viewer (that doesn't necessarily map to a TEDS column)
		// This means a new column for the hierarchy viewer
		TEDSTABLEVIEWER_API virtual void AddCustomRowWidget(const TSharedRef<FTedsTableViewerColumn>& InColumn) override;

		// Execute the given callback for each row that is selected in the table viewer
		TEDSTABLEVIEWER_API virtual void ForEachSelectedRow(TFunctionRef<void(RowHandle)> InCallback) const override;

		// Get the row handle for the widget row the table viewer's contents are stored in
		TEDSTABLEVIEWER_API virtual RowHandle GetWidgetRowHandle() const override;
		
		// Select the given row in the table viewer
		TEDSTABLEVIEWER_API virtual void SetSelection(RowHandle Row, bool bSelected, const ESelectInfo::Type SelectInfo) const override;

		// Scroll the given row into view in the table viewer
		TEDSTABLEVIEWER_API virtual void ScrollIntoView(RowHandle Row) const override;

		TEDSTABLEVIEWER_API virtual void ClearSelection() const override;
		
		TEDSTABLEVIEWER_API virtual TSharedRef<SWidget> AsWidget() override;

		// Get if a specific row is selected
		TEDSTABLEVIEWER_API virtual bool IsSelected(RowHandle InRow) const override;

		// Get if a specific row is selected exclusively
		TEDSTABLEVIEWER_API virtual bool IsSelectedExclusively(RowHandle InRow) const override;

		// Recursively Expand all rows currently present in the viewer
		TEDSTABLEVIEWER_API void ExpandAll() const;

		// Recursively Collapse all rows currently present in the viewer
		TEDSTABLEVIEWER_API void CollapseAll() const;

		// Expand this row and all its parents
		TEDSTABLEVIEWER_API void ExpandWithParents(RowHandle Row);

		// Set a handler that this viewer will forward its drop operations to.
		TEDSTABLEVIEWER_API void SetDropHandler(TUniquePtr<FWidgetDropHandler> InDropHandler);
		
	protected:
		
		void GetTableViewerRowMapping(const FName& InTableViewerIdentifier, const RowHandle InRow, FMapKey& OutMapKey) const;
		
		RowHandle AddOrFindTableViewerUIRow(const TableViewerItemPtr InItem) const;

		void InitializeAllTableViewerUIRows() const;
		
		TSharedRef<ITableRow> MakeTableRowWidget(TableViewerItemPtr InItem, const TSharedRef<STableViewBase>& OwnerTable) const;

		bool IsItemVisible(TableViewerItemPtr InItem) const;

		void CreateInternalWidget();

		void RefreshColumnWidgets();

		void OnExpansionChanged(TableViewerItemPtr Item, bool bExpanded) const;
		
		void OnSetExpansionRecursive(TableViewerItemPtr Item, bool bExpanded) const;

		void OnListSelectionChanged(TableViewerItemPtr Item, ESelectInfo::Type SelectInfo);

		void AddWidgetColumns();

		TableViewerItemPtr GetParentRow(TableViewerItemPtr InItem) const;

		void OnModelChanged();

		void SetExpansionState_Internal(FRowHandleArrayView Rows, bool bExpanded) const;

		// Drag and Drop handlers
		TEDSTABLEVIEWER_API virtual void OnDragEnter(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;
		TEDSTABLEVIEWER_API virtual FReply OnDragOver(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;
		TEDSTABLEVIEWER_API virtual void OnDragLeave(const FDragDropEvent& DragDropEvent) override;
		TEDSTABLEVIEWER_API virtual FReply OnDrop( const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent ) override;

	private:

		// Unique identifier for this Hierarchy
		FName TableViewerIdentifier;
		
		// The actual ListView widget that displays the rows
		TSharedPtr<STedsTreeView> TreeView;

		// The actual header widget
		TSharedPtr<SHeaderRow> HeaderRowWidget;

		// Our model class
		TSharedPtr<FTedsTableViewerModel> Model;

		// Delegate fired when the selection changes
		FOnSelectionChanged OnSelectionChanged;

		// If we want to persist all Table Viewer Rows and their mapping to this table ID when the viewer is destructed
		// (keeping them around will persist its data while the editor is open)
		bool bPersistUIRowsOnClose = false;
		
		// Delegate fired when getting the row mapping, will use the default 'TableId.DataRowHandle' if nothing is bound
		TableViewerUtils::FGetTableViewerRowMapping GetCustomTableViewerRowMapping;

		// Wrapper Teds Widget around our contents so we can use Teds columns to specify behavior
		TSharedPtr<ITedsWidget> TedsWidget;

		// The message to show in place of the table viewer when there are no rows provided by the current query stack
		TAttribute<FText> EmptyRowsMessage;

		// A specialized query stack node to only contain the top level rows present in the actual query stack we are using.
		TSharedPtr<QueryStack::IRowNode> TopLevelRowsNode;

		// The executor used to update the hierarchy stack.
		QueryStack::FExplicitUpdateExecutor QueryStackExecutor;
		
		// Store the Instance count and the created name for use by the executor and identifier (so it can avoid recreation)
		inline static int InstanceCounter = 0;
		FName ExecutorInstanceName;
		
		// The cached list of top level rows
		TArray<TableViewerItemPtr> TopLevelRows;

		// Hierarchy data that is used to extract information about the parent of a row
		TSharedPtr<IHierarchyViewerDataInterface> HierarchyInterface;

		// If we need to update the expansion state of all rows the next time the model changes
		mutable TOptional<bool> bExpansionStateChangeRequested;

		// If the model hasn't been initialized yet, will check during OnModelChanged and let any PostModelInitialization functions run
		bool bModelInitialized = false;

		// If any newly viewed rows (after initialization) should be expanded by default
		bool bExpandNewRows = true;

		// Drag and Drop Handler
		TUniquePtr<FWidgetDropHandler> DropHandler;
		
		// Whether to show hierarchy lines visually
		bool bShowWires = false;
	};
}