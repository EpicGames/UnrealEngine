// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Misc/Variant.h"
#include "TedsQueryStackInterfaces.h"
#include "TedsRowFilterNode.h"
#include "TedsQuerySearchNode.h"
#include "Widgets/SCompoundWidget.h"

namespace UE::Editor::DataStorage
{
	namespace QueryStack
	{
		class FRowQueryResultsNode;
	}

	/*
	 * A search bar widget that can be used to search across a TEDS table viewer widget using a QueryStack Search node.
	 * Example future usage:
	 * 
	 *	SNew(STedsSearchBox)
	 *	.OutSearchNode(&OutSearchNode) // TSharedPtr<QueryStack::IRowNode> OutSearchNode
	 *	.InSearchableRowNode(ReferenceQueryResultsNode)
	 */
	class STedsSearchBox : public SCompoundWidget
	{
	public:
		using ESearchableQueryFlags = QueryStack::FQuerySearchNode::ESyncActions;

		SLATE_BEGIN_ARGS(STedsSearchBox) {}

		/** 
		 * The node provider to search through.
		 * If InSearchableQueryNode is also set, this node will be ignored.
		 */
		SLATE_ARGUMENT(TSharedPtr<QueryStack::IRowNode>, InSearchableRowNode)

		/**
		 * Query node used for searching.If this and InSearchableRowNode are set, the query node will take priority
		 * and InSearchableRowNode will be ignored. If there's no active search this node will execute the query without
		 * filtering and collect the results.
		 */
		SLATE_ARGUMENT(TSharedPtr<QueryStack::IQueryNode>, InSearchableQueryNode)

		/** If a searchable query is set, this sets how updates happen that run the query. */
		SLATE_ARGUMENT(ESearchableQueryFlags, InSearchableQueryFlags)

		/** 
		 * The result node containing a list of rows. This can be used as an input to other
		 * nodes in the Query Stack.
		 * If a search is active this will result in a list of rows filtered to the search criteria.
		 * If there's no search than this will return the results of running the query if a query node was
		 * provided. If a row node was provided the original list of rows will be returned.
		 */
		SLATE_ARGUMENT(TSharedPtr<QueryStack::IRowNode>*, OutSearchNode)
			
		SLATE_END_ARGS()
	
		TEDSTABLEVIEWER_API void Construct(const FArguments& InArgs);

	private:
		void OnTextChanged(const FText& InSearchText);

		TVariant<
			TSharedPtr<QueryStack::FRowFilterNode>,
			TSharedRef<QueryStack::FQuerySearchNode>
		> SearchNode;
		FText SearchText;
	};
}