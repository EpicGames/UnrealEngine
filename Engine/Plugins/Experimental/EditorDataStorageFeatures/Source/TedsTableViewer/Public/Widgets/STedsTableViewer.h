// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "ITedsTableViewer.h"
#include "DataStorage/Handles.h"
#include "TedsQueryStackInterfaces.h"
#include "TedsTableViewerModel.h"
#include "TedsTableViewerUtils.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/SListView.h"

class ITableRow;
class STableViewBase;
class SHeaderRow;

namespace UE::Editor
{
	class FWidgetDropHandler;
	
	namespace DataStorage
	{
		class ITedsWidget;
		namespace Widgets
		{
			class FTableViewerWidgetDropHandler;
		}
	}
}

namespace UE::Editor::DataStorage
{
	class FTedsTableViewerColumn;

	/*
	 * A table viewer widget can be used to show a visual representation of data in TEDS.
	 * The rows to display can be specified using a RowQueryStack, and the columns to display are directly input into the widget
	 * Example usage:
	 * 
	 *	SNew(STedsTableViewer)
     *		.QueryStack(MakeShared<UE::Editor::DataStorage::FQueryStackNode_RowView>(&Rows))
	 *		.Columns({FTypedElementLabelColumn::StaticStruct(), FTypedElementClassTypeInfoColumn::StaticStruct());
	 */
	class STedsTableViewer : public SCompoundWidget, public ITableViewer
	{
	public:
		
		// Delegate fired when the selection in the table viewer changes
		DECLARE_DELEGATE_TwoParams(FOnSelectionChanged, RowHandle, ESelectInfo::Type)

		SLATE_BEGIN_ARGS(STedsTableViewer)
			: _CanSelectGeneratedColumn(true)
			, _ListSelectionMode(ESelectionMode::Type::Single)
			, _PersistUIRowsOnClose(false)
			, _ItemHeight()
			, _ItemPadding(FMargin(0.f))
			, _ShowHiddenRows(false)
		{
			
		}
		
		// Unique Name of this Table
		SLATE_ARGUMENT(FName, TableViewerIdentifier)

		// Query Stack that will supply the rows to be displayed
		SLATE_ARGUMENT(TSharedPtr<QueryStack::IRowNode>, QueryStack)

		// The Columns that this table viewer will display
		SLATE_ARGUMENT(TArray<TWeakObjectPtr<const UScriptStruct>>, Columns)

		// Generic Metadata about all widgets table viewer is displaying
		SLATE_ARGUMENT(TSharedPtr<FMetaData>, GenericMetaData)

		// Metadata about the columns the table viewer is displaying
		SLATE_ARGUMENT(TArray<TSharedPtr<FColumnMetaData>>, ColumnMetaData)

		// The widget purpose to use to create the cell widgets
		SLATE_ARGUMENT(IUiProvider::FPurposeID, CellWidgetPurpose)
			
		// The widget purpose to use to create the header widgets
		SLATE_ARGUMENT(IUiProvider::FPurposeID, HeaderWidgetPurpose)

		// Whether the user can select the columns generated by right-clicking on the header menu.
		SLATE_ARGUMENT(bool, CanSelectGeneratedColumn)
		
		// Delegate called when the selection changes
		SLATE_EVENT(FOnSelectionChanged, OnSelectionChanged)

		// The selection mode for the table viewer (single/multi etc)
		SLATE_ARGUMENT(ESelectionMode::Type, ListSelectionMode)

		// Persists all TableViewer Row UI rows from this viewer past destruction
		SLATE_ARGUMENT(bool, PersistUIRowsOnClose);

		// Delegate called to define a custom row mapping for the Table Viewer Row Widget Rows
		// Used in cases where you need the UI Rows to persist across RowHandle changes
		SLATE_EVENT(TableViewerUtils::FGetTableViewerRowMapping, GetCustomTableViewerRowMapping)

		// The message to show in place of the table viewer when there are no rows provided by the current query stack
		// Empty = simply show the column headers instead of a message
		SLATE_ATTRIBUTE(FText, EmptyRowsMessage)
		
		// The primary column in the table viewer is the one that is sorted by default
		SLATE_ARGUMENT(TWeakObjectPtr<const UScriptStruct>, PrimaryColumn)

		/** The height of the list item */
		SLATE_ATTRIBUTE(float, ItemHeight)

		/** The padding of the list item */
		SLATE_ATTRIBUTE(FMargin, ItemPadding)

		// If true, rows tagged with FHideRowFromUITag will still be displayed
		SLATE_ARGUMENT(bool, ShowHiddenRows)

		SLATE_END_ARGS()

	public:
		TEDSTABLEVIEWER_API STedsTableViewer();
		TEDSTABLEVIEWER_API virtual ~STedsTableViewer() override;
		
		TEDSTABLEVIEWER_API void Construct(const FArguments& InArgs);

		// Sets the Table Identifier (Can be used in case data is dynamic and created post widget init)
		TEDSTABLEVIEWER_API virtual void SetTableViewerIdentifier(const FName& NewIdentifier) override;

		// Clear the current QueryStack being displayed, set it to the given node, and recreate the sorting nodes 
		TEDSTABLEVIEWER_API virtual void SetQueryStack(TSharedPtr<QueryStack::IRowNode> InRowQueryStack) override;
		
		// Clear the current list of columns being displayed and set it to the given list
		TEDSTABLEVIEWER_API virtual void SetColumns(const TArray<TWeakObjectPtr<const UScriptStruct>>& InColumns) override;

		// Add a custom per-row widget to the table viewer (that doesn't necessarily map to a TEDS column)
		// This means a new column for the table viewer
		TEDSTABLEVIEWER_API virtual void AddCustomRowWidget(const TSharedRef<FTedsTableViewerColumn>& InColumn) override;

		// Execute the given callback for each row that is selected in the table viewer
		TEDSTABLEVIEWER_API virtual void ForEachSelectedRow(TFunctionRef<void(RowHandle)> InCallback) const override;

		// Get the row handle for the widget row the table viewer's contents are stored in
		TEDSTABLEVIEWER_API virtual RowHandle GetWidgetRowHandle() const override;
		
		// Select the given row in the table viewer
		TEDSTABLEVIEWER_API virtual void SetSelection(RowHandle Row, bool bSelected, const ESelectInfo::Type SelectInfo) const override;

		// Scroll the given row into view in the table viewer
		TEDSTABLEVIEWER_API virtual void ScrollIntoView(RowHandle Row) const override;

		TEDSTABLEVIEWER_API virtual void ClearSelection() const override;
		
		TEDSTABLEVIEWER_API virtual TSharedRef<SWidget> AsWidget() override;

		TEDSTABLEVIEWER_API virtual bool IsSelected(RowHandle InRow) const override;

		TEDSTABLEVIEWER_API virtual bool IsSelectedExclusively(RowHandle InRow) const override;

		// Drag and Drop handlers
		TEDSTABLEVIEWER_API virtual void OnDragEnter(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;
		TEDSTABLEVIEWER_API virtual FReply OnDragOver(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;
		TEDSTABLEVIEWER_API virtual void OnDragLeave(const FDragDropEvent& DragDropEvent) override;
		TEDSTABLEVIEWER_API virtual FReply OnDrop( const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent ) override;
		
		// Set a handler that this viewer will forward its drop operations to.
		TEDSTABLEVIEWER_API void SetDropHandler(TUniquePtr<FWidgetDropHandler> InDropHandler);

	protected:
	
		void GetTableViewerRowMapping(const FName& InTableViewerIdentifier, const RowHandle InRow, FMapKey& OutMapKey) const;

		RowHandle AddOrFindTableViewerUIRow(const TableViewerItemPtr InItem) const;

		void InitializeAllTableViewerUIRows() const;
		
		TSharedRef<ITableRow> MakeTableRowWidget(TableViewerItemPtr InItem, const TSharedRef<STableViewBase>& OwnerTable) const;

		bool IsItemVisible(TableViewerItemPtr InItem) const;

		void CreateInternalWidget();

		void RefreshColumnWidgets();

		void OnListSelectionChanged(TableViewerItemPtr Item, ESelectInfo::Type SelectInfo);

		void AddWidgetColumns();

		void OnModelChanged();

	private:
		// Unique identifier for this TableViewer
		FName TableViewerIdentifier;
		
		// The actual height of each item
		TAttribute<float> ItemHeight;

		// The actual padding between each item
		TAttribute<FMargin> ItemPadding;

		// The actual ListView widget that displays the rows
		TSharedPtr<SListView<TableViewerItemPtr>> ListView;

		// The actual header widget
		TSharedPtr<SHeaderRow> HeaderRowWidget;

		// Our model class
		TSharedPtr<FTedsTableViewerModel> Model;

		// Store the Instance count and the created name for use by the identifier
		inline static int InstanceCounter = 0;

		// Delegate fired when the selection changes
		FOnSelectionChanged OnSelectionChanged;
		
		// If we want to persist all Table Viewer Rows and their mapping to this table ID when the viewer is destructed
		// (keeping them around will persist its data while the editor is open)
		bool bPersistUIRowsOnClose = false;
        		
		// Delegate fired when getting the row mapping, will use the default 'TableId.DataRowHandle' if nothing is bound
		TableViewerUtils::FGetTableViewerRowMapping GetCustomTableViewerRowMapping;

		// Wrapper Teds Widget around our contents so we can use Teds columns to specify behavior
		TSharedPtr<ITedsWidget> TedsWidget;

		// The message to show in place of the table viewer when there are no rows provided by the current query stack
		TAttribute<FText> EmptyRowsMessage;
		
		// If the model hasn't been initialized yet, will check during OnModelChanged and let any PostModelInitialization functions run
		bool bModelInitialized = false;

		// Drag and Drop Handler
		TUniquePtr<FWidgetDropHandler> DropHandler;
	};
}