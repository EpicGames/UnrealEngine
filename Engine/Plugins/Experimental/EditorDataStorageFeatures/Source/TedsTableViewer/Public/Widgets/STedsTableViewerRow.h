// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "TedsTableViewerModel.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableRow.h"

namespace UE::Editor::DataStorage
{
	class ITableViewer;
}

namespace UE::Editor::DataStorage
{
	class STedsTableViewer;
	
	/** Widget that represents a row in the table viewer. Generates widgets for each column on demand. */
	class STedsTableViewerRow
		: public SMultiColumnTableRow< TableViewerItemPtr >
	{

	public:

		SLATE_BEGIN_ARGS( STedsTableViewerRow )
			: _WidgetRowHandle(InvalidRowHandle)
			, _TableViewerWidgetRowHandle(InvalidRowHandle)
			, _Padding(FMargin(0.f, 4.f))
			, _ItemHeight()
			, _ShowWires(false)
		{}

		/** The list item for this row */
		SLATE_ARGUMENT( TableViewerItemPtr, Item )

		/** Widget row representing this table viewer row in the UI */
		SLATE_ARGUMENT( RowHandle, WidgetRowHandle )

		/** Widget row representing the whole table viewer widget */
		SLATE_ARGUMENT( RowHandle, TableViewerWidgetRowHandle )

		/** The amount of padding to use */
		SLATE_ATTRIBUTE( FMargin, Padding )
			
		/** The height of the list item */
		SLATE_ATTRIBUTE( float, ItemHeight )

		/** Whether to draw when showing hierarchies */
		SLATE_ARGUMENT( bool, ShowWires )
			
		SLATE_END_ARGS()
		
		/** Construct function for this widget */
		void Construct( const FArguments& InArgs, TSharedRef<STableViewBase> OwnerTableView, TSharedRef<FTedsTableViewerModel> InTableViewerModel, TWeakPtr<const ITableViewer> InTableViewer);

		/** Overridden from SMultiColumnTableRow. Generates a widget for this column of the tree row. */
		virtual TSharedRef<SWidget> GenerateWidgetForColumn( const FName& ColumnName ) override;

	protected:

		/** Add some common columns to the widget row for each cell widget created */
		void SetupWidgetColumns(ICoreProvider& InStorage, const RowHandle& InUIRowHandle);

	private:
		/** Get the current ItemHeight as an OptionalSize for the Box HeightOverride */
		FOptionalSize GetCurrentItemHeight() const;

		/** Drag & Drop handlers */
		void HandleOnDragEnter(const FDragDropEvent& DragDropEvent);
		void HandleOnDragLeave(const FDragDropEvent& DragDropEvent);
		FReply HandleOnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent);

	protected:
		TAttribute<float> ItemHeight;
		RowHandle WidgetRowHandle = InvalidRowHandle;
		RowHandle TableViewerWidgetRowHandle = InvalidRowHandle;
		TSharedPtr<FTedsTableViewerModel> TableViewerModel;
		TWeakPtr<const ITableViewer> WeakTableViewer;
		TableViewerItemPtr Item;
		bool bShowWires = false;
	};

	/** Widget that represents a row in the hierarchy viewers. Generates widgets for each column and adds an expander arrow to the very first column. */
	class SHierarchyViewerRow : public STedsTableViewerRow
	{
	public:
		virtual TSharedRef<SWidget> GenerateWidgetForColumn( const FName& ColumnName ) override;
	};
} // namespace UE::Editor::DataStorage
