// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "ITedsTableViewer.h"
#include "DataStorage/Handles.h"
#include "TedsQueryStackInterfaces.h"
#include "TedsTableViewerModel.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/STileView.h"

class ITableRow;
class STableViewBase;
class SHeaderRow;

namespace UE::Editor::DataStorage
{
	class ITedsWidget;
}

namespace UE::Editor::DataStorage
{
	/* Table Viewer where each row is represented as a tile using STileView
	 * Unlike STedsTableViewer which takes a list of columns to display the tile viewer simply uses a single widget purpose
	 * to create tiles for each row.
	 */
	class STedsTileViewer : public SCompoundWidget, public ITableViewer
	{
	public:
		
		// Delegate fired when the selection in the table viewer changes
		DECLARE_DELEGATE_OneParam(FOnSelectionChanged, RowHandle)

		SLATE_BEGIN_ARGS(STedsTileViewer)
			: _WidgetPurpose()
			, _SelectionMode(ESelectionMode::Type::Single)
			, _ItemHeight(128)
			, _ItemWidth(128)
			, _TileStyle(&FAppStyle::Get().GetWidgetStyle<FTableRowStyle>("TableView.Row"))
			, _TilePadding(FMargin(0.f))
			, _ItemAlignment(EListItemAlignment::EvenlyDistributed)
			, _ShowHiddenRows(false)
		{
			
		}

		// Query Stack that will supply the rows to be displayed
		SLATE_ARGUMENT(TSharedPtr<QueryStack::IRowNode>, QueryStack)

		// Optional list of columns to give TEDS UI when creating the tile widget
		SLATE_ARGUMENT(TArray<TWeakObjectPtr<const UScriptStruct>>, Columns)

		// The widget purpose to use to create the tile widget
		SLATE_ARGUMENT(IUiProvider::FPurposeID, WidgetPurpose)
		
		// Delegate called when the selection changes
		SLATE_EVENT(FOnSelectionChanged, OnSelectionChanged)
			
		// The selection mode for the table viewer (single/multi etc)
		SLATE_ARGUMENT(ESelectionMode::Type, SelectionMode)

		// Height of each tile
		SLATE_ATTRIBUTE( float, ItemHeight )

		// Width of each tile
		SLATE_ATTRIBUTE( float, ItemWidth )

		// Style of each tile
		SLATE_STYLE_ARGUMENT( FTableRowStyle, TileStyle )

		// Padding between each tile
		SLATE_ARGUMENT( FMargin, TilePadding )

		// Tile View alignment
		SLATE_ATTRIBUTE( EListItemAlignment, ItemAlignment )

		// If true, rows tagged with FHideRowFromUITag will still be displayed
		SLATE_ARGUMENT( bool, ShowHiddenRows )

		SLATE_END_ARGS()

	public:
		
		TEDSTABLEVIEWER_API void Construct(const FArguments& InArgs);

		// Execute the given callback for each row that is selected in the table viewer
		TEDSTABLEVIEWER_API virtual void ForEachSelectedRow(TFunctionRef<void(RowHandle)> InCallback) const override;

		// Get the row handle for the widget row the table viewer's contents are stored in
		TEDSTABLEVIEWER_API virtual RowHandle GetWidgetRowHandle() const override;
		
		// Select the given row in the table viewer
		TEDSTABLEVIEWER_API virtual void SetSelection(RowHandle Row, bool bSelected, const ESelectInfo::Type SelectInfo) const override;

		// Scroll the given row into view in the table viewer
		TEDSTABLEVIEWER_API virtual void ScrollIntoView(RowHandle Row) const override;

		// Deselect all items in the table viewer
		TEDSTABLEVIEWER_API virtual void ClearSelection() const override;

		// Get the table viewer as an SWidget
		TEDSTABLEVIEWER_API virtual TSharedRef<SWidget> AsWidget() override;

		TEDSTABLEVIEWER_API virtual bool IsSelected(RowHandle InRow) const override;

		TEDSTABLEVIEWER_API virtual bool IsSelectedExclusively(RowHandle InRow) const override;
		
		// Sets the Table Identifier (Can be used in case data is dynamic and created post widget init)
        TEDSTABLEVIEWER_API virtual void SetTableViewerIdentifier(const FName& NewIdentifier) override;
		
		// Clear the current QueryStack being displayed, set it to the given node, and recreate the sorting nodes 
        TEDSTABLEVIEWER_API virtual void SetQueryStack(TSharedPtr<QueryStack::IRowNode> InRowQueryStack) override;

		// Clear the current list of columns being displayed and set it to the given list
		TEDSTABLEVIEWER_API virtual void SetColumns(const TArray<TWeakObjectPtr<const UScriptStruct>>& InColumns) override;

		// Add a custom per-row widget to the table viewer (that doesn't necessarily map to a TEDS column)
		TEDSTABLEVIEWER_API virtual void AddCustomRowWidget(const TSharedRef<FTedsTableViewerColumn>& InColumn) override;

	protected:
		
		TSharedRef<ITableRow> MakeTileWidget(TableViewerItemPtr InItem, const TSharedRef<STableViewBase>& OwnerTable) const;

		bool IsItemVisible(TableViewerItemPtr InItem) const;

		void OnListSelectionChanged(TableViewerItemPtr Item, ESelectInfo::Type SelectInfo);

		void AddWidgetColumns();
		
		void CreateTileWidgetConstructor();

	private:

		// Whether dragging is allowed
		bool bAllowDragging = true;

		// The actual ListView widget that displays the rows
		TSharedPtr<STileView<TableViewerItemPtr>> TileView;

		// Our model class
		TSharedPtr<FTedsTableViewerModel> Model;

		// Delegate fired when the selection changes
		FOnSelectionChanged OnSelectionChanged;

		// Wrapper Teds Widget around our contents so we can use Teds columns to specify behavior
		TSharedPtr<ITedsWidget> TedsWidget;

		// The widget purpose used to generate tiles
		IUiProvider::FPurposeID WidgetPurpose;

		// Optional columns to use when matching the tile widget
		TArray<TWeakObjectPtr<const UScriptStruct>> Columns;

		// Style name to use for the rows (tiles)
		const FTableRowStyle* TableRowStyle = nullptr;

		// Padding between each tile
		FMargin TilePadding;
	};
}
