// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DragAndDrop/Widgets/WidgetDropHandler.h"

namespace UE::Editor::DataStorage
{
	class ITableViewer;
}

#define UE_API TEDSTABLEVIEWER_API

namespace UE::Editor::DataStorage::Widgets
{
	class FTableViewerWidgetDropHandler : public FWidgetDropHandler
	{
	public:
		UE_API FTableViewerWidgetDropHandler(TWeakPtr<ITableViewer> InTableViewer, TWeakObjectPtr<UTypedElementSelectionSet> InSelectionSet);
		UE_API virtual ~FTableViewerWidgetDropHandler() override;

	protected:
		virtual EUpdateParameters UpdateParameters(ICoreProvider& Storage, const FGeometry& Geometry, const FDragDropEvent& InputEvent) override;
		virtual void SetRowsFromParameters(ICoreProvider& Storage, FRowHandleArrayView Rows) const override;
		virtual void Stop(ICoreProvider& Storage, UDropOperationSystem& DropSystem) override;
		
		RowHandle GetDropTarget(ICoreProvider& Storage) const;
	private:
		TWeakPtr<ITableViewer> TableViewer;
		RowHandle LastTargetRow = InvalidRowHandle;
	};
}

#undef UE_API
