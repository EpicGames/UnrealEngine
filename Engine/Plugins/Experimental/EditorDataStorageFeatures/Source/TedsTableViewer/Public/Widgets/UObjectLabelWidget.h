// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/Interfaces/TypedElementDataStorageFactory.h"
#include "Elements/Interfaces/TypedElementDataStorageUiInterface.h"
#include "Internationalization/Text.h"
#include "UObject/ObjectMacros.h"

#include "UObjectLabelWidget.generated.h"

namespace UE::Editor::DataStorage
{
	class ICoreProvider;
} // UE::Editor::DataStorage

class UScriptStruct;

UCLASS()
class UUObjectLabelWidgetFactory : public UEditorDataStorageFactory
{
	GENERATED_BODY()

public:
	~UUObjectLabelWidgetFactory() override = default;

	TEDSTABLEVIEWER_API void RegisterWidgetConstructors(UE::Editor::DataStorage::ICoreProvider& DataStorage,
		UE::Editor::DataStorage::IUiProvider& DataStorageUi) const override;
};

// Widget to show labels for UObjects in TEDS
USTRUCT()
struct FUObjectLabelWidgetConstructor : public FSimpleWidgetConstructor
{
	GENERATED_BODY()

public:
	TEDSTABLEVIEWER_API FUObjectLabelWidgetConstructor();
	TEDSTABLEVIEWER_API explicit FUObjectLabelWidgetConstructor(const UScriptStruct* TypeInfo);
	~FUObjectLabelWidgetConstructor() override = default;

	TEDSTABLEVIEWER_API virtual TSharedPtr<SWidget> CreateWidget(
		UE::Editor::DataStorage::ICoreProvider* DataStorage,
		UE::Editor::DataStorage::IUiProvider* DataStorageUi,
		UE::Editor::DataStorage::RowHandle TargetRow,
		UE::Editor::DataStorage::RowHandle WidgetRow, 
		const UE::Editor::DataStorage::FMetaDataView& Arguments) override;

protected:
};