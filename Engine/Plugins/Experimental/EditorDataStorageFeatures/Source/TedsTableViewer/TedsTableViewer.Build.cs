// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class TedsTableViewer : ModuleRules
{
	public TedsTableViewer(ReadOnlyTargetRules Target) : base(Target)
	{
		// Enable truncation warnings in this module
		CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Warning;

		if (Target.bBuildEditor)
		{
			PublicIncludePaths.AddRange(new string[] {});
			PrivateIncludePaths.AddRange(new string[] {});

			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"TedsQueryStack",
					"TypedElementFramework",
					"TedsOperations",
				});

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
					"EditorWidgets",
					"Engine",
					"InputCore",
					"Slate",
					"SlateCore",
					"ToolMenus",
					"ToolWidgets",
					"WorkspaceMenuStructure",
				});
		}
	}
}