// Copyright Epic Games, Inc. All Rights Reserved.

#include "TedsTypeInfoFactory.h"

#include "Elements/Columns/TedsTypeInfoColumns.h"

#include "Modules/ModuleManager.h"
#include "ClassViewerFilter.h"
#include "ClassViewerModule.h"

#include "TedsTypeInfoModule.h"
#include "TedsTypeInfoUtils.h"

#include "Containers/VersePath.h"
#include "DataStorage/Features.h"
#include "Elements/Interfaces/TypedElementDataStorageCompatibilityInterface.h"
#include "Elements/Interfaces/TypedElementDataStorageInterface.h"
#include "Elements/Columns/TypedElementLabelColumns.h"
#include "Elements/Framework/TypedElementQueryBuilder.h"
#include "TedsAlerts.h"
#include "TedsAlertColumns.h"

#include "HAL/IConsoleManager.h"
#include "UObject/InstanceDataObjectUtils.h"
#include "UObject/PropertyBagRepository.h"

#include "UObject/UObjectIterator.h"
#include "VerseVM/VVMVerseClass.h"

namespace UE::Editor::DataStorage::Internal
{
	// Helper for unmangling verse names. Copied here as there's no shared helper util currently
	static FString UnmanglePropertyName(const FName MaybeMangledName, bool& bOutNameWasMangled)
	{
		FString Result = MaybeMangledName.ToString();
		if (Result.StartsWith(TEXTVIEW("__verse_0x")))
		{
			// chop "__verse_0x" (10 char) + CRC (8 char) + "_" (1 char)
			Result = Result.RightChop(19);
			bOutNameWasMangled = true;
		}
		else
		{
			bOutNameWasMangled = false;
		}
		return Result;
	}
}

TAutoConsoleVariable<bool> CVarTEDSTypeInfoOnlyVerseProperties(TEXT("TEDS.Feature.TypeInfoIntegration.OnlyStoreVerseProperties"),
	true,
	TEXT("When true we will only store properties of Verse Classes"));

void UTypeInfoFactory::PreRegister(UE::Editor::DataStorage::ICoreProvider& DataStorage)
{
	DataStorage.OnUpdateCompleted().AddUObject(this, &UTypeInfoFactory::OnDatabaseUpdateCompleted);
}

void UTypeInfoFactory::RegisterTables(UE::Editor::DataStorage::ICoreProvider& DataStorage, UE::Editor::DataStorage::ICompatibilityProvider& DataStorageCompatability)
{
	using namespace UE::Editor::DataStorage;

	Super::RegisterTables(DataStorage, DataStorageCompatability);

	DataStorage.RegisterTable({
		FTypeInfoTag::StaticStruct(),
		FTypedElementLabelColumn::StaticStruct(),
	}, TypeInfo::TypeTableName);

	PropertyTable = DataStorage.RegisterTable({
		FPropertyInfoTag::StaticStruct(),
		FPropertyNameColumn::StaticStruct(),
		FPropertyOwnerTypeRowHandleColumn::StaticStruct(),
		FPropertyTypeNameInfoColumn::StaticStruct(),
		FPropertyTypeNameColumn::StaticStruct(),
	}, TypeInfo::PropertyTableName);
}

void UTypeInfoFactory::RegisterHierarchies(UE::Editor::DataStorage::ICoreProvider& DataStorage)
{
	using namespace UE::Editor::DataStorage;

	const FHierarchyRegistrationParams ClassHierarchyParams
	{
		.Name = TypeInfo::ClassHierarchyName,
	};

	DataStorage.RegisterHierarchy(ClassHierarchyParams);

	const FHierarchyRegistrationParams PropertyHierarchyParams
	{
		.Name = TypeInfo::PropertyHierarchyName,
	};

	PropertyHierarchyHandle = DataStorage.RegisterHierarchy(PropertyHierarchyParams);
}

void UTypeInfoFactory::RegisterQueries(UE::Editor::DataStorage::ICoreProvider& DataStorage)
{
	using namespace UE::Editor::DataStorage;
	using namespace UE::Editor::DataStorage::Queries;

	DataStorage.RegisterQuery(
		Select(
			TEXT("Setup Class Hierarchy Info"),
			FProcessor(EQueryTickPhase::DuringPhysics, DataStorage.GetQueryTickGroupName(EQueryTickGroups::Default))
				.MakeActivatable(TypeInfo::ClassSetupActionName),
			[this](IQueryContext& Context, RowHandle Row, const FTypedElementClassTypeInfoColumn& ClassTypeColumn)
			{
				if (const UClass* ClassInfo = ClassTypeColumn.TypeInfo.Get())
				{
					const UClass* ParentClass = ClassInfo->GetSuperClass();

					RowHandle ParentClassRow = Context.LookupMappedRow(TypeInfo::TypeMappingDomain, FMapKey(ParentClass));

					Context.SetParentRow(Row, ParentClassRow);
				}

				Context.RemoveColumns(Row, { FTypeInfoRequiresHierarchyUpdateTag::StaticStruct() });
			})
			.AccessesHierarchy(TypeInfo::ClassHierarchyName)
			.Where()
				.All<FTypeInfoRequiresHierarchyUpdateTag>()
			.Compile());

	// Populate our type info on register if our CVar is enabled
	static const auto TedsTypeInfoEnabledCVar = IConsoleManager::Get().FindConsoleVariable(TEXT("TEDS.Feature.TypeInfoIntegration.Enable"));
	if (TedsTypeInfoEnabledCVar && TedsTypeInfoEnabledCVar->GetBool())
	{
		RefreshAllTypeInfo();
	}

	TypeInfo::FTedsTypeInfoModule::GetChecked().SetTypeInfoFactoryEnabled();
}

void UTypeInfoFactory::PreShutdown(UE::Editor::DataStorage::ICoreProvider& DataStorage)
{
	DataStorage.OnUpdateCompleted().RemoveAll(this);
}

void UTypeInfoFactory::OnDatabaseUpdateCompleted()
{
	if (bRefreshTypeInfoQueued)
	{
		PopulateTypeInfo<UClass>();
		bRefreshTypeInfoQueued = false;
	}
}

void UTypeInfoFactory::ClearAllTypeInfo()
{
	using namespace UE::Editor::DataStorage;
	ClearTypeInfoByTag<FTypeInfoTag>();
	ClearTypeInfoByTag<FPropertyInfoTag>();
}

void UTypeInfoFactory::RefreshAllTypeInfo()
{
	ClearAllTypeInfo();
	bRefreshTypeInfoQueued = true;
}

template <class Type>
void UTypeInfoFactory::PopulateTypeInfo()
{
	using namespace UE::Editor::DataStorage;
	ICoreProvider* DataStorage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName);
	checkf(DataStorage, TEXT("Attempted to add type info to Teds with a null DataStorage"));

	// Block adding anything to the data table if our CVar is off
	static const auto TedsTypeInfoEnabledCVar = IConsoleManager::Get().FindConsoleVariable(TEXT("TEDS.Feature.TypeInfoIntegration.Enable"));
	if (TedsTypeInfoEnabledCVar && !TedsTypeInfoEnabledCVar->GetBool())
	{
		return;
	}

	bool bOnlyAddVerseProperties = CVarTEDSTypeInfoOnlyVerseProperties.GetValueOnGameThread();

	TMap<Type*, UE::Editor::DataStorage::RowHandle> TypesWithPendingPropertyInfo;
	UE::Editor::DataStorage::RowHandle OutRowhandle = UE::Editor::DataStorage::InvalidRowHandle;
	for (TObjectIterator<Type> TypeIterator; TypeIterator; ++TypeIterator)
	{
		Type* CurrentType = *TypeIterator;
		if (CurrentType && TryAddTypeInfo(*DataStorage, CurrentType, OutRowhandle))
		{
			if (!bOnlyAddVerseProperties || Cast<UVerseClass>(CurrentType))
			{
				TypesWithPendingPropertyInfo.Emplace(CurrentType, OutRowhandle);
			}
		}
	}

	// Separate type info pass from property info pass. We can lookup types during property pass and time slice properties in the future
	for (const TPair<Type*, UE::Editor::DataStorage::RowHandle> TypeWithPendingPropertyInfo : TypesWithPendingPropertyInfo)
	{
		AddPropertyInfo(*DataStorage, TypeWithPendingPropertyInfo.Value, TypeWithPendingPropertyInfo.Key);
	}

	DataStorage->ActivateQueries(TypeInfo::ClassSetupActionName);
	DataStorage->ActivateQueries(TypeInfo::PostClassSetupActionName);
}

bool UTypeInfoFactory::TryAddTypeInfo(UE::Editor::DataStorage::ICoreProvider& DataStorage, const UStruct* InTypeInfo,
	UE::Editor::DataStorage::RowHandle& OutTypeRowHandle)
{
	// Do not include InstanceDataObject types in TEDS
	if (UE::IsClassOfInstanceDataObjectClass(InTypeInfo))
	{
		return false;
	}

	using namespace UE::Editor::DataStorage;

	ICompatibilityProvider* Compatibility = GetMutableDataStorageFeature<ICompatibilityProvider>(CompatibilityFeatureName);
	checkf(Compatibility, TEXT("Attempted to add type info to Teds with a null CompatibilityProvider"));

	// We have 2 major type cases: Classes and Structs
	
	// Case 1: Classes and Verse Classes
	const UClass* ClassInfo = Cast<UClass>(InTypeInfo);
	if (ClassInfo && FilterClassInfo(ClassInfo))
	{
		if (RowHandle TypeRowHandle = Compatibility->AddCompatibleObject(const_cast<UStruct*>(InTypeInfo)))
		{
			AddCommonColumns(DataStorage, TypeRowHandle, ClassInfo);
			AddClassColumns(DataStorage, TypeRowHandle, ClassInfo);

			// Only Classes can be UVerseClasses
			if (const UVerseClass* VerseClassInfo = Cast<UVerseClass>(ClassInfo))
			{
				AddVerseColumns(DataStorage, TypeRowHandle, VerseClassInfo);
				AddIdoColumns(DataStorage, TypeRowHandle, VerseClassInfo);
			}

			DataStorage.MapRow(TypeInfo::TypeMappingDomain, FMapKey(ClassInfo), TypeRowHandle);
			OutTypeRowHandle = TypeRowHandle;

			return true;
		}
	}

	// Case 2: Structs
	// Currently unsupported as filter will return false
	const UScriptStruct* StructData = Cast<UScriptStruct>(InTypeInfo);
	if (StructData && FilterStructInfo(StructData))
	{
		if (RowHandle TypeRowHandle = Compatibility->AddCompatibleObject(const_cast<UStruct*>(InTypeInfo)))
		{
			AddCommonColumns(DataStorage, TypeRowHandle, StructData);
			AddStructColumns(DataStorage, TypeRowHandle, StructData);

			DataStorage.MapRow(TypeInfo::TypeMappingDomain, FMapKey(StructData), TypeRowHandle);
			OutTypeRowHandle = TypeRowHandle;

			return true;
		}
	}

	return false;
}

void UTypeInfoFactory::AddPropertyInfo(UE::Editor::DataStorage::ICoreProvider& DataStorage, UE::Editor::DataStorage::RowHandle OwnerTypeRowHandle, const UStruct* InTypeInfo)
{
	using namespace UE::Editor::DataStorage;
	checkf(InTypeInfo, TEXT("Attempted to add property info row(s) with null TypeInfo"));

	// Use a TFieldIterator with IncludeSuper and IncludeInterface properties default off
	// We only want to store the immediate properties per class and will reference interface and super through class hierarchy
	for (TFieldIterator<FProperty> CurrentPropertyIt(InTypeInfo); CurrentPropertyIt; ++CurrentPropertyIt)
	{
		if (RowHandle PropertyRowHandle = DataStorage.AddRow(PropertyTable))
		{
			// TODO: Cleanup property rows when associated type row is destroyed
			// We get away with it for now because Verse Properties are destroyed during compiles and we refresh the whole type info table then
			DataStorage.SetParentRow(PropertyHierarchyHandle, PropertyRowHandle, OwnerTypeRowHandle);

			if (Cast<UVerseClass>(InTypeInfo))
			{
				DataStorage.AddColumn<FVersePropertyTag>(PropertyRowHandle);
			}

			AddCommonPropertyColumns(DataStorage, OwnerTypeRowHandle, PropertyRowHandle, *CurrentPropertyIt);
		}
	}
}

bool UTypeInfoFactory::FilterStructInfo(const UStruct* /*StructInfo*/)
{
	// Structs not supported currently
	return false;
}

bool UTypeInfoFactory::FilterClassInfo(const UClass* ClassInfo)
{
	if (!ClassInfo)
	{
		return false;
	}

	FClassViewerModule& ClassViewerModule = FModuleManager::LoadModuleChecked<FClassViewerModule>("ClassViewer");
	const TSharedPtr<IClassViewerFilter>& GlobalClassFilter = ClassViewerModule.GetGlobalClassViewerFilter();
	TSharedRef<FClassViewerFilterFuncs> ClassFilterFuncs = ClassViewerModule.CreateFilterFuncs();

	FClassViewerInitializationOptions ClassViewerOptions = {};
	const bool bPassesClassViewerFilter = !GlobalClassFilter.IsValid() || GlobalClassFilter->IsClassAllowed(ClassViewerOptions, ClassInfo, ClassFilterFuncs);

	return bPassesClassViewerFilter;
}

void UTypeInfoFactory::AddCommonColumns(UE::Editor::DataStorage::ICoreProvider& DataStorage, UE::Editor::DataStorage::RowHandle TypeRowHandle, const UStruct* InTypeInfo)
{
	using namespace UE::Editor::DataStorage;
	checkf(InTypeInfo, TEXT("Attempted to add type info columns with null TypeInfo"));

	DataStorage.AddColumn(TypeRowHandle, FTypedElementLabelColumn{ .Label = InTypeInfo->GetDisplayNameText().ToString() });
	DataStorage.AddColumn(TypeRowHandle, FTypeInfoObjectPathColumn{ .ObjectPath = InTypeInfo->GetPathName() });
	DataStorage.AddColumn(TypeRowHandle, FTypeInfoVersePathColumn{ .VersePath = InTypeInfo->GetVersePath().AsText() });
	DataStorage.AddColumns<FTypeInfoTag, FTypeInfoRequiresHierarchyUpdateTag>(TypeRowHandle);
}

void UTypeInfoFactory::AddStructColumns(UE::Editor::DataStorage::ICoreProvider& DataStorage, UE::Editor::DataStorage::RowHandle TypeRowHandle, const UScriptStruct* InStructInfo)
{
	using namespace UE::Editor::DataStorage;
	checkf(InStructInfo, TEXT("Attempted to add type info columns with null StructInfo"));

	DataStorage.AddColumn(TypeRowHandle, FTypedElementScriptStructTypeInfoColumn{ .TypeInfo = InStructInfo });
	DataStorage.AddColumn<FStructTypeInfoTag>(TypeRowHandle);
}

void UTypeInfoFactory::AddClassColumns(UE::Editor::DataStorage::ICoreProvider& DataStorage, UE::Editor::DataStorage::RowHandle TypeRowHandle, const UClass* InClassInfo)
{
	using namespace UE::Editor::DataStorage;
	checkf(InClassInfo, TEXT("Attempted to add type info columns with null ClassInfo"));

	DataStorage.AddColumn(TypeRowHandle, FTypedElementClassTypeInfoColumn{ .TypeInfo = InClassInfo });

	if (InClassInfo->HasAnyClassFlags(CLASS_Interface))
	{
		DataStorage.AddColumns<FClassTypeInfoTag, FTypeInfoInterfaceTag>(TypeRowHandle);
	}
	else
	{
		DataStorage.AddColumn<FClassTypeInfoTag>(TypeRowHandle);
	}
}

void UTypeInfoFactory::AddVerseColumns(UE::Editor::DataStorage::ICoreProvider& DataStorage, UE::Editor::DataStorage::RowHandle TypeRowHandle, const UVerseClass* InVerseTypeInfo)
{
	using namespace UE::Editor::DataStorage;
	checkf(InVerseTypeInfo, TEXT("Attempted to add type info columns with null VerseTypeInfo"));

	DataStorage.AddColumn<FVerseTypeInfoTag>(TypeRowHandle);

	if (InVerseTypeInfo->IsUniversallyAccessible())
	{
		DataStorage.AddColumn<FVerseTypeInfoAccessLevel>(TypeRowHandle, TEXT("Public"));
	}
	else if (InVerseTypeInfo->IsEpicInternal())
	{
		DataStorage.AddColumn<FVerseTypeInfoAccessLevel>(TypeRowHandle, TEXT("Epic_Internal"));
	}
}

void UTypeInfoFactory::AddIdoColumns(UE::Editor::DataStorage::ICoreProvider& DataStorage, UE::Editor::DataStorage::RowHandle TypeRowHandle, const UVerseClass* InVerseTypeInfo)
{
	using namespace UE::Editor::DataStorage;


	checkf(InVerseTypeInfo, TEXT("Attempted to add type info columns with null VerseTypeInfo"));

	// Try find an InstanceDataObject from type
	if (UE::FPropertyBagRepository::Get().FindFirstInstanceThatNeedsFixup(InVerseTypeInfo) != nullptr)
	{
		DataStorage.AddColumn<FHasUnknownPropertiesTag>(TypeRowHandle);

		TArray<UObject*> AllInstances;
		GetObjectsOfClass(InVerseTypeInfo, AllInstances, /*bIncludeDerivedClasses =*/ false);

		uint8 InstanceWithUnknownPropertiesCount = 0;
		for (UObject* Instance : AllInstances)
		{
			if (UE::FPropertyBagRepository::Get().RequiresFixup(Instance))
			{
				++InstanceWithUnknownPropertiesCount;
			}
		}

		const FText AlertMessagePattern = NSLOCTEXT("InstanceDataObjectTypes", "UnknownPropertiesAlertMessage",
			"Unknown properties found in {InstanceCount} {InstanceCount}|plural(one=instance, other=instances).");

		FFormatNamedArguments AlertMessageArgs;
		AlertMessageArgs.Add(TEXT("InstanceCount"), InstanceWithUnknownPropertiesCount);

		const FText AlertMessage = FText::Format(AlertMessagePattern, AlertMessageArgs);

		static FName AlertName("UnknownPropertiesAlert");

		Alerts::AddAlert(DataStorage, TypeRowHandle, AlertName, AlertMessage, Columns::FAlertColumnType::Warning);
	}
}

void UTypeInfoFactory::AddCommonPropertyColumns(UE::Editor::DataStorage::ICoreProvider& DataStorage, UE::Editor::DataStorage::RowHandle OwnerTypeRowHandle,
	UE::Editor::DataStorage::RowHandle PropertyRowHandle, const FProperty* InPropertyInfo)
{
	using namespace UE::Editor::DataStorage;

	checkf(InPropertyInfo, TEXT("Attempted to add property info columns with null PropertyInfo"));

	bool bNameWasMangled = false;
	FString PropertyName = UE::Editor::DataStorage::Internal::UnmanglePropertyName(InPropertyInfo->GetFName(), bNameWasMangled);

	UE::FPropertyTypeName PropertyTypeNameInfo(InPropertyInfo);
	DataStorage.AddColumn(PropertyRowHandle, FPropertyNameColumn{ .PropertyName = FText::FromString(PropertyName) });
	DataStorage.AddColumn(PropertyRowHandle, FPropertyOwnerTypeRowHandleColumn{ .PropertyOwnerTypeRowHandle = OwnerTypeRowHandle });
	DataStorage.AddColumn(PropertyRowHandle, FPropertyTypeNameInfoColumn{ .PropertyTypeNameInfo = PropertyTypeNameInfo });

	if (FFieldClass* PropertyClass = InPropertyInfo->GetClass())
	{
		DataStorage.AddColumn(PropertyRowHandle,
			FPropertyTypeNameColumn{ .PropertyTypeName = FText::FromString(PropertyTypeNameInfo.GetName().ToString()) });
	}

	if (!InPropertyInfo->HasAnyPropertyFlags(CPF_BlueprintReadOnly))
	{
		DataStorage.AddColumn<FPropertyMutableTag>(PropertyRowHandle);
	}

	DataStorage.AddColumn<FPropertyInfoTag>(PropertyRowHandle);
}

template <class TagToClear>
void UTypeInfoFactory::ClearTypeInfoByTag()
{
	using namespace UE::Editor::DataStorage;
	ICoreProvider* DataStorage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName);

	checkf(DataStorage, TEXT("Attempted to clear type info from Teds with a null DataStorage"));

	DataStorage->RemoveAllRowsWith<TagToClear>();
}
