// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "DataStorage/CommonTypes.h"

#include "DataStorage/Handles.h"
#include "Elements/Interfaces/TypedElementDataStorageFactory.h"
#include "UObject/ObjectMacros.h"

#include "TedsTypeInfoFactory.generated.h"

struct FSolBuildResults;

class UStruct;
class UScriptStruct;
class UClass;
class UVerseClass;

namespace UE::Editor::DataStorage
{
	class ICoreProvider;
	class ICompatibilityProvider;
	class IHierarchyAccessInterface;
} // UE::Editor::DataStorage

USTRUCT(meta = (DisplayName = "Row requires a pass to assign its type hierarchy info"))
struct FDataStorageTypeInfoRequiresHierarchyUpdateTag : public FEditorDataStorageTag
{
	GENERATED_BODY()
};

UCLASS()
class UTypeInfoFactory final : public UEditorDataStorageFactory
{
	GENERATED_BODY()

public:

	virtual ~UTypeInfoFactory() override = default;

	//~ Begin UEditorDataStorageFactory interface
	void PreRegister(UE::Editor::DataStorage::ICoreProvider& DataStorage) override;
	void RegisterTables(UE::Editor::DataStorage::ICoreProvider& DataStorage, UE::Editor::DataStorage::ICompatibilityProvider& DataStorageCompatability) override;
	void RegisterHierarchies(UE::Editor::DataStorage::ICoreProvider& DataStorage) override;
	void RegisterQueries(UE::Editor::DataStorage::ICoreProvider& DataStorage) override;
	void PreShutdown(UE::Editor::DataStorage::ICoreProvider& DataStorage) override;
	//~ End UEditorDataStorageFactory interface

	TEDSTYPEINFO_API void ClearAllTypeInfo();
	TEDSTYPEINFO_API void RefreshAllTypeInfo();

private:

	void OnDatabaseUpdateCompleted();

	template <class Type>
	void PopulateTypeInfo();

	template <class TagToClear>
	void ClearTypeInfoByTag();

	bool TryAddTypeInfo(UE::Editor::DataStorage::ICoreProvider& DataStorage, const UStruct* InTypeInfo, UE::Editor::DataStorage::RowHandle& OutTypeRowHandle);
	void AddPropertyInfo(UE::Editor::DataStorage::ICoreProvider& DataStorage, UE::Editor::DataStorage::RowHandle OwnerTypeRowHandle, const UStruct* InTypeInfo);

	bool FilterStructInfo(const UStruct* StructInfo);
	bool FilterClassInfo(const UClass* ClassInfo);

	void AddCommonColumns(UE::Editor::DataStorage::ICoreProvider& DataStorage, UE::Editor::DataStorage::RowHandle TypeRowHandle, const UStruct* InTypeInfo);
	void AddStructColumns(UE::Editor::DataStorage::ICoreProvider& DataStorage, UE::Editor::DataStorage::RowHandle TypeRowHandle, const UScriptStruct* InStructInfo);
	void AddClassColumns(UE::Editor::DataStorage::ICoreProvider& DataStorage, UE::Editor::DataStorage::RowHandle TypeRowHandle, const UClass* InClassInfo);
	void AddVerseColumns(UE::Editor::DataStorage::ICoreProvider& DataStorage, UE::Editor::DataStorage::RowHandle TypeRowHandle, const UVerseClass* InVerseTypeInfo);
	void AddIdoColumns(UE::Editor::DataStorage::ICoreProvider& DataStorage, UE::Editor::DataStorage::RowHandle TypeRowHandle, const UVerseClass* InVerseTypeInfo);

	void AddCommonPropertyColumns(UE::Editor::DataStorage::ICoreProvider& DataStorage, UE::Editor::DataStorage::RowHandle TypeRowhandle,
		UE::Editor::DataStorage::RowHandle PropertyRowhandle, const FProperty* InPropertyInfo);

	bool bRefreshTypeInfoQueued = false;

	UE::Editor::DataStorage::TableHandle PropertyTable = UE::Editor::DataStorage::InvalidTableHandle;
	UE::Editor::DataStorage::FHierarchyHandle PropertyHierarchyHandle;
};

namespace UE::Editor::DataStorage
{
	using FTypeInfoRequiresHierarchyUpdateTag = FDataStorageTypeInfoRequiresHierarchyUpdateTag;
}