// Copyright Epic Games, Inc. All Rights Reserved.

#include "TedsTypeInfoModule.h"

#include "HAL/IConsoleManager.h"
#include "Modules/ModuleManager.h"
#include "TedsTypeInfoFactory.h"

#include "DataStorage/Features.h"
#include "Elements/Interfaces/TypedElementDataStorageInterface.h"

namespace UE::Editor::DataStorage::TypeInfo
{
	namespace Private
	{
		TAutoConsoleVariable<bool> CVarTEDSTypeInfoEnabled(TEXT("TEDS.Feature.TypeInfoIntegration.Enable"),
			true,
			TEXT("When true we will store type info into TEDS and propagate type updates when they occur"),
			FConsoleVariableDelegate::CreateLambda([](IConsoleVariable* Variable)
				{
					const bool bIsEnabled = Variable->GetBool();
					FTedsTypeInfoModule& Module = FTedsTypeInfoModule::GetChecked();

					if (bIsEnabled)
					{
						Module.RefreshTypeInfo();
					}
					else
					{
						Module.FlushTypeInfo();
					}
				}));
	}
	void FTedsTypeInfoModule::StartupModule()
	{
		FModuleManager::Get().LoadModule(TEXT("TypedElementFramework"));
		FModuleManager::Get().LoadModule(TEXT("EditorDataStorageHierarchy"));

		UE::Editor::DataStorage::OnEditorDataStorageFeaturesEnabled().AddLambda([this]()
			{
				bDataStorageFeaturesEnabled = true;
			});
	}

	void FTedsTypeInfoModule::ShutdownModule()
	{
		IModuleInterface::ShutdownModule();
	}

	FTedsTypeInfoModule* FTedsTypeInfoModule::Get()
	{
		return FModuleManager::Get().LoadModulePtr<FTedsTypeInfoModule>(TEXT("TedsTypeInfo"));
	}

	FTedsTypeInfoModule& FTedsTypeInfoModule::GetChecked()
	{
		return FModuleManager::Get().LoadModuleChecked<FTedsTypeInfoModule>(TEXT("TedsTypeInfo"));
	}

	void FTedsTypeInfoModule::EnableTedsTypeInfoIntegration()
	{
		Private::CVarTEDSTypeInfoEnabled.AsVariable()->Set(true);
		RefreshTypeInfo();
	}

	void FTedsTypeInfoModule::DisableTedsTypeInfoIntegration()
	{
		FlushTypeInfo();
		Private::CVarTEDSTypeInfoEnabled.AsVariable()->Set(false);
	}

	bool FTedsTypeInfoModule::IsTedsTypeInfoIntegrationEnabled()
	{
		return Private::CVarTEDSTypeInfoEnabled.GetValueOnGameThread();
	}

	void FTedsTypeInfoModule::FlushTypeInfo()
	{
		using namespace UE::Editor::DataStorage;

		if (!IsTedsTypeInfoIntegrationEnabled())
		{
			return;
		}

		if (!bDataStorageFeaturesEnabled || bTypeInfoFactoryEnabled)
		{
			return;
		}

		ICoreProvider* DataStorage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName);

		ensure(DataStorage);

		UTypeInfoFactory* TypeInfoFactory = DataStorage->FindFactory<UTypeInfoFactory>();
		ensure(TypeInfoFactory);

		TypeInfoFactory->ClearAllTypeInfo();
	}

	void FTedsTypeInfoModule::RefreshTypeInfo()
	{
		using namespace UE::Editor::DataStorage;

		if (!IsTedsTypeInfoIntegrationEnabled())
		{
			return;
		}

		if (!bDataStorageFeaturesEnabled || !bTypeInfoFactoryEnabled)
		{
			return;
		}

		ICoreProvider* DataStorage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName);

		ensure(DataStorage);

		UTypeInfoFactory* TypeInfoFactory = DataStorage->FindFactory<UTypeInfoFactory>();
		ensure(TypeInfoFactory);

		TypeInfoFactory->RefreshAllTypeInfo();
	}
} // namespace UE::Editor::TypeData

IMPLEMENT_MODULE(UE::Editor::DataStorage::TypeInfo::FTedsTypeInfoModule, TedsTypeInfo);
