// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "DataStorage/CommonTypes.h"
#include "DataStorage/Handles.h"
#include "Elements/Columns/TypedElementTypeInfoColumns.h"

#include "TedsTypeInfoColumns.generated.h"

// Type Info Columns Start
USTRUCT(meta = (DisplayName = "Row is a type"))
struct FDataStorageTypeInfoTag : public FEditorDataStorageTag
{
	GENERATED_BODY()
};

USTRUCT(meta = (DisplayName = "Row type is a class"))
struct FDataStorageClassTypeInfoTag : public FEditorDataStorageTag
{
	GENERATED_BODY()
};

USTRUCT(meta = (DisplayName = "Row type is a struct"))
struct FDataStorageStructTypeInfoTag : public FEditorDataStorageTag
{
	GENERATED_BODY()
};

USTRUCT(meta = (DisplayName = "Row type is an Interface"))
struct FDataStorageTypeInfoInterfaceTag : public FEditorDataStorageTag
{
	GENERATED_BODY()
};

USTRUCT(meta = (DisplayName = "Row type is a Verse type"))
struct FDataStorageVerseTypeInfoTag : public FEditorDataStorageTag
{
	GENERATED_BODY()
};

USTRUCT(meta = (DisplayName = "Row type is virtualized"))
struct FDataStorageVirtualTypeInfoTag : public FEditorDataStorageTag
{
	GENERATED_BODY()
};

USTRUCT(meta = (DisplayName = "Type Object Path"))
struct FDataStorageTypeInfoObjectPathColumn final : public FEditorDataStorageColumn
{
	GENERATED_BODY()

	UPROPERTY(meta = (Searchable, Sortable))
	FString ObjectPath;
};

USTRUCT(meta = (DisplayName = "Type Verse Path"))
struct FDataStorageTypeInfoVersePathColumn final : public FEditorDataStorageColumn
{
	GENERATED_BODY()

	UPROPERTY(meta = (Searchable, Sortable))
	FText VersePath;
};

USTRUCT(meta = (DisplayName = "Verse type access level", EditorDataStorage_DynamicColumnTemplate))
struct FDataStorageVerseTypeInfoAccessLevel : public FEditorDataStorageTag
{
	GENERATED_BODY()
};

USTRUCT(meta = (DisplayName = "Row type has unknown properties"))
struct FDataStorageHasUnknownPropertiesTag : public FEditorDataStorageTag
{
	GENERATED_BODY()
};
// Type Info Columns End

// Property Info Columns Start
USTRUCT(meta = (DisplayName = "Property Name"))
struct FDataStoragePropertyNameColumn final : public FEditorDataStorageColumn
{
	GENERATED_BODY()

	UPROPERTY(meta = (Searchable, Sortable))
	FText PropertyName;
};

USTRUCT(meta = (DisplayName = "Property Owner Type Row Handle"))
struct FDataStoragePropertyOwnerTypeRowHandleColumn final : public FEditorDataStorageColumn
{
	GENERATED_BODY()

	UE::Editor::DataStorage::RowHandle PropertyOwnerTypeRowHandle = UE::Editor::DataStorage::InvalidRowHandle;
};

USTRUCT(meta = (DisplayName = "Property Type Name"))
struct FDataStoragePropertyTypeNameColumn final : public FEditorDataStorageColumn
{
	GENERATED_BODY()

	UPROPERTY(meta = (Searchable, Sortable))
	FText PropertyTypeName;
};

USTRUCT(meta = (DisplayName = "Property Type Name Info"))
struct FDataStoragePropertyTypeNameInfoColumn final : public FEditorDataStorageColumn
{
	GENERATED_BODY()

	UE::FPropertyTypeName PropertyTypeNameInfo;
};

USTRUCT(meta = (DisplayName = "Property is mutable"))
struct FDataStoragePropertyMutableTag : public FEditorDataStorageTag
{
	GENERATED_BODY()
};

USTRUCT(meta = (DisplayName = "Row is a property"))
struct FDataStoragePropertyInfoTag : public FEditorDataStorageTag
{
	GENERATED_BODY()
};

USTRUCT(meta = (DisplayName = "Property is owned by a Verse type"))
struct FDataStorageVersePropertyTag : public FEditorDataStorageTag
{
	GENERATED_BODY()
};
// Property Info Columns End

namespace UE::Editor::DataStorage
{
	// Type Info Columns Start
	using FTypeInfoTag = FDataStorageTypeInfoTag;
	using FClassTypeInfoTag = FDataStorageClassTypeInfoTag;
	using FStructTypeInfoTag = FDataStorageStructTypeInfoTag;
	using FTypeInfoInterfaceTag = FDataStorageTypeInfoInterfaceTag;
	using FTypeInfoObjectPathColumn = FDataStorageTypeInfoObjectPathColumn;
	using FTypeInfoVersePathColumn = FDataStorageTypeInfoVersePathColumn;
	using FVerseTypeInfoTag = FDataStorageVerseTypeInfoTag;
	using FVirtualTypeInfoTag = FDataStorageVirtualTypeInfoTag;
	using FVerseTypeInfoAccessLevel = FDataStorageVerseTypeInfoAccessLevel;
	using FHasUnknownPropertiesTag = FDataStorageHasUnknownPropertiesTag;
	// Type Info Tags End

	// Property Info Columns Start
	using FPropertyNameColumn = FDataStoragePropertyNameColumn;
	using FPropertyOwnerTypeRowHandleColumn = FDataStoragePropertyOwnerTypeRowHandleColumn;
	using FPropertyTypeNameColumn = FDataStoragePropertyTypeNameColumn;
	using FPropertyTypeNameInfoColumn = FDataStoragePropertyTypeNameInfoColumn;
	using FPropertyMutableTag = FDataStoragePropertyMutableTag;
	using FPropertyInfoTag = FDataStoragePropertyInfoTag;
	using FVersePropertyTag = FDataStorageVersePropertyTag;
	// Property Info Tags End
}

namespace UE::Editor::DataStorage::TypeInfo
{
	static const FName TypeMappingDomain = "TypeMappingDomain";
	static const FName TypeTableName = "Type Information";
	static const FName PropertyTableName = "Property Information";
	static const FName ClassHierarchyName = "ClassHierarchy";
	static const FName PropertyHierarchyName = "PropertyHierarchy";
}