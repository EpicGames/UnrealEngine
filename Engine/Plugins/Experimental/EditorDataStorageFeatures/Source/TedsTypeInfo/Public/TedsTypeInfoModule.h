// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"

namespace UE::Editor::DataStorage::TypeInfo
{
	class FTedsTypeInfoModule : public IModuleInterface
	{
	public:
		virtual void StartupModule() override;
		virtual void ShutdownModule() override;

		TEDSTYPEINFO_API static FTedsTypeInfoModule* Get();
		TEDSTYPEINFO_API static FTedsTypeInfoModule& GetChecked();

		TEDSTYPEINFO_API void EnableTedsTypeInfoIntegration();
		TEDSTYPEINFO_API void DisableTedsTypeInfoIntegration();
		TEDSTYPEINFO_API bool IsTedsTypeInfoIntegrationEnabled();

		TEDSTYPEINFO_API void RefreshTypeInfo();
		TEDSTYPEINFO_API void FlushTypeInfo();

		void SetTypeInfoFactoryEnabled() { bTypeInfoFactoryEnabled = true; }

	private:

		bool bTypeInfoFactoryEnabled = false;
		bool bDataStorageFeaturesEnabled = false;
	};
}