// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "UObject/NameTypes.h"

namespace UE::Editor::DataStorage::TypeInfo
{
	static const FName ClassSetupActionName = TEXT("ClassSetupAction");
	static const FName PostClassSetupActionName = TEXT("PostClassSetupAction");
}