// Copyright Epic Games, Inc. All Rights Reserved.

#include "TedsTypedElementActorHandleFactory.h"

#include "Columns/TedsActorComponentCompatibilityColumns.h"
#include "TedsTypedElementBridge/TedsTypedElementBridgeCapabilities.h"
#include "Elements/Columns/TypedElementHandleColumn.h"
#include "DataStorage/Features.h"
#include "Elements/Columns/TypedElementCompatibilityColumns.h"
#include "Elements/Framework/EngineElementsLibrary.h"
#include "Elements/Framework/TypedElementList.h"
#include "Elements/Framework/TypedElementQueryBuilder.h"
#include "Elements/Interfaces/TypedElementDataStorageInterface.h"
#include "GameFramework/Actor.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TedsTypedElementActorHandleFactory)

void UTypedElementActorHandleDataStorageFactory::PreRegister(UE::Editor::DataStorage::ICoreProvider& DataStorage)
{
	Super::PreRegister(DataStorage);
	
	BridgeEnableDelegateHandle = UE::Editor::DataStorage::Compatibility::OnTypedElementBridgeEnabled().AddUObject(this, &UTypedElementActorHandleDataStorageFactory::HandleBridgeEnabled);
}

void UTypedElementActorHandleDataStorageFactory::PreShutdown(UE::Editor::DataStorage::ICoreProvider& DataStorage)
{
	UE::Editor::DataStorage::Compatibility::OnTypedElementBridgeEnabled().Remove(BridgeEnableDelegateHandle);
	BridgeEnableDelegateHandle.Reset();
}

void UTypedElementActorHandleDataStorageFactory::RegisterQueries(UE::Editor::DataStorage::ICoreProvider& DataStorage)
{
	Super::RegisterQueries(DataStorage);

	if (UE::Editor::DataStorage::Compatibility::IsTypedElementBridgeEnabled())
	{
		RegisterQuery_ActorHandlePopulate(DataStorage);
		RegisterQuery_ActorComponentHandlePopulate(DataStorage);
	}

	using namespace UE::Editor::DataStorage::Queries;
	GetAllActorsQuery = DataStorage.RegisterQuery(
	Select()
		.ReadOnly<FTypedElementUObjectColumn>()
	.Where()
		.All<FTypedElementActorTag>()
	.Compile());

	GetAllActorComponentsQuery = DataStorage.RegisterQuery(
	Select()
		.ReadOnly<FTypedElementUObjectColumn>()
	.Where(TColumn<FActorComponentTypeTag>())
	.Compile());
}

void UTypedElementActorHandleDataStorageFactory::RegisterQuery_ActorHandlePopulate(UE::Editor::DataStorage::ICoreProvider& DataStorage)
{
	using namespace UE::Editor::DataStorage::Queries;

	if (!ensureMsgf(ActorHandlePopulateQuery == InvalidQueryHandle, TEXT("Already registered query")))
	{
		return;
	}
	
	ActorHandlePopulateQuery = DataStorage.RegisterQuery(
	Select(TEXT("Populate actor typed element handles"),
		FObserver::OnAdd<FTypedElementUObjectColumn>(),
		[](IQueryContext& Context, RowHandle Row, const FTypedElementUObjectColumn& ObjectColumn)
		{
			if (UObject* Object = ObjectColumn.Object.Get())
			{
				if (!ensureMsgf(Cast<AActor>(Object), TEXT("AActor cast was unsuccessful despite being in the Actor Table")))
				{
					return;
				}
				const FTypedElementHandle Handle = UEngineElementsLibrary::AcquireEditorActorElementHandle(static_cast<AActor*>(Object));
				Context.AddColumn(Row, Compatibility::FTypedElementColumn
				{
					.Handle = Handle
				});
			}
		})
	.Where()
		.All<FTypedElementActorTag>()
	.Compile());
}

void UTypedElementActorHandleDataStorageFactory::RegisterQuery_ActorComponentHandlePopulate(UE::Editor::DataStorage::ICoreProvider& DataStorage)
{
	using namespace UE::Editor::DataStorage::Queries;

	if (!ensureMsgf(ActorComponentHandlePopulateQuery == InvalidQueryHandle, TEXT("Already registered query")))
	{
		return;
	}
	
	ActorComponentHandlePopulateQuery = DataStorage.RegisterQuery(
		Select(TEXT("Populate actor component typed element handles"),
			FObserver::OnAdd<FTypedElementUObjectColumn>(),
			[](IQueryContext& Context, RowHandle Row, const FTypedElementUObjectColumn& ObjectColumn)
			{
				if (UObject* Object = ObjectColumn.Object.Get())
				{
					if (!ensureMsgf(Cast<UActorComponent>(Object), TEXT("UActorComponent cast was unsuccessful despite being in the Actor Component Table")))
					{
						return;
					}
					const FTypedElementHandle Handle = UEngineElementsLibrary::AcquireEditorComponentElementHandle(static_cast<UActorComponent*>(Object));
					Context.AddColumn(Row, Compatibility::FTypedElementColumn
					{
						.Handle = Handle
					});
				}
			})
		.Where(TColumn<FActorComponentTypeTag>())
		.Compile());
}

static void PopulateElementHandles(
	UE::Editor::DataStorage::ICoreProvider& DataStorage,
	UE::Editor::DataStorage::QueryHandle Query,
	TFunctionRef<FTypedElementHandle(const FTypedElementUObjectColumn&)> GetHandle)
{
	using namespace UE::Editor::DataStorage;
	using namespace Queries;

	TArray<RowHandle> RowHandles;
	TArray<TWeakObjectPtr<UObject>> Objects;

	DataStorage.RunQuery(Query, CreateDirectQueryCallbackBinding(
		[&RowHandles, &Objects](IDirectQueryContext& Context, const FTypedElementUObjectColumn* Fragments)
		{
			RowHandles.Append(Context.GetRowHandles());
			for (const FTypedElementUObjectColumn& Fragment : TConstArrayView<const FTypedElementUObjectColumn>(Fragments, Context.GetRowCount()))
			{
				Objects.Add(Fragment.Object);
			}
		}));

	for (int32 Index = 0; Index < RowHandles.Num(); ++Index)
	{
		if (FTypedElementHandle Handle = GetHandle(FTypedElementUObjectColumn{.Object = Objects[Index]}))
		{
			DataStorage.AddColumn(RowHandles[Index], Compatibility::FTypedElementColumn{.Handle = Handle});
		}
	}
}

void UTypedElementActorHandleDataStorageFactory::HandleBridgeEnabled(bool bEnabled)
{
	using namespace UE::Editor::DataStorage;

	ICoreProvider* DataStorage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName);

	if (bEnabled)
	{
		PopulateElementHandles(*DataStorage, GetAllActorsQuery,
			[](const FTypedElementUObjectColumn& Column) -> FTypedElementHandle
			{
				const AActor* Actor = Cast<AActor>(Column.Object);
				return Actor ? UEngineElementsLibrary::AcquireEditorActorElementHandle(Actor) : FTypedElementHandle{};
			});
		RegisterQuery_ActorHandlePopulate(*DataStorage);

		PopulateElementHandles(*DataStorage, GetAllActorComponentsQuery,
			[](const FTypedElementUObjectColumn& Column) -> FTypedElementHandle
			{
				const UActorComponent* Component = Cast<UActorComponent>(Column.Object);
				return Component ? UEngineElementsLibrary::AcquireEditorComponentElementHandle(Component) : FTypedElementHandle{};
			});
		RegisterQuery_ActorComponentHandlePopulate(*DataStorage);
	}
	else
	{
		DataStorage->UnregisterQuery(ActorHandlePopulateQuery);
		DataStorage->UnregisterQuery(ActorComponentHandlePopulateQuery);
		ActorHandlePopulateQuery = InvalidQueryHandle;
		ActorComponentHandlePopulateQuery = InvalidQueryHandle;
	}
}

