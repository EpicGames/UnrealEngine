// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/Framework/TypedElementHandle.h"
#include "Elements/Interfaces/TypedElementDataStorageInterface.h"
#include "Elements/Interfaces/TypedElementDataStorageFactory.h"
#include "UObject/UObjectGlobals.h"

#include "TedsTypedElementActorHandleFactory.generated.h"

class UTypedElementRegistry;

/**
 * This class is responsible for acquiring and registering Actor Typed Element Handles
 * with TEDS/
 */
UCLASS(Transient)
class UTypedElementActorHandleDataStorageFactory : public UEditorDataStorageFactory
{
	GENERATED_BODY()
public:

	virtual void PreRegister(UE::Editor::DataStorage::ICoreProvider& DataStorage) override;
	virtual void RegisterQueries(UE::Editor::DataStorage::ICoreProvider& DataStorage) override;
	virtual void PreShutdown(UE::Editor::DataStorage::ICoreProvider& DataStorage) override;
private:
	virtual void RegisterQuery_ActorHandlePopulate(UE::Editor::DataStorage::ICoreProvider& DataStorage);
	virtual void RegisterQuery_ActorComponentHandlePopulate(UE::Editor::DataStorage::ICoreProvider& DataStorage);
	void HandleBridgeEnabled(bool bEnabled);

	FDelegateHandle BridgeEnableDelegateHandle;
	UE::Editor::DataStorage::QueryHandle ActorHandlePopulateQuery = UE::Editor::DataStorage::InvalidQueryHandle;
	UE::Editor::DataStorage::QueryHandle GetAllActorsQuery = UE::Editor::DataStorage::InvalidQueryHandle;
	UE::Editor::DataStorage::QueryHandle ActorComponentHandlePopulateQuery = UE::Editor::DataStorage::InvalidQueryHandle;
    UE::Editor::DataStorage::QueryHandle GetAllActorComponentsQuery = UE::Editor::DataStorage::InvalidQueryHandle;
};


