// Copyright Epic Games, Inc. All Rights Reserved.

#include "TedsTypedElementBridge/TedsTypedElementBridgeCapabilities.h"
#include "TedsTypedElementBridge/TedsTypedElementBridgeQueries.h"

#include "HAL/IConsoleManager.h"

namespace UE::Editor::DataStorage::Compatibility
{
	FOnTypedElementBridgeEnabled& OnTypedElementBridgeEnabled()
	{
		static FOnTypedElementBridgeEnabled OnBridgeEnabled;
		return OnBridgeEnabled;
	}

	bool IsTypedElementBridgeEnabled()
	{
		return UTypedElementBridgeDataStorageFactory::IsEnabled();
	}
} // namespace UE::Editor::DataStorage::Compatibility
