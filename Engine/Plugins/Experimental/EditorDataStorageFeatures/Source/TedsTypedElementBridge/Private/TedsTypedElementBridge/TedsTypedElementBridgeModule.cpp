// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modules/ModuleManager.h"
#include "Modules/ModuleInterface.h"

class FTedsTypedElementBridgeModule : public IModuleInterface
{
public:

	virtual void StartupModule() override
	{
	}

	virtual void ShutdownModule() override
	{
	}
};

IMPLEMENT_MODULE(FTedsTypedElementBridgeModule, TedsTypedElementBridge);

