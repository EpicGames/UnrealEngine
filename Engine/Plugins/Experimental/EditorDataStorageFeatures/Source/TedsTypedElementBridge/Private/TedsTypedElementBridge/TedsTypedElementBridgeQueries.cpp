// Copyright Epic Games, Inc. All Rights Reserved.

#include "TedsTypedElementBridge/TedsTypedElementBridgeQueries.h"

#include "TedsTypedElementBridge/TedsTypedElementBridgeCapabilities.h"
#include "Elements/Columns/TypedElementHandleColumn.h"
#include "DataStorage/Features.h"
#include "Elements/Columns/TypedElementCompatibilityColumns.h"
#include "Elements/Framework/EngineElementsLibrary.h"
#include "Elements/Framework/TypedElementList.h"
#include "Elements/Framework/TypedElementQueryBuilder.h"
#include "Elements/Interfaces/TypedElementDataStorageInterface.h"
#include "HAL/IConsoleManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TedsTypedElementBridgeQueries)

namespace UE::Editor::DataStorage::Compatibility::Private
{
	bool bBridgeEnabled = true;
	FAutoConsoleVariableRef CVarBridgeEnabled(
		TEXT("TEDS.TypedElementBridge.Enable"),
		bBridgeEnabled,
		TEXT("Automatically populated TEDS with TypedElementHandles"));
} // namespace UE::Editor::DataStorage::Compatibility::Private

uint8 UTypedElementBridgeDataStorageFactory::GetOrder() const
{
	return 110;
}

void UTypedElementBridgeDataStorageFactory::PreRegister(UE::Editor::DataStorage::ICoreProvider& DataStorage)
{
	Super::PreRegister(DataStorage);
	DebugEnabledDelegateHandle = UE::Editor::DataStorage::Compatibility::Private::CVarBridgeEnabled->OnChangedDelegate().AddUObject(this, &UTypedElementBridgeDataStorageFactory::HandleOnEnabled);
}

void UTypedElementBridgeDataStorageFactory::PreShutdown(UE::Editor::DataStorage::ICoreProvider& DataStorage)
{
	UE::Editor::DataStorage::Compatibility::Private::CVarBridgeEnabled->OnChangedDelegate().Remove(DebugEnabledDelegateHandle);
	DebugEnabledDelegateHandle.Reset();
	CleanupTypedElementColumns(DataStorage);

	Super::PreShutdown(DataStorage);
}

void UTypedElementBridgeDataStorageFactory::RegisterQueries(UE::Editor::DataStorage::ICoreProvider& DataStorage)
{
	Super::RegisterQueries(DataStorage);

	if (IsEnabled())
	{
		RegisterQuery_NewUObject(DataStorage);
	}
}

bool UTypedElementBridgeDataStorageFactory::IsEnabled()
{
	return UE::Editor::DataStorage::Compatibility::Private::CVarBridgeEnabled->GetBool();
}

void UTypedElementBridgeDataStorageFactory::RegisterQuery_NewUObject(UE::Editor::DataStorage::ICoreProvider& DataStorage)
{
	using namespace UE::Editor::DataStorage::Queries;
	
	RemoveTypedElementRowHandleQuery = DataStorage.RegisterQuery(
		Select()
			.ReadOnly<UE::Editor::DataStorage::Compatibility::FTypedElementColumn>()
		.Compile());
}

void UTypedElementBridgeDataStorageFactory::UnregisterQuery_NewUObject(UE::Editor::DataStorage::ICoreProvider& DataStorage)
{
	DataStorage.UnregisterQuery(RemoveTypedElementRowHandleQuery);
}

void UTypedElementBridgeDataStorageFactory::CleanupTypedElementColumns(UE::Editor::DataStorage::ICoreProvider& DataStorage)
{
	using namespace UE::Editor::DataStorage::Queries;

	// Remove any TEv1 handles
	{
		TArray<RowHandle> Handles;
		DataStorage.RunQuery(
			RemoveTypedElementRowHandleQuery,
			CreateDirectQueryCallbackBinding(
				[&Handles](IDirectQueryContext& Context, const RowHandle*)
				{
					Handles.Append(Context.GetRowHandles());
				}));
		
		DataStorage.BatchAddRemoveColumns(TConstArrayView<RowHandle>(Handles), {}, {Compatibility::FTypedElementColumn::StaticStruct()});
	}
}

void UTypedElementBridgeDataStorageFactory::HandleOnEnabled(IConsoleVariable* CVar)
{
	using namespace UE::Editor::DataStorage;
	ICoreProvider* DataStorage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName);
	bool bIsEnabled = CVar->GetBool();

	if (bIsEnabled)
	{
		RegisterQuery_NewUObject(*DataStorage);
		Compatibility::OnTypedElementBridgeEnabled().Broadcast(bIsEnabled);
	}
	else
	{
		Compatibility::OnTypedElementBridgeEnabled().Broadcast(bIsEnabled);
		CleanupTypedElementColumns(*DataStorage);
		UnregisterQuery_NewUObject(*DataStorage);
	}
}
