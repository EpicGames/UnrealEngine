// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/Framework/TypedElementHandle.h"
#include "Elements/Interfaces/TypedElementDataStorageInterface.h"
#include "Elements/Interfaces/TypedElementDataStorageFactory.h"
#include "UObject/UObjectGlobals.h"

#include "TedsTypedElementBridgeQueries.generated.h"

/**
 * This class is responsible for running queries that will ensure Typed Element Handles
 * are cleaned up when TEDS is shut down.
 */
UCLASS(Transient)
class UTypedElementBridgeDataStorageFactory : public UEditorDataStorageFactory
{
	GENERATED_BODY()
public:
	~UTypedElementBridgeDataStorageFactory() override = default;
	virtual uint8 GetOrder() const override;
	virtual void PreRegister(UE::Editor::DataStorage::ICoreProvider& DataStorage) override;
	virtual void PreShutdown(UE::Editor::DataStorage::ICoreProvider& DataStorage) override;
	virtual void RegisterQueries(UE::Editor::DataStorage::ICoreProvider& DataStorage) override;

	static bool IsEnabled();

private:
	void RegisterQuery_NewUObject(UE::Editor::DataStorage::ICoreProvider& DataStorage);
	void UnregisterQuery_NewUObject(UE::Editor::DataStorage::ICoreProvider& DataStorage);
	void CleanupTypedElementColumns(UE::Editor::DataStorage::ICoreProvider& DataStorage);
	void HandleOnEnabled(IConsoleVariable* CVar);
	
	UE::Editor::DataStorage::QueryHandle RemoveTypedElementRowHandleQuery = UE::Editor::DataStorage::InvalidQueryHandle;
	FDelegateHandle DebugEnabledDelegateHandle;
};
