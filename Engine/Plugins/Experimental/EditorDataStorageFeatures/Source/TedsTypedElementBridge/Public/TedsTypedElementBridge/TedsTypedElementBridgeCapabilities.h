// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"

namespace UE::Editor::DataStorage::Compatibility
{
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnTypedElementBridgeEnabled, bool /*bEnabled*/);

	TEDSTYPEDELEMENTBRIDGE_API FOnTypedElementBridgeEnabled& OnTypedElementBridgeEnabled();
	TEDSTYPEDELEMENTBRIDGE_API bool IsTypedElementBridgeEnabled();
} // namespace UE::Editor::DataStorage::Compatibility
