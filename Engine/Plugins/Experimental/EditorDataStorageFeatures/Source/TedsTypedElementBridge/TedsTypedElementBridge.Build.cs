// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class TedsTypedElementBridge : ModuleRules
{
	public TedsTypedElementBridge(ReadOnlyTargetRules Target) : base(Target)
	{
		// Enable truncation warnings in this module
		CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Warning;

		if (Target.bBuildEditor)
		{
			PublicIncludePaths.AddRange(new string[] {});
			PrivateIncludePaths.AddRange(new string[] {});

			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"TypedElementFramework",
					"TedsQueryStack"
				});

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
					"Engine",
					"InputCore",
					"Slate",
					"SlateCore",
					"WorkspaceMenuStructure",
					"ToolWidgets",
					"TedsActorCompatibility"
				});
		}
	}
}