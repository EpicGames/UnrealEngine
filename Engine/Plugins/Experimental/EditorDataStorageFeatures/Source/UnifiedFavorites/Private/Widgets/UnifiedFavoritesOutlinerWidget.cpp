// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/UnifiedFavoritesOutlinerWidget.h"

#include "Columns/SlateHeaderColumns.h"
#include "Elements/Columns/TypedElementCompatibilityColumns.h"
#include "Elements/Columns/TypedElementFolderColumns.h"
#include "Elements/Columns/TypedElementMiscColumns.h"
#include "Styling/AppStyle.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SBox.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(UnifiedFavoritesOutlinerWidget)

#define LOCTEXT_NAMESPACE "UnifiedFavoritesOutlinerWidget"

//
// Outliner Favorite Header Widget
//

FOutlinerFavoriteHeaderConstructor::FOutlinerFavoriteHeaderConstructor()
	: FSimpleWidgetConstructor(StaticStruct())
{
}

TSharedPtr<SWidget> FOutlinerFavoriteHeaderConstructor::CreateWidget(
	UE::Editor::DataStorage::ICoreProvider* DataStorage,
	UE::Editor::DataStorage::IUiProvider* DataStorageUi,
	UE::Editor::DataStorage::RowHandle TargetRow,
	UE::Editor::DataStorage::RowHandle WidgetRow,
	const UE::Editor::DataStorage::FMetaDataView& Arguments)
{
	using namespace UE::Editor::DataStorage;
	DataStorage->AddColumn(WidgetRow, FHeaderWidgetSizeColumn
		{
			.ColumnSizeMode = EColumnSizeMode::Fixed,
			.Width = 24.0f
		});

	return SNew(SBox)
		.VAlign(VAlign_Center)
		.HAlign(HAlign_Center)
		.HeightOverride(18.f)
		.WidthOverride(18.f)
		[
			SNew(SImage)
			.ColorAndOpacity(FAppStyle::Get().GetSlateColor("Colors.Foreground"))
			.Image(FAppStyle::Get().GetBrush("Icons.Star"))
			.ToolTipText(LOCTEXT("OutlinerFavoriteHeaderTooltip", "Favorite"))
		];
}

//
// Outliner Factory
//

void UUnifiedFavoritesOutlinerFactory::RegisterWidgetConstructors(UE::Editor::DataStorage::ICoreProvider& DataStorage,
	UE::Editor::DataStorage::IUiProvider& DataStorageUi) const
{
	using namespace UE::Editor::DataStorage;
	using namespace UE::Editor::DataStorage::Queries;

	FConditions OutlinerColumns = TColumn<FFavoriteTag>() || TColumn<FTypedElementUObjectColumn>() || TColumn<FFolderTag>();

	DataStorageUi.RegisterWidgetFactory<FOutlinerFavoriteHeaderConstructor>(
		DataStorageUi.FindPurpose(IUiProvider::FPurposeInfo("SceneOutliner", "Header", NAME_None).GeneratePurposeID()), OutlinerColumns);
}

#undef LOCTEXT_NAMESPACE
