// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/Interfaces/TypedElementDataStorageFactory.h"
#include "Elements/Interfaces/TypedElementDataStorageUiInterface.h"

#include "UnifiedFavoritesOutlinerWidget.generated.h"

// Outliner-specific header override
USTRUCT()
struct FOutlinerFavoriteHeaderConstructor : public FSimpleWidgetConstructor
{
	GENERATED_BODY()

public:
	FOutlinerFavoriteHeaderConstructor();
	~FOutlinerFavoriteHeaderConstructor() override = default;

	virtual TSharedPtr<SWidget> CreateWidget(
		UE::Editor::DataStorage::ICoreProvider* DataStorage,
		UE::Editor::DataStorage::IUiProvider* DataStorageUi,
		UE::Editor::DataStorage::RowHandle TargetRow,
		UE::Editor::DataStorage::RowHandle WidgetRow,
		const UE::Editor::DataStorage::FMetaDataView& Arguments) override;
};

// Factory — registers only the Outliner header override
UCLASS()
class UUnifiedFavoritesOutlinerFactory : public UEditorDataStorageFactory
{
	GENERATED_BODY()

public:
	~UUnifiedFavoritesOutlinerFactory() override = default;

	virtual void RegisterWidgetConstructors(
		UE::Editor::DataStorage::ICoreProvider& DataStorage,
		UE::Editor::DataStorage::IUiProvider& DataStorageUi) const override;
};
