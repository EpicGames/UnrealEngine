// Copyright Epic Games, Inc. All Rights Reserved.

#include "UnifiedFavoritesTableWidget.h"

#include "TedsAssetDataColumns.h"
#include "UnifiedFavoritesWidget.h"
#include "Widgets/AssetDataLabelWidget.h"
#include "Columns/SlateHeaderColumns.h"
#include "Elements/Columns/TypedElementCompatibilityColumns.h"
#include "Elements/Columns/TypedElementFolderColumns.h"
#include "Elements/Columns/TypedElementMiscColumns.h"
#include "Styling/AppStyle.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Text/STextBlock.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(UnifiedFavoritesTableWidget)

#define LOCTEXT_NAMESPACE "UnifiedFavoritesTableWidget"

namespace UE::Editor::DataStorage::Private
{
	//
	// Column Sorter
	//

	class FFavoriteWidgetSorter final : public FColumnSorterInterface
	{
	public:
		virtual int32 Compare(const ICoreProvider& Storage, RowHandle Left, RowHandle Right) const override
		{
			return 0;
		}

		virtual FPrefixInfo CalculatePrefix(const ICoreProvider& Storage, RowHandle Row, uint32 ByteIndex) const override
		{
			return CreateSortPrefix(ByteIndex, static_cast<int32>(!Storage.HasColumns<FFavoriteTag>(Row)));
		}

		virtual ESortType GetSortType() const override
		{
			return ESortType::FixedSize64;
		}

		virtual FText GetShortName() const override
		{
			return LOCTEXT("FavoriteWidgetSorter", "Favorite Sorter");
		}
	};

	//
	// Helpers
	//

	TSharedRef<SWidget> MakeFavoritesWidgetContainer(const TSharedRef<SWidget>& FavoritesWidget)
	{
		return SNew(SBox)
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Center)
			.HeightOverride(18.f)
			.WidthOverride(18.f)
			[
				FavoritesWidget
			];
	}
}

//
// Table Favorite Header Widget
//

FTableFavoriteHeaderConstructor::FTableFavoriteHeaderConstructor()
	: FSimpleWidgetConstructor(StaticStruct())
{
}

TSharedPtr<SWidget> FTableFavoriteHeaderConstructor::CreateWidget(UE::Editor::DataStorage::ICoreProvider* DataStorage,
	UE::Editor::DataStorage::IUiProvider* DataStorageUi,
	UE::Editor::DataStorage::RowHandle TargetRow, UE::Editor::DataStorage::RowHandle WidgetRow, const UE::Editor::DataStorage::FMetaDataView& Arguments)
{
	using namespace UE::Editor::DataStorage;
	DataStorage->AddColumn(WidgetRow, FHeaderWidgetSizeColumn
		{
			.ColumnSizeMode = EColumnSizeMode::Fixed,
			.Width = 24.0f
		});

	return Private::MakeFavoritesWidgetContainer(
		SNew(SImage)
		.ColorAndOpacity(FAppStyle::Get().GetSlateColor("Colors.Foreground"))
		.Image(FAppStyle::Get().GetBrush("Icons.Star"))
		.ToolTipText(LOCTEXT("TableFavoriteHeaderTooltip", "Favorite"))
	);
}

//
// Table Favorite Widget
//

FTableFavoriteWidgetConstructor::FTableFavoriteWidgetConstructor()
	: FSimpleWidgetConstructor(StaticStruct())
{
}

TSharedPtr<SWidget> FTableFavoriteWidgetConstructor::CreateWidget(
	UE::Editor::DataStorage::ICoreProvider* DataStorage,
	UE::Editor::DataStorage::IUiProvider* DataStorageUi,
	UE::Editor::DataStorage::RowHandle TargetRow,
	UE::Editor::DataStorage::RowHandle WidgetRow,
	const UE::Editor::DataStorage::FMetaDataView& Arguments)
{

	using namespace UE::Editor::DataStorage;
	if (!DataStorage->IsRowAvailable(TargetRow))
	{
		return SNew(STextBlock)
			.Text(LOCTEXT("MissingRowReferenceColumn", "Unable to retrieve row reference."));
	}

	return Private::MakeFavoritesWidgetContainer(
		SNew(SUnifiedFavoritesWidget, DataStorage)
		.RowHandle(TargetRow)
	);
}

TArray<TSharedPtr<const UE::Editor::DataStorage::FColumnSorterInterface>> FTableFavoriteWidgetConstructor::ConstructColumnSorters(
	UE::Editor::DataStorage::ICoreProvider* DataStorage, UE::Editor::DataStorage::IUiProvider* DataStorageUi,
	const UE::Editor::DataStorage::FMetaDataView& Arguments)
{
	using namespace UE::Editor::DataStorage;
	return TArray<TSharedPtr<const FColumnSorterInterface>>(
		{
			MakeShared<Private::FFavoriteWidgetSorter>()
		});
}

//
// Table Factory
//

void UUnifiedFavoritesTableFactory::RegisterWidgetConstructors(UE::Editor::DataStorage::ICoreProvider& DataStorage,
	UE::Editor::DataStorage::IUiProvider& DataStorageUi) const
{
	using namespace UE::Editor::DataStorage;
	using namespace UE::Editor::DataStorage::Queries;

	// Outliner
	FConditions OutlinerColumns = TColumn<FFavoriteTag>() || TColumn<FTypedElementUObjectColumn>() || TColumn<FFolderTag>();

	DataStorageUi.RegisterWidgetFactory<FTableFavoriteWidgetConstructor>(
		DataStorageUi.FindPurpose(IUiProvider::FPurposeInfo("SceneOutliner", "Cell", NAME_None).GeneratePurposeID()), OutlinerColumns);
}

#undef LOCTEXT_NAMESPACE
