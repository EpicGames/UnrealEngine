// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/Interfaces/TypedElementDataStorageFactory.h"
#include "Elements/Interfaces/TypedElementDataStorageUiInterface.h"

#include "UnifiedFavoritesTableWidget.generated.h"

//
// Table Constructors
//

USTRUCT()
struct FTableFavoriteWidgetConstructor : public FSimpleWidgetConstructor
{
	GENERATED_BODY()

public:
	FTableFavoriteWidgetConstructor();
	~FTableFavoriteWidgetConstructor() override = default;

	virtual TSharedPtr<SWidget> CreateWidget(
		UE::Editor::DataStorage::ICoreProvider* DataStorage,
		UE::Editor::DataStorage::IUiProvider* DataStorageUi,
		UE::Editor::DataStorage::RowHandle TargetRow,
		UE::Editor::DataStorage::RowHandle WidgetRow,
		const UE::Editor::DataStorage::FMetaDataView& Arguments) override;
	virtual TArray<TSharedPtr<const UE::Editor::DataStorage::FColumnSorterInterface>> ConstructColumnSorters(
		UE::Editor::DataStorage::ICoreProvider* DataStorage, UE::Editor::DataStorage::IUiProvider* DataStorageUi,
		const UE::Editor::DataStorage::FMetaDataView& Arguments) override;
};

USTRUCT()
struct FTableFavoriteHeaderConstructor : public FSimpleWidgetConstructor
{
	GENERATED_BODY()

public:
	FTableFavoriteHeaderConstructor();
	~FTableFavoriteHeaderConstructor() override = default;

	virtual TSharedPtr<SWidget> CreateWidget(
		UE::Editor::DataStorage::ICoreProvider* DataStorage,
		UE::Editor::DataStorage::IUiProvider* DataStorageUi,
		UE::Editor::DataStorage::RowHandle TargetRow,
		UE::Editor::DataStorage::RowHandle WidgetRow,
		const UE::Editor::DataStorage::FMetaDataView& Arguments) override;
};

//
// Factory
//

UCLASS()
class UUnifiedFavoritesTableFactory : public UEditorDataStorageFactory
{
	GENERATED_BODY()

public:
	~UUnifiedFavoritesTableFactory() override = default;

	virtual void RegisterWidgetConstructors(UE::Editor::DataStorage::ICoreProvider& DataStorage, UE::Editor::DataStorage::IUiProvider& DataStorageUi) const override;
};
