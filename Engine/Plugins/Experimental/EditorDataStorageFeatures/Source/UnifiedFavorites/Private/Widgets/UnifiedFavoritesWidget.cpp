// Copyright Epic Games, Inc. All Rights Reserved.

#include "UnifiedFavoritesWidget.h"

#include "Elements/Columns/TypedElementMiscColumns.h"

namespace UE::Editor::DataStorage
{
	void SUnifiedFavoritesWidget::Construct(const FArguments& InArgs, ICoreProvider* InDataStorage)
	{
		check(InDataStorage);
		DataStorage = InDataStorage;
		TargetRow = InArgs._RowHandle;

		FavoriteBrush    = FAppStyle::GetBrush("Icons.Star");
		NotFavoriteBrush = FAppStyle::GetBrush("PropertyWindow.Favorites_Disabled");

		ActiveColor = InArgs._ActiveColor;
		NotActiveColor = InArgs._NotActiveColor;

		SImage::Construct(
			SImage::FArguments()
			.ColorAndOpacity(this, &SUnifiedFavoritesWidget::GetForegroundColor)
			.Image(this, &SUnifiedFavoritesWidget::GetBrush)
		);
	}


	bool SUnifiedFavoritesWidget::IsFavorite() const
	{
		return DataStorage->HasColumns<FFavoriteTag>(TargetRow);
	}


	void SUnifiedFavoritesWidget::ToggleFavorite() const
	{
		if (!IsInteractable())
		{
			return;
		}

		DataStorage->AddColumn<FTypedElementSyncFromWorldTag>(TargetRow);

		if (DataStorage->HasColumns<FFavoriteTag>(TargetRow))
		{
			DataStorage->RemoveColumn<FFavoriteTag>(TargetRow);
		}
		else
		{
			DataStorage->AddColumn<FFavoriteTag>(TargetRow);
		}
	}

	bool SUnifiedFavoritesWidget::IsInteractable() const
	{
		return DataStorage->IsRowAvailable(TargetRow);
	}

	void SUnifiedFavoritesWidget::SetTargetRow(RowHandle InTargetRow)
	{
		TargetRow = InTargetRow;
	}

	FReply SUnifiedFavoritesWidget::HandleClick() const
	{
		if (!IsEnabled())
		{
			return FReply::Unhandled();
		}

		ToggleFavorite();

		return FReply::Handled();
	}

	FReply SUnifiedFavoritesWidget::OnMouseButtonDoubleClick(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent)
	{
		return HandleClick();
	}

	FReply SUnifiedFavoritesWidget::OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
	{
		if (MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton)
		{
			return HandleClick();
		}

		return FReply::Unhandled();
	}

	FReply SUnifiedFavoritesWidget::OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
	{
		if (MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton)
		{
			return FReply::Handled();
		}

		return FReply::Unhandled();
	}

	const FSlateBrush* SUnifiedFavoritesWidget::GetBrush() const
	{
		return IsFavorite() ? FavoriteBrush : NotFavoriteBrush;
	}

	FSlateColor SUnifiedFavoritesWidget::GetForegroundColor() const
	{
		if (!IsInteractable())
		{
			return FLinearColor::Transparent;
		}

		if (!IsFavorite() && !IsHovered())
		{
			return NotActiveColor;
		}

		FLinearColor Color = ActiveColor.GetSpecifiedColor();

		if (IsHovered())
		{
			constexpr float Darken = .8f;
			Color *= Darken;
		}

		return Color;
	}
}
