// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/Interfaces/TypedElementDataStorageInterface.h"
#include "Widgets/Images/SImage.h"


namespace UE::Editor::DataStorage
{
	/** Widget responsible for managing the favorite state of a TEDS row handle */
	class SUnifiedFavoritesWidget final : public SImage
	{
	public:
		SLATE_BEGIN_ARGS(SUnifiedFavoritesWidget) :
			_ActiveColor(FAppStyle::Get().GetSlateColor("Colors.Foreground")),
			_NotActiveColor(FAppStyle::Get().GetSlateColor("Colors.Transparent"))
			{
			}
			SLATE_ARGUMENT(FSlateColor, ActiveColor)
			SLATE_ARGUMENT(FSlateColor, NotActiveColor)
			SLATE_ARGUMENT(UE::Editor::DataStorage::RowHandle, RowHandle)
		SLATE_END_ARGS()

			/** Construct this widget */
			void Construct(const FArguments& InArgs, ICoreProvider* InDataStorage);

			virtual bool IsInteractable() const override;

			/** Directly sets the resolved target row (called by the mapping resolution processor). */
			void SetTargetRow(RowHandle InTargetRow);

	private:
		bool IsFavorite() const;

		void ToggleFavorite() const;

		FReply HandleClick() const;

		virtual FReply OnMouseButtonDoubleClick(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent) override;

		virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;

		virtual FReply OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;

		virtual FSlateColor GetForegroundColor() const override;

		const FSlateBrush* GetBrush() const;

	private:
		ICoreProvider* DataStorage;

		RowHandle TargetRow = InvalidRowHandle;

		FSlateColor ActiveColor;
		FSlateColor NotActiveColor;

		const FSlateBrush* FavoriteBrush;
		const FSlateBrush* NotFavoriteBrush;
	};
}
