// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class UnifiedFavorites : ModuleRules
{
	public UnifiedFavorites(ReadOnlyTargetRules Target) : base(Target)
	{
		// Enable truncation warnings in this module
		CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Warning;

		if (Target.bBuildEditor)
		{
			PublicIncludePaths.AddRange(new string[] {});
			PrivateIncludePaths.AddRange(new string[] {});

			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
					"EditorFramework",
					"Engine",
					"SceneOutliner",
					"Slate",
					"SlateCore",
					"TypedElementFramework",
					"TypedElementRuntime"
				});
				
			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"WorkspaceMenuStructure",
					"ToolMenus",
					"ApplicationCore",
					"InputCore",
					"TedsQueryStack",
					"TedsActorCompatibility",
					"TedsOperations",
					"LevelEditor",
					"LevelInstanceEditor",
					"TedsOutliner",
					"ToolWidgets",
					"TedsTableViewer",
					"ContentBrowser",
					"TedsAssetData"
				});
		}
	}
}